// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class AnimGraphRuntime : ModuleRules
{
	public AnimGraphRuntime(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] { 
				"Core", 
				"CoreUObject", 
				"Engine",
                "AnimationCore",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"TraceLog",
			}
		);

        SetupModulePhysicsSupport(Target);

		// External users of this library do not need to know about Eigen.
        AddEngineThirdPartyPrivateStaticDependencies(Target,
                "Eigen"
                );

        PublicDependencyModuleNames.AddRange(
            new string[] {
				"GeometryCollectionEngine",
            }
        );
    }
}
