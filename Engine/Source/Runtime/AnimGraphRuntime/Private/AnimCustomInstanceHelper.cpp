// Copyright Epic Games, Inc. All Rights Reserved.


#include "AnimCustomInstanceHelper.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"

/////////////////////////////////////////////////////
// FAnimCustomInstanceHelper
/////////////////////////////////////////////////////


bool FAnimCustomInstanceHelper::ShouldCreateCustomInstancePlayer(const USkeletalMeshComponent* SkeletalMeshComponent)
{
	const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
	const USkeleton*     Skeleton     = SkeletalMesh ? SkeletalMesh->GetSkeleton() : nullptr;

	// create proper anim instance to animate
	UAnimInstance* AnimInstance = SkeletalMeshComponent->GetAnimInstance();

	return (AnimInstance == nullptr || SkeletalMeshComponent->GetAnimationMode() != EAnimationMode::AnimationBlueprint ||
		AnimInstance->GetClass() != SkeletalMeshComponent->AnimClass || !Skeleton || !AnimInstance->CurrentSkeleton);
}