// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimExecutionContextLibrary.h"
#include "Animation/AnimExecutionContext.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimClassInterface.h"
#include "Animation/AnimInstanceProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimExecutionContextLibrary)

UAnimInstance* UAnimExecutionContextLibrary::GetAnimInstance(const FAnimExecutionContext& Context)
{
	return CastChecked<UAnimInstance>(Context.GetBaseContext()->GetAnimInstanceObject());
}

FAnimNodeReference UAnimExecutionContextLibrary::GetAnimNodeReference(UAnimInstance* Instance, int32 Index)
{
	IAnimClassInterface* AnimClassInterface = IAnimClassInterface::GetFromClass(Instance->GetClass());
	const TArray<FStructProperty*>& AnimNodeProperties = AnimClassInterface->GetAnimNodeProperties();

	// As the index is patched during compilation, it needs to be reversed here
	int32 ReverseIndex = AnimNodeProperties.Num() - 1 - Index;
	return FAnimNodeReference(Instance, ReverseIndex);
}

FAnimInitializationContext UAnimExecutionContextLibrary::ConvertToInitializationContext(const FAnimExecutionContext& Context, EAnimExecutionContextConversionResult& Result)
{
	return FAnimExecutionContext::ConvertToType<FAnimInitializationContext>(Context, Result);
}

FAnimUpdateContext UAnimExecutionContextLibrary::ConvertToUpdateContext(const FAnimExecutionContext& Context, EAnimExecutionContextConversionResult& Result)
{
	return FAnimExecutionContext::ConvertToType<FAnimUpdateContext>(Context, Result);
}

float UAnimExecutionContextLibrary::GetDeltaTime(const FAnimUpdateContext& UpdateContext)
{
	if(FAnimationUpdateContext* Context = UpdateContext.GetContext())
	{
		return Context->GetDeltaTime();
	}

	return 0.0f;
}

float UAnimExecutionContextLibrary::GetCurrentWeight(const FAnimUpdateContext& UpdateContext)
{
	if(FAnimationUpdateContext* Context = UpdateContext.GetContext())
	{
		return Context->GetFinalBlendWeight();
	}

	return 0.0f;
}

bool UAnimExecutionContextLibrary::IsActive(const FAnimExecutionContext& Context)
{
	FAnimationBaseContext* BaseContext = Context.GetBaseContext();
	check(BaseContext);
	return BaseContext->IsActive();
}

FAnimPoseContext UAnimExecutionContextLibrary::ConvertToPoseContext(const FAnimExecutionContext& Context, EAnimExecutionContextConversionResult& Result)
{
	return FAnimExecutionContext::ConvertToType<FAnimPoseContext>(Context, Result);
}

FAnimComponentSpacePoseContext UAnimExecutionContextLibrary::ConvertToComponentSpacePoseContext(const FAnimExecutionContext& Context, EAnimExecutionContextConversionResult& Result)
{
	return FAnimExecutionContext::ConvertToType<FAnimComponentSpacePoseContext>(Context, Result);
}
