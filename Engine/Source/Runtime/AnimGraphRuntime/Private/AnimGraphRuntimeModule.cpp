// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "HAL/IConsoleManager.h"
#include "Modules/ModuleManager.h"
#include "BoneControllers/AnimNode_AnimDynamics.h"
#include "UObject/UObjectIterator.h"
#include "Animation/AnimInstance.h"
#include "Animation/AttributeTypes.h"
#include "Algo/Sort.h"
#include "Animation/AnimBlueprintGeneratedClass.h"

//////////////////////////////////////////////////////////////////////////
// FAnimGraphRuntimeModule

class FAnimGraphRuntimeModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};

static FAutoConsoleCommand AuditLoadedAnimGraphs(
	TEXT("a.AuditLoadedAnimGraphs"),
	TEXT("Audit memory breakdown of currently loaded anim graphs. Writes results to the log."), 
	FConsoleCommandDelegate::CreateLambda([]()
	{
		UE_LOGF(LogAnimation, Display, "--- BEGIN audit all loaded anim graphs ---");
		UE_LOGF(LogAnimation, Display, "NodeType, Size (B), Count, TotalSize (B)");

		TMap<UScriptStruct*, uint64> PerNodeTypeCounts;
		TSet<UAnimBlueprintGeneratedClass*> Classes;

		int32 NumAnimInstances = 0;

		for(TObjectIterator<UAnimInstance> It; It; ++It)
		{
			NumAnimInstances++;
			
			UAnimInstance* AnimInstance = *It;
			if(UAnimBlueprintGeneratedClass* Class = Cast<UAnimBlueprintGeneratedClass>(AnimInstance->GetClass()))
			{
				Classes.Add(Class);
				
				for(FStructProperty* NodeStructProperty : Class->AnimNodeProperties)
				{
					if(uint64* CountPtr = PerNodeTypeCounts.Find(NodeStructProperty->Struct))
					{
						(*CountPtr)++;
					}
					else
					{
						PerNodeTypeCounts.Add(NodeStructProperty->Struct, 1);
					}
				}
				
			}
		}

		uint64 TotalAnimGraphMemory = 0;

		TArray<TPair<UScriptStruct*, uint64>> SortedPerNodeTypeCounts;
		SortedPerNodeTypeCounts.Reserve(PerNodeTypeCounts.Num());
		for(const TPair<UScriptStruct*, uint64> NodeTypeCountPair : PerNodeTypeCounts)
		{
			SortedPerNodeTypeCounts.Add(NodeTypeCountPair);
		}
		Algo::Sort(SortedPerNodeTypeCounts, [](const TPair<UScriptStruct*, uint64>& InEntry0, const TPair<UScriptStruct*, uint64>& InEntry1)
		{
			const UScriptStruct* NodeStruct0 = InEntry0.Key;
			const UScriptStruct* NodeStruct1 = InEntry1.Key;
			return NodeStruct0->GetName() < NodeStruct1->GetName();
		});
				
		for(const TPair<UScriptStruct*, uint64> NodeTypeCountPair : SortedPerNodeTypeCounts)
		{
			const UScriptStruct* NodeStruct = NodeTypeCountPair.Key;
			const uint64 Count = NodeTypeCountPair.Value;
			const uint64 Size = NodeStruct->GetStructureSize();

			TotalAnimGraphMemory += Count * Size;
					
			UE_LOGF(LogAnimation, Display, "%ls, %llu, %llu, %llu", *NodeStruct->GetName(), Size, Count, Size * Count);
		}

		UE_LOGF(LogAnimation, Display, "Total anim graph mem, %llu Bytes, (%llu KB)", TotalAnimGraphMemory, TotalAnimGraphMemory / 1024ull);
		UE_LOGF(LogAnimation, Display, "UAnimInstance count, %d", NumAnimInstances);
		UE_LOGF(LogAnimation, Display, "Unique UAnimInstance classes, %d", Classes.Num());	

		UE_LOGF(LogAnimation, Display, "--- END audit all loaded anim graphs ---");
	}),
	ECVF_Cheat);

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FAnimGraphRuntimeModule, AnimGraphRuntime);
