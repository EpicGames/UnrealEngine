// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphRuntimeTrace.h"

#if ANIM_TRACE_ENABLED

#include "ObjectTrace.h"
#include "Trace/Trace.inl"
#include "Animation/BlendSpace.h"
#include "AnimNodes/AnimNode_BlendSpacePlayer.h"
#include "AnimNodes/AnimNode_BlendSpaceGraphBase.h"
#include "Animation/AnimInstanceProxy.h"

UE_TRACE_EVENT_BEGIN(Animation, BlendSpacePlayer)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, AnimInstanceId)
	UE_TRACE_EVENT_FIELD(uint64, BlendSpaceId)
	UE_TRACE_EVENT_FIELD(int32, NodeId)
	UE_TRACE_EVENT_FIELD(float, PositionX)
	UE_TRACE_EVENT_FIELD(float, PositionY)
	UE_TRACE_EVENT_FIELD(float, PositionZ)
	UE_TRACE_EVENT_FIELD(float, FilteredPositionX)
	UE_TRACE_EVENT_FIELD(float, FilteredPositionY)
	UE_TRACE_EVENT_FIELD(float, FilteredPositionZ)
	UE_TRACE_EVENT_END()

void FAnimGraphRuntimeTrace::OutputBlendSpacePlayer(const FAnimationBaseContext& InContext, const FAnimNode_BlendSpacePlayerBase& InNode)
{
	bool bEventEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bEventEnabled)
	{
		return;
	}

	check(InContext.AnimInstanceProxy);

	TRACE_OBJECT(InContext.AnimInstanceProxy->GetAnimInstanceObject());
	TRACE_OBJECT(InNode.GetBlendSpace());

	FVector SampleCoordinates = InNode.GetPosition();
	FVector FilteredPosition = InNode.GetFilteredPosition();

	UE_TRACE_LOG(Animation, BlendSpacePlayer, AnimationChannel)
		<< BlendSpacePlayer.Cycle(FPlatformTime::Cycles64())
		<< BlendSpacePlayer.AnimInstanceId(FObjectTrace::GetObjectId(InContext.AnimInstanceProxy->GetAnimInstanceObject()))
		<< BlendSpacePlayer.BlendSpaceId(FObjectTrace::GetObjectId(InNode.GetBlendSpace()))
		<< BlendSpacePlayer.NodeId(InContext.GetCurrentNodeId())
		<< BlendSpacePlayer.PositionX(static_cast<float>(SampleCoordinates.X))
		<< BlendSpacePlayer.PositionY(static_cast<float>(SampleCoordinates.Y))
		<< BlendSpacePlayer.PositionZ(static_cast<float>(SampleCoordinates.Z))
		<< BlendSpacePlayer.FilteredPositionX(static_cast<float>(FilteredPosition.X))
		<< BlendSpacePlayer.FilteredPositionY(static_cast<float>(FilteredPosition.Y))
		<< BlendSpacePlayer.FilteredPositionZ(static_cast<float>(FilteredPosition.Z));
}

void FAnimGraphRuntimeTrace::OutputBlendSpacePlayerValues(
	const FAnimationBaseContext& InContext,
	int32 InNodeId,
	const UBlendSpace* InBlendSpace,
	FVector InPosition,
	FVector InFilteredPosition)
{
	if (!UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel))
	{
		return;
	}

	check(InContext.AnimInstanceProxy);

	TRACE_OBJECT(InContext.AnimInstanceProxy->GetAnimInstanceObject());
	TRACE_OBJECT(InBlendSpace);

	UE_TRACE_LOG(Animation, BlendSpacePlayer, AnimationChannel)
		<< BlendSpacePlayer.Cycle(FPlatformTime::Cycles64())
		<< BlendSpacePlayer.AnimInstanceId(FObjectTrace::GetObjectId(InContext.AnimInstanceProxy->GetAnimInstanceObject()))
		<< BlendSpacePlayer.BlendSpaceId(FObjectTrace::GetObjectId(InBlendSpace))
		<< BlendSpacePlayer.NodeId(InNodeId)
		<< BlendSpacePlayer.PositionX(static_cast<float>(InPosition.X))
		<< BlendSpacePlayer.PositionY(static_cast<float>(InPosition.Y))
		<< BlendSpacePlayer.PositionZ(static_cast<float>(InPosition.Z))
		<< BlendSpacePlayer.FilteredPositionX(static_cast<float>(InFilteredPosition.X))
		<< BlendSpacePlayer.FilteredPositionY(static_cast<float>(InFilteredPosition.Y))
		<< BlendSpacePlayer.FilteredPositionZ(static_cast<float>(InFilteredPosition.Z));
}

void FAnimGraphRuntimeTrace::OutputBlendSpace(const FAnimationBaseContext& InContext, const FAnimNode_BlendSpaceGraphBase& InNode)
{
	bool bEventEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AnimationChannel);
	if (!bEventEnabled)
	{
		return;
	}

	check(InContext.AnimInstanceProxy);

	TRACE_OBJECT(InContext.AnimInstanceProxy->GetAnimInstanceObject());
	TRACE_OBJECT(InNode.GetBlendSpace());

	FVector Coordinates = InNode.GetPosition();
	FVector FilteredCoordinates = InNode.GetFilteredPosition();

	UE_TRACE_LOG(Animation, BlendSpacePlayer, AnimationChannel)
		<< BlendSpacePlayer.Cycle(FPlatformTime::Cycles64())
		<< BlendSpacePlayer.AnimInstanceId(FObjectTrace::GetObjectId(InContext.AnimInstanceProxy->GetAnimInstanceObject()))
		<< BlendSpacePlayer.BlendSpaceId(FObjectTrace::GetObjectId(InNode.GetBlendSpace()))
		<< BlendSpacePlayer.NodeId(InContext.GetCurrentNodeId())
		<< BlendSpacePlayer.PositionX(static_cast<float>(Coordinates.X))
		<< BlendSpacePlayer.PositionY(static_cast<float>(Coordinates.Y))
		<< BlendSpacePlayer.PositionZ(static_cast<float>(Coordinates.Z))
		<< BlendSpacePlayer.FilteredPositionX(static_cast<float>(FilteredCoordinates.X))
		<< BlendSpacePlayer.FilteredPositionY(static_cast<float>(FilteredCoordinates.Y))
		<< BlendSpacePlayer.FilteredPositionZ(static_cast<float>(FilteredCoordinates.Z));
}

#endif