// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_AimOffsetLookAt.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimStats.h"
#include "AnimationRuntime.h"
#include "Animation/BlendSpace.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshSocket.h"
#include "DrawDebugHelpers.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_AimOffsetLookAt)

TAutoConsoleVariable<int32> CVarAimOffsetLookAtEnable(TEXT("a.AnimNode.AimOffsetLookAt.Enable"), 1, TEXT("Enable/Disable LookAt AimOffset"));
TAutoConsoleVariable<int32> CVarAimOffsetLookAtDebug(TEXT("a.AnimNode.AimOffsetLookAt.Debug"), 0, TEXT("Toggle LookAt AimOffset debug"));

/////////////////////////////////////////////////////
// FAnimNode_AimOffsetLookAt

void FAnimNode_AimOffsetLookAt::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_BlendSpacePlayer::Initialize_AnyThread(Context);
	BasePose.Initialize(Context);
}

void FAnimNode_AimOffsetLookAt::OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance)
{
	FAnimNode_BlendSpacePlayer::OnInitializeAnimInstance(InProxy, InAnimInstance);

	SocketBoneReference.BoneName = NAME_None;
	if (USkeletalMeshComponent* SkelMeshComp = InAnimInstance->GetSkelMeshComponent())
	{
		if (USkeletalMesh* SkelMesh = SkelMeshComp->GetSkeletalMeshAsset())
		{
			if (const USkeletalMeshSocket* Socket = SkelMesh->FindSocket(SourceSocketName))
			{
				SocketLocalTransform = Socket->GetSocketLocalTransform();
				SocketBoneReference.BoneName = Socket->BoneName;
			}
			else if (SkelMeshComp->GetBoneIndex(SourceSocketName) != INDEX_NONE)
			{
				SocketLocalTransform.SetIdentity();
				SocketBoneReference.BoneName = SourceSocketName;
			}

			if (const USkeletalMeshSocket* Socket = SkelMesh->FindSocket(PivotSocketName))
			{
				PivotSocketLocalTransform = Socket->GetSocketLocalTransform();
				PivotSocketBoneReference.BoneName = Socket->BoneName;
			}
			else if (SkelMeshComp->GetBoneIndex(PivotSocketName) != INDEX_NONE)
			{
				PivotSocketLocalTransform.SetIdentity();
				PivotSocketBoneReference.BoneName = PivotSocketName;
			}
		}
	}
}

void FAnimNode_AimOffsetLookAt::UpdateAssetPlayer(const FAnimationUpdateContext& Context)
{
	bIsLODEnabled = IsLODEnabled(Context.AnimInstanceProxy);
 	if (bIsLODEnabled)
 	{
 		FAnimNode_BlendSpacePlayer::UpdateAssetPlayer(Context);
 	}

	BasePose.Update(Context);

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Playback Time"), InternalTimeAccumulator);
}

void FAnimNode_AimOffsetLookAt::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	FAnimNode_BlendSpacePlayer::CacheBones_AnyThread(Context);
	BasePose.CacheBones(Context);

	SocketBoneReference.Initialize(Context.AnimInstanceProxy->GetRequiredBones());
	PivotSocketBoneReference.Initialize(Context.AnimInstanceProxy->GetRequiredBones());
}

void FAnimNode_AimOffsetLookAt::Evaluate_AnyThread(FPoseContext& Context)
{
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(AimOffsetLookAt, !IsInGameThread());

	// Evaluate base pose
	BasePose.Evaluate(Context);

	if (bIsLODEnabled && FAnimWeight::IsRelevant(Alpha) && (CVarAimOffsetLookAtEnable.GetValueOnAnyThread() == 1))
	{
		UpdateFromLookAtTarget(Context);

		// Evaluate MeshSpaceRotation additive blendspace
		FPoseContext MeshSpaceRotationAdditivePoseContext(Context);
		FAnimNode_BlendSpacePlayer::Evaluate_AnyThread(MeshSpaceRotationAdditivePoseContext);

		// Accumulate poses together
		FAnimationPoseData BaseAnimationPoseData(Context);
		const FAnimationPoseData AdditiveAnimationPoseData(MeshSpaceRotationAdditivePoseContext);
		FAnimationRuntime::AccumulateMeshSpaceRotationAdditiveToLocalPose(BaseAnimationPoseData, AdditiveAnimationPoseData, Alpha);

		// Resulting rotations are not normalized, so normalize here.
		Context.Pose.NormalizeRotations();
	}
}

void FAnimNode_AimOffsetLookAt::UpdateFromLookAtTarget(FPoseContext& LocalPoseContext)
{
	UBlendSpace* CurrentBlendSpace = GetBlendSpace();

	const FBoneContainer& RequiredBones = LocalPoseContext.Pose.GetBoneContainer();
	if (CurrentBlendSpace && SocketBoneReference.IsValidToEvaluate(RequiredBones))
	{
		FCSPose<FCompactPose> GlobalPose;
		GlobalPose.InitPose(LocalPoseContext.Pose);

		const FCompactPoseBoneIndex SocketBoneIndex = SocketBoneReference.GetCompactPoseIndex(RequiredBones);
		const FTransform BoneTransform = GlobalPose.GetComponentSpaceTransform(SocketBoneIndex);

		FTransform SourceComponentTransform = SocketLocalTransform * BoneTransform;
		if (PivotSocketBoneReference.IsValidToEvaluate(RequiredBones))
		{
			const FCompactPoseBoneIndex PivotSocketBoneIndex = PivotSocketBoneReference.GetCompactPoseIndex(RequiredBones);
			const FTransform PivotBoneComponentTransform = PivotSocketLocalTransform * GlobalPose.GetComponentSpaceTransform(PivotSocketBoneIndex);

			SourceComponentTransform.SetTranslation(PivotBoneComponentTransform.GetTranslation());
		}

		FAnimInstanceProxy* AnimProxy = LocalPoseContext.AnimInstanceProxy;
		check(AnimProxy);
		const FTransform SourceWorldTransform = SourceComponentTransform * AnimProxy->GetComponentTransform();
		const FTransform ActorTransform = AnimProxy->GetActorTransform();

		// Convert Target to Actor Space
		const FTransform TargetWorldTransform(LookAtLocation);

		const FVector DirectionToTarget = ActorTransform.InverseTransformVectorNoScale(TargetWorldTransform.GetLocation() - SourceWorldTransform.GetLocation()).GetSafeNormal();
		const FVector CurrentDirection = ActorTransform.InverseTransformVectorNoScale(SourceWorldTransform.TransformVector(SocketAxis).GetSafeNormal());

		const FVector AxisX = FVector::ForwardVector;
		const FVector AxisY = FVector::RightVector;
		const FVector AxisZ = FVector::UpVector;

		const FVector2D CurrentCoords = FMath::GetAzimuthAndElevation(CurrentDirection, AxisX, AxisY, AxisZ);
		const FVector2D TargetCoords = FMath::GetAzimuthAndElevation(DirectionToTarget, AxisX, AxisY, AxisZ);
		CurrentBlendInput.X = FRotator::NormalizeAxis(FMath::RadiansToDegrees(TargetCoords.X - CurrentCoords.X));
		CurrentBlendInput.Y = FRotator::NormalizeAxis(FMath::RadiansToDegrees(TargetCoords.Y - CurrentCoords.Y));

#if ENABLE_DRAW_DEBUG
		if (CVarAimOffsetLookAtDebug.GetValueOnAnyThread() == 1)
		{
			AnimProxy->AnimDrawDebugLine(SourceWorldTransform.GetLocation(), TargetWorldTransform.GetLocation(), FColor::Green);
			AnimProxy->AnimDrawDebugLine(SourceWorldTransform.GetLocation(), SourceWorldTransform.GetLocation() + SourceWorldTransform.TransformVector(SocketAxis).GetSafeNormal() * (TargetWorldTransform.GetLocation() - SourceWorldTransform.GetLocation()).Size(), FColor::Red);
			AnimProxy->AnimDrawDebugCoordinateSystem(ActorTransform.GetLocation(), ActorTransform.GetRotation().Rotator(), 100.f);

			FString DebugString = FString::Printf(TEXT("Socket (X:%f, Y:%f), Target (X:%f, Y:%f), Result (X:%f, Y:%f)")
				, FMath::RadiansToDegrees(CurrentCoords.X)
				, FMath::RadiansToDegrees(CurrentCoords.Y)
				, FMath::RadiansToDegrees(TargetCoords.X)
				, FMath::RadiansToDegrees(TargetCoords.Y)
				, CurrentBlendInput.X
				, CurrentBlendInput.Y);
			AnimProxy->AnimDrawDebugOnScreenMessage(DebugString, FColor::Red);
		}
#endif // ENABLE_DRAW_DEBUG
	}

	// Update Blend Space, including the smoothing/filtering, and put the result into BlendSampleDataCache.
	if (CurrentBlendSpace)
	{
		const FVector BlendSpacePosition(CurrentBlendInput.X, CurrentBlendInput.Y, 0.f);
		const FVector FilteredBlendInput = CurrentBlendSpace->FilterInput(
			&BlendFilter, BlendSpacePosition, DeltaTimeRecord.Delta);

		CurrentBlendSpace->UpdateBlendSamples(
			FilteredBlendInput, DeltaTimeRecord.Delta, BlendSampleDataCache, CachedTriangulationIndex);
	}
}

void FAnimNode_AimOffsetLookAt::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	DebugLine += FString::Printf(TEXT("(Play Time: %.3f)"), InternalTimeAccumulator);
	DebugData.AddDebugItem(DebugLine);

	BasePose.GatherDebugData(DebugData);
}

FVector FAnimNode_AimOffsetLookAt::GetPosition() const
{
	// Use our calculated coordinates rather than the folded values
	return CurrentBlendInput;
}

FAnimNode_AimOffsetLookAt::FAnimNode_AimOffsetLookAt()
	: SocketLocalTransform(FTransform::Identity)
	, PivotSocketLocalTransform(FTransform::Identity)
	, LODThreshold(INDEX_NONE)
	, SourceSocketName(NAME_None)
	, PivotSocketName(NAME_None)
	, LookAtLocation(ForceInitToZero)
	, SocketAxis(1.0f, 0.0f, 0.0f)
	, Alpha(1.f)
	, CurrentBlendInput(FVector::ZeroVector)
	, bIsLODEnabled(false)
{
}

