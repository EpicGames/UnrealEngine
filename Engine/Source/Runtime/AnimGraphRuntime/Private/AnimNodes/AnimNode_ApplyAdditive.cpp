// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_ApplyAdditive.h"
#include "AnimationRuntime.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_ApplyAdditive)

/////////////////////////////////////////////////////
// FAnimNode_ApplyAdditive

void FAnimNode_ApplyAdditive::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_Base::Initialize_AnyThread(Context);

	Base.Initialize(Context);
	Additive.Initialize(Context);

	AlphaBoolBlend.Reinitialize();
	AlphaScaleBiasClamp.Reinitialize();
}

void FAnimNode_ApplyAdditive::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	Base.CacheBones(Context);
	Additive.CacheBones(Context);
}

void FAnimNode_ApplyAdditive::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)
	Base.Update(Context);

	ActualAlpha = 0.f;
	if (IsLODEnabled(Context.AnimInstanceProxy))
	{
		// @note: If you derive from this class, and if you have input that you rely on for base
		// this is not going to work	
		GetEvaluateGraphExposedInputs().Execute(Context);

		switch (AlphaInputType)
		{
		case EAnimAlphaInputType::Float:
			ActualAlpha = AlphaScaleBias.ApplyTo(AlphaScaleBiasClamp.ApplyTo(Alpha, Context.GetDeltaTime()));
			break;
		case EAnimAlphaInputType::Bool:
			ActualAlpha = AlphaBoolBlend.ApplyTo(bAlphaBoolEnabled, Context.GetDeltaTime());
			break;
		case EAnimAlphaInputType::Curve:
			if (UAnimInstance* AnimInstance = Cast<UAnimInstance>(Context.AnimInstanceProxy->GetAnimInstanceObject()))
			{
				ActualAlpha = AlphaScaleBiasClamp.ApplyTo(AnimInstance->GetCurveValue(AlphaCurveName), Context.GetDeltaTime());
			}
			break;
		};

		if (FAnimWeight::IsRelevant(ActualAlpha))
		{
			Additive.Update(Context.FractionalWeight(ActualAlpha));
		}
	}

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Alpha"), ActualAlpha);
}

void FAnimNode_ApplyAdditive::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(ApplyAdditive, !IsInGameThread());

	//@TODO: Could evaluate Base into Output and save a copy
	if (FAnimWeight::IsRelevant(ActualAlpha))
	{
		const bool bExpectsAdditivePose = true;
		FPoseContext AdditiveEvalContext(Output, bExpectsAdditivePose);

		Base.Evaluate(Output);
		Additive.Evaluate(AdditiveEvalContext);

		FAnimationPoseData OutAnimationPoseData(Output);
		const FAnimationPoseData AdditiveAnimationPoseData(AdditiveEvalContext);

		FAnimationRuntime::AccumulateAdditivePose(OutAnimationPoseData, AdditiveAnimationPoseData, ActualAlpha, AAT_LocalSpaceBase);
		Output.Pose.NormalizeRotations();
	}
	else
	{
		Base.Evaluate(Output);
	}
}

FAnimNode_ApplyAdditive::FAnimNode_ApplyAdditive()
	: Alpha(1.0f)
	, LODThreshold(INDEX_NONE)
	, AlphaCurveName(NAME_None)
	, ActualAlpha(0.f)
	, AlphaInputType(EAnimAlphaInputType::Float)
	, bAlphaBoolEnabled(true)
{
}

void FAnimNode_ApplyAdditive::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("(Alpha: %.1f%%)"), ActualAlpha*100.f);

	DebugData.AddDebugItem(DebugLine);
	Base.GatherDebugData(DebugData.BranchFlow(1.f));
	Additive.GatherDebugData(DebugData.BranchFlow(ActualAlpha));
}

