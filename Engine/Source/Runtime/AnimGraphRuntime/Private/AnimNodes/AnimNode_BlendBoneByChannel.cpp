// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_BlendBoneByChannel.h"
#include "AnimationRuntime.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimTrace.h"
#include "Animation/AnimStats.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_BlendBoneByChannel)

/////////////////////////////////////////////////////
// FAnimNode_BlendBoneByChannel

void FAnimNode_BlendBoneByChannel::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_Base::Initialize_AnyThread(Context);

	A.Initialize(Context);
	B.Initialize(Context);
}

void FAnimNode_BlendBoneByChannel::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	A.CacheBones(Context);
	B.CacheBones(Context);

	// Pre-validate bone entries, so we don't waste cycles every frame figuring it out.
	ValidBoneEntries.Reset();
	const FBoneContainer& BoneContainer = Context.AnimInstanceProxy->GetRequiredBones();
	for (FBlendBoneByChannelEntry& Entry : BoneDefinitions)
	{
		Entry.SourceBone.Initialize(BoneContainer);
		Entry.TargetBone.Initialize(BoneContainer);

		if (Entry.SourceBone.IsValidToEvaluate(BoneContainer) && Entry.TargetBone.IsValidToEvaluate(BoneContainer)
			&& (Entry.bBlendTranslation || Entry.bBlendRotation || Entry.bBlendScale))
		{
			ValidBoneEntries.Add(Entry);
		}
	}
}

void FAnimNode_BlendBoneByChannel::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FAnimNode_BlendBoneByChannel_Update);
	GetEvaluateGraphExposedInputs().Execute(Context);

	InternalBlendAlpha = AlphaScaleBias.ApplyTo(Alpha);
	bBIsRelevant = FAnimWeight::IsRelevant(InternalBlendAlpha) && (ValidBoneEntries.Num() > 0);

	A.Update(Context);
	if (bBIsRelevant)
	{
		B.Update(Context.FractionalWeight(InternalBlendAlpha));
	}

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Alpha"), InternalBlendAlpha);
}

void FAnimNode_BlendBoneByChannel::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(BlendBoneByChannel, !IsInGameThread());

	A.Evaluate(Output);

	if (bBIsRelevant)
	{
		const FBoneContainer& BoneContainer = Output.AnimInstanceProxy->GetRequiredBones();

		FPoseContext PoseB(Output);
		B.Evaluate(PoseB);

		// Faster code path in local space
		if (TransformsSpace == EBoneControlSpace::BCS_BoneSpace)
		{
			const FCompactPose& SourcePose = PoseB.Pose;
			FCompactPose& TargetPose = Output.Pose;

			for (const FBlendBoneByChannelEntry& Entry : ValidBoneEntries)
			{
				const FCompactPoseBoneIndex SourceBoneIndex = Entry.SourceBone.GetCompactPoseIndex(BoneContainer);
				const FCompactPoseBoneIndex TargetBoneIndex = Entry.TargetBone.GetCompactPoseIndex(BoneContainer);

				const FTransform SourceTransform = SourcePose[SourceBoneIndex];
				FTransform& TargetTransform = TargetPose[TargetBoneIndex];

				// Blend Transforms.
				FTransform BlendedTransform;
				BlendedTransform.Blend(TargetTransform, SourceTransform, InternalBlendAlpha);

				// Filter through channels
				{
					if (Entry.bBlendTranslation)
					{
						TargetTransform.SetTranslation(BlendedTransform.GetTranslation());
					}

					if (Entry.bBlendRotation)
					{
						TargetTransform.SetRotation(BlendedTransform.GetRotation());
					}

					if (Entry.bBlendScale)
					{
						TargetTransform.SetScale3D(BlendedTransform.GetScale3D());
					}
				}
			}
		}
		// Slower code path where local transforms have to be converted to a different space.
		else
		{
			FCSPose<FCompactPose> TargetPoseCmpntSpace;
			TargetPoseCmpntSpace.InitPose(Output.Pose);

			FCSPose<FCompactPose> SourcePoseCmpntSpace;
			SourcePoseCmpntSpace.InitPose(PoseB.Pose);

			TArray<FBoneTransform> QueuedModifiedBoneTransforms;

			const FTransform ComponentTransform = Output.AnimInstanceProxy->GetComponentTransform();

			for (const FBlendBoneByChannelEntry& Entry : ValidBoneEntries)
			{
				const FCompactPoseBoneIndex SourceBoneIndex = Entry.SourceBone.GetCompactPoseIndex(BoneContainer);
				const FCompactPoseBoneIndex TargetBoneIndex = Entry.TargetBone.GetCompactPoseIndex(BoneContainer);

				FTransform SourceTransform = SourcePoseCmpntSpace.GetComponentSpaceTransform(SourceBoneIndex);
				FTransform TargetTransform = TargetPoseCmpntSpace.GetComponentSpaceTransform(TargetBoneIndex);

				// Convert Transforms to correct space.
				FAnimationRuntime::ConvertCSTransformToBoneSpace(ComponentTransform, SourcePoseCmpntSpace, SourceTransform, SourceBoneIndex, TransformsSpace);
				FAnimationRuntime::ConvertCSTransformToBoneSpace(ComponentTransform, TargetPoseCmpntSpace, TargetTransform, TargetBoneIndex, TransformsSpace);

				// Blend Transforms.
				FTransform BlendedTransform;
				BlendedTransform.Blend(TargetTransform, SourceTransform, InternalBlendAlpha);

				// Filter through channels
				{
					if (Entry.bBlendTranslation)
					{
						TargetTransform.SetTranslation(BlendedTransform.GetTranslation());
					}

					if (Entry.bBlendRotation)
					{
						TargetTransform.SetRotation(BlendedTransform.GetRotation());
					}

					if (Entry.bBlendScale)
					{
						TargetTransform.SetScale3D(BlendedTransform.GetScale3D());
					}
				}

				// Convert blended and filtered result back in component space.
				FAnimationRuntime::ConvertBoneSpaceTransformToCS(ComponentTransform, TargetPoseCmpntSpace, TargetTransform, TargetBoneIndex, TransformsSpace);

				// Queue transform to be applied after all transforms have been created.
				// So we don't have parent bones affecting children bones.
				QueuedModifiedBoneTransforms.Add(FBoneTransform(TargetBoneIndex, TargetTransform));
			}

			if (QueuedModifiedBoneTransforms.Num() > 0)
			{
				// Sort OutBoneTransforms so indices are in increasing order.
				QueuedModifiedBoneTransforms.Sort(FCompareBoneTransformIndex());

				// Apply transforms
				TargetPoseCmpntSpace.SafeSetCSBoneTransforms(QueuedModifiedBoneTransforms);

				// Turn Component Space poses back into local space.
				FCSPose<FCompactPose>::ConvertComponentPosesToLocalPoses(MoveTemp(TargetPoseCmpntSpace), Output.Pose);
			}
		}
	}
}

void FAnimNode_BlendBoneByChannel::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("(Alpha: %.1f%%)"), InternalBlendAlpha * 100);
	DebugData.AddDebugItem(DebugLine);

	A.GatherDebugData(DebugData.BranchFlow(1.f));
	B.GatherDebugData(DebugData.BranchFlow(InternalBlendAlpha));
}

