// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_BlendListBase.h"
#include "AnimationRuntime.h"
#include "Animation/AnimInertializationSyncScope.h"
#include "Animation/BlendProfile.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNode_Inertialization.h"
#include "Animation/AnimTrace.h"
#include "Animation/SkeletonRemapping.h"
#include "Animation/SkeletonRemappingRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_BlendListBase)

/////////////////////////////////////////////////////
// FAnimNode_BlendListBase

void FAnimNode_BlendListBase::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_Base::Initialize_AnyThread(Context);
	const int32 NumPoses = BlendPose.Num();
	const TArray<float>& CurrentBlendTimes = GetBlendTimes();
	checkSlow(CurrentBlendTimes.Num() == NumPoses);

	PerBlendData.Reset(NumPoses);
	if (NumPoses > 0)
	{
		// If we have at least 1 pose we initialize to full weight on
		// the first pose
		PerBlendData.AddZeroed(NumPoses);
		PerBlendData[0].Weight = 1.0f;

		for (int32 ChildIndex = 0; ChildIndex < NumPoses; ++ChildIndex)
		{
			BlendPose[ChildIndex].Initialize(Context);
		}
	}

	Initialize();
}

void FAnimNode_BlendListBase::Initialize()
{
	SetCurrentBlendProfile(GetBlendProfile());

	const int32 NumPoses = BlendPose.Num();

	LastActiveChildIndex = INDEX_NONE;

	EAlphaBlendOption CurrentBlendType = GetBlendType();
	UCurveFloat* CurrentCustomBlendCurve = GetCustomBlendCurve();
	for (int32 i = 0; i < PerBlendData.Num(); ++i)
	{
		FAlphaBlend& Blend = PerBlendData[i].Blend;

		Blend.SetBlendTime(0.0f);
		Blend.SetBlendOption(CurrentBlendType);
		Blend.SetCustomCurve(CurrentCustomBlendCurve);

		if (CurrentBlendProfile)
		{
			PerBlendData[i].StartAlpha = 0.0f;
		}
	}

	PerBlendData[0].Blend.SetAlpha(1.0f);
	PerBlendData[0].StartAlpha = 1.0f;
}

void FAnimNode_BlendListBase::SetCurrentBlendProfile(UBlendProfile* NewBlendProfile)
{
	CurrentBlendProfile = NewBlendProfile;
	if (CurrentBlendProfile && PerBoneSampleData.IsEmpty())
	{
		InitializePerBoneData();
	}
}

void FAnimNode_BlendListBase::InitializePerBoneData()
{
	if (CurrentBlendProfile)
	{
		const int32 NumPoses = BlendPose.Num();

		// Initialise per-bone data
		PerBoneSampleData.Empty(NumPoses);
		PerBoneSampleData.AddZeroed(NumPoses);

		for (int32 Idx = 0; Idx < NumPoses; ++Idx)
		{
			FBlendSampleData& SampleData = PerBoneSampleData[Idx];
			SampleData.SampleDataIndex = Idx;
			SampleData.PerBoneBlendData.AddZeroed(CurrentBlendProfile->GetNumBlendEntries());
		}
	}
}

void FAnimNode_BlendListBase::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	for(int32 ChildIndex=0; ChildIndex<BlendPose.Num(); ChildIndex++)
	{
		BlendPose[ChildIndex].CacheBones(Context);
	}
}

void FAnimNode_BlendListBase::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)
	GetEvaluateGraphExposedInputs().Execute(Context);

	const int32 NumPoses = BlendPose.Num();
	const TArray<float>& CurrentBlendTimes = GetBlendTimes();
	checkSlow(PerBlendData.Num() == NumPoses);
	bool bRequestedInertializationOnActiveChildIndexChange = false;
	
	if (NumPoses > 0)
	{
		// Handle a change in the active child index; adjusting the target weights
		const int32 ChildIndex = GetActiveChildIndex();
		
		if (ChildIndex != LastActiveChildIndex)
		{
			SetCurrentBlendProfile(GetBlendProfile());	// Blend profile can change based on the active child

			bool LastChildIndexIsInvalid = (LastActiveChildIndex == INDEX_NONE);
			
			const float CurrentWeight = PerBlendData[ChildIndex].Weight;
			const float DesiredWeight = 1.0f;
			const float WeightDifference = FMath::Clamp<float>(FMath::Abs<float>(DesiredWeight - CurrentWeight), 0.0f, 1.0f);

			// scale by the weight difference since we want always consistency:
			// - if you're moving from 0 to full weight 1, it will use the normal blend time
			// - if you're moving from 0.5 to full weight 1, it will get there in half the time
			float RemainingBlendTime;
			if (LastChildIndexIsInvalid)
			{
				RemainingBlendTime = 0.0f;
			}
			else if (GetTransitionType() == EBlendListTransitionType::Inertialization)
			{
				UE::Anim::IInertializationRequester* InertializationRequester = Context.GetMessage<UE::Anim::IInertializationRequester>();
				if (InertializationRequester)
				{
					FInertializationRequest Request;
					Request.Duration = CurrentBlendTimes[ChildIndex];
					Request.BlendProfile = CurrentBlendProfile;
					Request.bUseBlendMode = true;
					Request.BlendMode = GetBlendType();
					Request.CustomBlendCurve = GetCustomBlendCurve();
#if ANIM_TRACE_ENABLED
					Request.NodeId = Context.GetCurrentNodeId();
					Request.AnimInstance = Context.AnimInstanceProxy->GetAnimInstanceObject();
#endif

					InertializationRequester->RequestInertialization(Request);
					InertializationRequester->AddDebugRecord(*Context.AnimInstanceProxy, Context.GetCurrentNodeId());
					bRequestedInertializationOnActiveChildIndexChange = true;
				}
				else
				{
					FAnimNode_Inertialization::LogRequestError(Context, BlendPose[ChildIndex]);
				}
				
				RemainingBlendTime = 0.0f;
			}
			else
			{
				RemainingBlendTime = CurrentBlendTimes[ChildIndex] * WeightDifference;
			}

			for (int32 i = 0; i < PerBlendData.Num(); ++i)
			{
				PerBlendData[i].RemainingTime = RemainingBlendTime;
			}

			// If we have a valid previous child and we're instantly blending - update that pose with zero weight
			if(RemainingBlendTime == 0.0f && !LastChildIndexIsInvalid)
			{
				BlendPose[LastActiveChildIndex].Update(Context.FractionalWeight(0.0f));
			}

			for(int32 i = 0; i < PerBlendData.Num(); ++i)
			{
				FAlphaBlend& Blend = PerBlendData[i].Blend;

				Blend.SetBlendTime(RemainingBlendTime);

				if(i == ChildIndex)
				{
					Blend.SetValueRange(PerBlendData[i].Weight, 1.0f);

					if (CurrentBlendProfile)
					{
						Blend.ResetAlpha();
						PerBlendData[i].StartAlpha = Blend.GetAlpha();
					}
				}
				else
				{
					Blend.SetValueRange(PerBlendData[i].Weight, 0.0f);
				}

				if (CurrentBlendProfile)
				{
					PerBlendData[i].StartAlpha = Blend.GetAlpha();
				}
			}

			// When bResetChildOnActivation is true and the weight of the new child is zero, we'll reinitialize the child.
			if (GetChildUpdateMode() == EBlendListChildUpdateMode::ResetChildOnActivate && CurrentWeight <= ZERO_ANIMWEIGHT_THRESH)
			{
				FAnimationInitializeContext ReinitializeContext(Context.AnimInstanceProxy, Context.SharedContext);

				// reinitialize
				BlendPose[ChildIndex].Initialize(ReinitializeContext);
			}

			LastActiveChildIndex = ChildIndex;
		}

		// Advance the weights
		//@TODO: This means we advance even in a frame where the target weights/times just got modified; is that desirable?
		float SumWeight = 0.0f;
		for (int32 i = 0; i < PerBlendData.Num(); ++i)
		{
			float& BlendWeight = PerBlendData[i].Weight;

			FAlphaBlend& Blend = PerBlendData[i].Blend;
			Blend.Update(Context.GetDeltaTime());
			BlendWeight = Blend.GetBlendedValue();

			SumWeight += BlendWeight;
		}

		// Renormalize the weights
		if ((SumWeight > ZERO_ANIMWEIGHT_THRESH) && (FMath::Abs<float>(SumWeight - 1.0f) > ZERO_ANIMWEIGHT_THRESH))
		{
			float ReciprocalSum = 1.0f / SumWeight;
			for (int32 i = 0; i < PerBlendData.Num(); ++i)
			{
				PerBlendData[i].Weight *= ReciprocalSum;
			}
		}

		// Update our active children
		for (int32 i = 0; i < BlendPose.Num(); ++i)
		{
			const float BlendWeight = PerBlendData[i].Weight;
			if (BlendWeight > ZERO_ANIMWEIGHT_THRESH)
			{
				FAnimationUpdateContext ChildContext = Context.FractionalWeight(BlendWeight);
				
				UE::Anim::TOptionalScopedGraphMessage<UE::Anim::FAnimInertializationSyncScope> InertializationSync(bRequestedInertializationOnActiveChildIndexChange, ChildContext);
				BlendPose[i].Update((i == ChildIndex) ? ChildContext : ChildContext.AsInactive());
			}
			else if (GetChildUpdateMode() == EBlendListChildUpdateMode::AlwaysTickChildren)
			{
				// With always update children on, even if weight is 0 we will update the child
				FAnimationUpdateContext ChildContext = Context.FractionalWeight(FAnimWeight::GetSmallestRelevantWeight());
				UE::Anim::TOptionalScopedGraphMessage<UE::Anim::FAnimInertializationSyncScope> InertializationSync(bRequestedInertializationOnActiveChildIndexChange, ChildContext);
				BlendPose[i].Update((i == ChildIndex) ? ChildContext : ChildContext.AsInactive());
			}
		}

		// If we're using a blend profile, extract the scales and build blend sample data
		if (CurrentBlendProfile)
		{
			for(int32 i = 0; i < BlendPose.Num(); ++i)
			{
				FBlendSampleData& PoseSampleData = PerBoneSampleData[i];
				const FBlendData& BlendData = PerBlendData[i];
				PoseSampleData.TotalWeight = BlendData.Weight;
				const bool bInverse = (CurrentBlendProfile->Mode == EBlendProfileMode::WeightFactor) ? (ChildIndex != i) : false;
				CurrentBlendProfile->UpdateBoneWeights(PoseSampleData, BlendData.Blend, BlendData.StartAlpha, BlendData.Weight, bInverse);
			}

			FBlendSampleData::NormalizeDataWeight(PerBoneSampleData);
		}
	}

#if ANIM_TRACE_ENABLED
	const int32 ChildIndex = GetActiveChildIndex();
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Active Index"), ChildIndex);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Active Weight"), PerBlendData[ChildIndex].Weight);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Active Blend Time"), CurrentBlendTimes[ChildIndex]);
#endif
}

void FAnimNode_BlendListBase::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER(BlendPosesInGraph, !IsInGameThread());

	// Build local arrays to pass to the BlendPosesTogether runtime
	TArray<int32, TInlineAllocator<8>> PosesToEvaluate;
	TArray<float, TInlineAllocator<8>> BlendWeights;
	const int32 PerBlendDataCount = PerBlendData.Num();
	for(int32 BlendIndex = 0; BlendIndex < PerBlendDataCount; ++BlendIndex)
	{
		const FBlendData& BlendData = PerBlendData[BlendIndex];
		BlendWeights.Add(BlendData.Weight);
		if(BlendData.Weight > ZERO_ANIMWEIGHT_THRESH)
		{
			PosesToEvaluate.Add(BlendIndex);
		}
	}

	const int32 NumPoses = PosesToEvaluate.Num();

	if ((NumPoses > 0) && (BlendPose.Num() == BlendWeights.Num()))
	{
		if(NumPoses == 1 && FAnimWeight::IsFullWeight(BlendWeights[PosesToEvaluate[0]]) && CurrentBlendProfile == nullptr)
		{
			// Single full weight pose - pass-through fast common case
			BlendPose[PosesToEvaluate[0]].Evaluate(Output);
		}
		else if (NumPoses == 2 && CurrentBlendProfile == nullptr && FMath::IsNearlyEqual(BlendWeights[PosesToEvaluate[0]] + BlendWeights[PosesToEvaluate[1]], 1.f))
		{
			// Fast path for two active poses without blend profile: evaluate first child directly into Output to avoid allocating a third pose.
			// Requires weights summing to 1.0 since BlendTwoPosesTogetherInPlace derives the second weight as (1 - first weight).
			const int32 FirstPoseIndex = PosesToEvaluate[0];
			const int32 SecondPoseIndex = PosesToEvaluate[1];

			BlendPose[FirstPoseIndex].Evaluate(Output);

			FPoseContext Pose2(Output);
			BlendPose[SecondPoseIndex].Evaluate(Pose2);

			FAnimationPoseData InOutPoseData(Output);
			const FAnimationPoseData PoseTwoData(Pose2);
			FAnimationRuntime::BlendTwoPosesTogetherInPlace(InOutPoseData, PoseTwoData, BlendWeights[FirstPoseIndex]);
		}
		else
		{
			// Scratch arrays for evaluation, stack allocated
			TArray<FCompactPose, TInlineAllocator<8>> FilteredPoses;
			TArray<FBlendedCurve, TInlineAllocator<8>> FilteredCurve;
			TArray<UE::Anim::FStackAttributeContainer, TInlineAllocator<8>> FilteredAttributes;

			FilteredPoses.SetNum(NumPoses, EAllowShrinking::No);
			FilteredCurve.SetNum(NumPoses, EAllowShrinking::No);
			FilteredAttributes.SetNum(NumPoses, EAllowShrinking::No);

			int32 NumActivePoses = 0;
			for (int32 i = 0; i < PosesToEvaluate.Num(); ++i)
			{
				int32 PoseIndex = PosesToEvaluate[i];

				FPoseContext EvaluateContext(Output);

				FPoseLink& CurrentPose = BlendPose[PoseIndex];
				CurrentPose.Evaluate(EvaluateContext);

				FilteredPoses[i].MoveBonesFrom(EvaluateContext.Pose);
				FilteredCurve[i].MoveFrom(EvaluateContext.Curve);
				FilteredAttributes[i].MoveFrom(EvaluateContext.CustomAttributes);
			}

			FAnimationPoseData OutAnimationPoseData(Output);
		
			// Use the calculated blend sample data if we're blending per-bone
			if (CurrentBlendProfile)
			{
				const USkeleton* TargetSkeleton = Output.Pose.GetBoneContainer().GetSkeletonAsset();
				const USkeleton* SourceSkeleton = CurrentBlendProfile->OwningSkeleton;
				const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(SourceSkeleton, TargetSkeleton);
				if (SkeletonRemapping.IsValid())
				{
					FAnimationRuntime::BlendPosesTogetherPerBoneRemapped(FilteredPoses, FilteredCurve, FilteredAttributes, CurrentBlendProfile, PerBoneSampleData, PosesToEvaluate, SkeletonRemapping, OutAnimationPoseData);
				}
				else
				{
					FAnimationRuntime::BlendPosesTogetherPerBone(FilteredPoses, FilteredCurve, FilteredAttributes, CurrentBlendProfile, PerBoneSampleData, PosesToEvaluate, OutAnimationPoseData);
				}
			}
			else
			{
				FAnimationRuntime::BlendPosesTogether(FilteredPoses, FilteredCurve, FilteredAttributes, BlendWeights, PosesToEvaluate, OutAnimationPoseData);
			}
		}
	}
	else
	{
		Output.ResetToRefPose();
	}
}

void FAnimNode_BlendListBase::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	const int32 NumPoses = BlendPose.Num();
	const int32 ChildIndex = GetActiveChildIndex();

	FString DebugLine = GetNodeName(DebugData);
	const TArray<float>& CurrentBlendTimes = GetBlendTimes();
	DebugLine += FString::Printf(TEXT("(Active: (%i/%i) Weight: %.1f%% Time %.3f)"), ChildIndex+1, NumPoses, PerBlendData[ChildIndex].Weight*100.f, CurrentBlendTimes[ChildIndex]);

	DebugData.AddDebugItem(DebugLine);
	
	for(int32 Pose = 0; Pose < NumPoses; ++Pose)
	{
		BlendPose[Pose].GatherDebugData(DebugData.BranchFlow(PerBlendData[Pose].Weight));
	}
}

const TArray<float>& FAnimNode_BlendListBase::GetBlendTimes() const
{
	return GET_ANIM_NODE_DATA(TArray<float>, BlendTime);
}

EBlendListTransitionType FAnimNode_BlendListBase::GetTransitionType() const
{
	return GET_ANIM_NODE_DATA(EBlendListTransitionType, TransitionType);
}

EAlphaBlendOption FAnimNode_BlendListBase::GetBlendType() const
{
	return GET_ANIM_NODE_DATA(EAlphaBlendOption, BlendType);
}

EBlendListChildUpdateMode FAnimNode_BlendListBase::GetChildUpdateMode() const
{
	return GET_ANIM_NODE_DATA(EBlendListChildUpdateMode, ChildUpateMode);
}

UCurveFloat* FAnimNode_BlendListBase::GetCustomBlendCurve() const
{
	return GET_ANIM_NODE_DATA(TObjectPtr<UCurveFloat>, CustomBlendCurve);
}

UBlendProfile* FAnimNode_BlendListBase::GetBlendProfile() const
{
	return GET_ANIM_NODE_DATA(TObjectPtr<UBlendProfile>, BlendProfile);
}
