// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_BlendListByBool.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_BlendListByBool)

/////////////////////////////////////////////////////
// FAnimNode_BlendListByBool

int32 FAnimNode_BlendListByBool::GetActiveChildIndex()
{
	// Note: Intentionally flipped boolean sense (the true input is #0, and the false input is #1)
	return GetActiveValue() ? 0 : 1;
}

bool FAnimNode_BlendListByBool::GetActiveValue() const
{
	return GET_ANIM_NODE_DATA(bool, bActiveValue);
}

bool FAnimNode_BlendListByBool::GetUseSeperateBlendProfiles() const
{
	return GET_ANIM_NODE_DATA(bool, bUseSeperateBlendProfileForFalse);
}

UBlendProfile* FAnimNode_BlendListByBool::GetBlendProfileForFalse() const
{
	return GET_ANIM_NODE_DATA(TObjectPtr<UBlendProfile>, BlendProfileForFalse);
}

UBlendProfile* FAnimNode_BlendListByBool::GetBlendProfile() const
{
	// If the option is ticked, we have a seperate profile for the false branch
	if (GetUseSeperateBlendProfiles() && GetActiveValue() == false)
	{
		return GetBlendProfileForFalse();
	}

	return Super::GetBlendProfile();
}
