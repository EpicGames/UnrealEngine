// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_BlendListByEnum.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_BlendListByEnum)

/////////////////////////////////////////////////////
// FAnimNode_BlendListByEnum

int32 FAnimNode_BlendListByEnum::GetActiveChildIndex()
{
	uint8 CurrentActiveEnumValue = GetActiveEnumValue();
	const TArray<int32>& CurrentEnumToPoseIndex = GetEnumToPoseIndex();
	if (CurrentEnumToPoseIndex.IsValidIndex(CurrentActiveEnumValue))
	{
		return CurrentEnumToPoseIndex[CurrentActiveEnumValue];
	}
	else
	{
		return 0;
	}
}

#if WITH_EDITORONLY_DATA
void FAnimNode_BlendListByEnum::SetEnumToPoseIndex(const TArray<int32>& InEnumToPoseIndex)
{
	EnumToPoseIndex = InEnumToPoseIndex;
}
#endif

const TArray<int32>& FAnimNode_BlendListByEnum::GetEnumToPoseIndex() const
{
	return GET_ANIM_NODE_DATA(TArray<int32>, EnumToPoseIndex);
}

uint8 FAnimNode_BlendListByEnum::GetActiveEnumValue() const
{
	return GET_ANIM_NODE_DATA(uint8, ActiveEnumValue);
}
