// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_BlendListByInt.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_BlendListByInt)

/////////////////////////////////////////////////////
// FAnimNode_BlendListByInt

int32 FAnimNode_BlendListByInt::GetActiveChildIndex()
{
	const int32 NumPoses = BlendPose.Num();
	int32 CurrentActiveChildIndex = GET_ANIM_NODE_DATA(int32, ActiveChildIndex);
	return FMath::Clamp<int32>(CurrentActiveChildIndex, 0, NumPoses - 1);
}

