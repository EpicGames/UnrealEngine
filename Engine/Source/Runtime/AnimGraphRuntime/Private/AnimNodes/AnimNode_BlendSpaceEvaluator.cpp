// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_BlendSpaceEvaluator.h"
#include "Animation/BlendSpace.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_BlendSpaceEvaluator)

/////////////////////////////////////////////////////
// FAnimNode_BlendSpaceEvaluator

FAnimNode_BlendSpaceEvaluator::FAnimNode_BlendSpaceEvaluator()
	: FAnimNode_BlendSpacePlayer()
	, NormalizedTime(0.f)
{
}

void FAnimNode_BlendSpaceEvaluator::UpdateAssetPlayer(const FAnimationUpdateContext& Context)
{
	GetEvaluateGraphExposedInputs().Execute(Context);
	InternalTimeAccumulator = FMath::Clamp(NormalizedTime, 0.f, 1.f);

	UpdateInternal(Context);

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Name"), GetBlendSpace() ? *GetBlendSpace()->GetName() : TEXT("None"));
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Asset"), GetBlendSpace());
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Playback Time"), InternalTimeAccumulator);
}

void FAnimNode_BlendSpaceEvaluator::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	UBlendSpace* CurrentBlendSpace = GetBlendSpace();
	if (CurrentBlendSpace)
	{
		DebugLine += FString::Printf(TEXT("('%s' Play Time: %.3f)"), *CurrentBlendSpace->GetName(), InternalTimeAccumulator);
		DebugData.AddDebugItem(DebugLine, true);
	}
}

float FAnimNode_BlendSpaceEvaluator::GetPlayRate() const
{
	return bTeleportToNormalizedTime ? 0.0f : 1.0f;;
}

