// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_BlendSpaceGraphBase.h"
#include "Animation/BlendSpace.h"
#include "Animation/AnimInstanceProxy.h"
#include "AnimGraphRuntimeTrace.h"
#include "Animation/AnimSyncScope.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_BlendSpaceGraphBase)

#if WITH_EDITORONLY_DATA
#include "Animation/AnimBlueprintGeneratedClass.h"
#endif

void FAnimNode_BlendSpaceGraphBase::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)

	check(BlendSpace != nullptr);

	BlendSampleDataCache.Empty();
	BlendSpace->InitializeFilter(&BlendFilter);

	// Initialize all of our poses
	for(FPoseLink& SamplePose : SamplePoseLinks)
	{
		SamplePose.Initialize(Context);
	}
}

void FAnimNode_BlendSpaceGraphBase::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)

	// Cache all of our poses
	for(FPoseLink& SamplePose : SamplePoseLinks)
	{
		SamplePose.CacheBones(Context);
	}
}

void FAnimNode_BlendSpaceGraphBase::UpdateInternal(const FAnimationUpdateContext& Context)
{
	check(BlendSpace != nullptr);

	// Filter input and update blend samples
	FVector BlendParams = GetPosition();
#if WITH_EDITORONLY_DATA
	if(bUsePreviewPosition)
	{
		// Consume any preview sample we have set
		BlendParams = PreviewPosition;
		bUsePreviewPosition = false;
	}
#endif
	const float DeltaTime = Context.GetDeltaTime();
	const FVector FilteredBlendParams = BlendSpace->FilterInput(&BlendFilter, BlendParams, DeltaTime);
	BlendSpace->UpdateBlendSamples(FilteredBlendParams, DeltaTime, BlendSampleDataCache, CachedTriangulationIndex);

	for(int32 SampleIndex = 0; SampleIndex < BlendSampleDataCache.Num(); ++SampleIndex)
	{
		check(SamplePoseLinks.IsValidIndex(BlendSampleDataCache[SampleIndex].SampleDataIndex));
		FPoseLink& SamplePoseLink = SamplePoseLinks[BlendSampleDataCache[SampleIndex].SampleDataIndex];
		FAnimationUpdateContext LinkContext = Context.FractionalWeight(BlendSampleDataCache[SampleIndex].TotalWeight);
		SamplePoseLink.Update(LinkContext);
	}

#if WITH_EDITORONLY_DATA
	if (FAnimBlueprintDebugData* DebugData = Context.AnimInstanceProxy->GetAnimBlueprintDebugData())
	{
		DebugData->RecordBlendSpacePlayer(Context.GetCurrentNodeId(), BlendSpace, BlendParams, FilteredBlendParams);
	}
#endif

	TRACE_BLENDSPACE(Context, *this);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Name"), *BlendSpace->GetName());
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Asset"), BlendSpace);
}

void FAnimNode_BlendSpaceGraphBase::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)
	GetEvaluateGraphExposedInputs().Execute(Context);

	const bool bApplySyncing = GroupName != NAME_None;
	UE::Anim::TOptionalScopedGraphMessage<UE::Anim::FAnimSyncGroupScope> Message(bApplySyncing, Context, Context, GroupName, GroupRole);

	UpdateInternal(Context);
}

void FAnimNode_BlendSpaceGraphBase::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)

	check(BlendSpace != nullptr);

	BlendSpace->GetAnimationPose(BlendSampleDataCache, SamplePoseLinks, FAnimExtractContext(), Output);
}

void FAnimNode_BlendSpaceGraphBase::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);
	if (BlendSpace)
	{
		DebugLine += FString::Printf(TEXT("('%s')"), *BlendSpace->GetName());

		DebugData.AddDebugItem(DebugLine, true);
	}
}

#if WITH_EDITORONLY_DATA
void FAnimNode_BlendSpaceGraphBase::SetPreviewPosition(FVector InVector)
{
	bUsePreviewPosition = true;
	PreviewPosition = InVector;
}
#endif

void FAnimNode_BlendSpaceGraphBase::SnapToPosition(const FVector& NewPosition)
{
	const int32 NumAxis = FMath::Min(BlendFilter.FilterPerAxis.Num(), 3);
	for (int32 Idx = 0; Idx < NumAxis; Idx++)
	{
		BlendFilter.FilterPerAxis[Idx].SetToValue(static_cast<float>(NewPosition[Idx]));
	}
}
