// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_BlendSpacePlayer.h"
#include "Animation/BlendSpace.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimInstanceProxy.h"
#include "AnimGraphRuntimeTrace.h"
#include "Animation/AnimInertializationSyncScope.h"
#include "Animation/AnimSync.h"
#include "Animation/AnimSyncScope.h"
#include "Animation/AnimStats.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_BlendSpacePlayer)

#if WITH_EDITORONLY_DATA
#include "Animation/AnimBlueprintGeneratedClass.h"
#endif

/////////////////////////////////////////////////////
// FAnimNode_BlendSpacePlayerBase

float FAnimNode_BlendSpacePlayerBase::GetCurrentAssetTime() const
{
	if(const FBlendSampleData* HighestWeightedSample = GetHighestWeightedSample())
	{
		return HighestWeightedSample->Time;
	}

	// No sample
	return 0.0f;
}

float FAnimNode_BlendSpacePlayerBase::GetCurrentAssetTimePlayRateAdjusted() const
{
	float Length = GetCurrentAssetLength();
	return GetPlayRate() < 0.0f ? Length - InternalTimeAccumulator * Length : Length * InternalTimeAccumulator;
}

float FAnimNode_BlendSpacePlayerBase::GetCurrentAssetLength() const
{
	if(const FBlendSampleData* HighestWeightedSample = GetHighestWeightedSample())
	{
		UBlendSpace* CurrentBlendSpace = GetBlendSpace();
		if (CurrentBlendSpace != nullptr)
		{
			const FBlendSample& Sample = CurrentBlendSpace->GetBlendSample(HighestWeightedSample->SampleDataIndex);
			return Sample.GetSamplePlayLength();
		}
	}

	// No sample
	return 0.0f;
}

void FAnimNode_BlendSpacePlayerBase::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_AssetPlayerBase::Initialize_AnyThread(Context);

	GetEvaluateGraphExposedInputs().Execute(Context);

	Reinitialize();

	PreviousBlendSpace = GetBlendSpace();
}

void FAnimNode_BlendSpacePlayerBase::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
}

void FAnimNode_BlendSpacePlayerBase::UpdateAssetPlayer(const FAnimationUpdateContext& Context)
{
	GetEvaluateGraphExposedInputs().Execute(Context);

	UpdateInternal(Context);
}

void FAnimNode_BlendSpacePlayerBase::UpdateInternal(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(UpdateInternal)

	UBlendSpace* CurrentBlendSpace = GetBlendSpace();
	if (CurrentBlendSpace != nullptr && CurrentBlendSpace->GetSkeleton() != nullptr)
	{	
		if (PreviousBlendSpace != CurrentBlendSpace)
		{
			Reinitialize(ShouldResetPlayTimeWhenBlendSpaceChanges());
		}

		const FVector Position = GetPosition();

		// Create a tick record and push into the closest scope
		UE::Anim::FAnimSyncGroupScope& SyncScope = Context.GetMessageChecked<UE::Anim::FAnimSyncGroupScope>();

		FAnimTickRecord TickRecord(
			CurrentBlendSpace, Position, BlendSampleDataCache, BlendFilter, IsLooping(), GetPlayRate(), ShouldTeleportToTime(),
			IsEvaluator(), Context.GetFinalBlendWeight(), /*inout*/ InternalTimeAccumulator, MarkerTickRecord);
		TickRecord.RootMotionWeightModifier = Context.GetRootMotionWeightModifier();
		TickRecord.DeltaTimeRecord = &DeltaTimeRecord;
		TickRecord.bRequestedInertialization = Context.GetMessage<UE::Anim::FAnimInertializationSyncScope>() != nullptr;
#if ANIM_TRACE_ENABLED
		TickRecord.SourceProxy = Context.AnimInstanceProxy; 
#endif
		TickRecord.GatherContextData(Context);

		SyncScope.AddTickRecord(TickRecord, GetSyncParams(TickRecord.bRequestedInertialization), UE::Anim::FAnimSyncDebugInfo(Context));

#if WITH_EDITORONLY_DATA
		if (FAnimBlueprintDebugData* DebugData = Context.AnimInstanceProxy->GetAnimBlueprintDebugData())
		{
			DebugData->RecordBlendSpacePlayer(Context.GetCurrentNodeId(), CurrentBlendSpace, Position, BlendFilter.GetFilterLastOutput());
		}
#endif

		PreviousBlendSpace = CurrentBlendSpace;
	}

	TRACE_BLENDSPACE_PLAYER(Context, *this);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Name"), CurrentBlendSpace ? *CurrentBlendSpace->GetName() : TEXT("None"));
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Asset"), CurrentBlendSpace);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Playback Time"), InternalTimeAccumulator);
}

void FAnimNode_BlendSpacePlayerBase::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread) 
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(BlendSpacePlayer, !IsInGameThread());

	UBlendSpace* CurrentBlendSpace = GetBlendSpace();
	if (CurrentBlendSpace != nullptr && CurrentBlendSpace->GetSkeleton() != nullptr)
	{
		FAnimationPoseData AnimationPoseData(Output);
		CurrentBlendSpace->GetAnimationPose(BlendSampleDataCache, FAnimExtractContext(static_cast<double>(InternalTimeAccumulator), Output.AnimInstanceProxy->ShouldExtractRootMotion(), DeltaTimeRecord, IsLooping()), AnimationPoseData);
	}
	else
	{
		Output.ResetToRefPose();
	}
}

void FAnimNode_BlendSpacePlayerBase::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	UBlendSpace* CurrentBlendSpace = GetBlendSpace();
	if (CurrentBlendSpace)
	{
		DebugLine += FString::Printf(TEXT("('%s' Play Time: %.3f)"), *CurrentBlendSpace->GetName(), InternalTimeAccumulator);

		DebugData.AddDebugItem(DebugLine, true);
	}
}

float FAnimNode_BlendSpacePlayerBase::GetTimeFromEnd(float CurrentTime) const
{
	// Blend-spaces use normalized time value
	const float PlayLength = 1.0f;
	return GetBlendSpace() != nullptr ? PlayLength - CurrentTime : 0.0f;
}

void FAnimNode_BlendSpacePlayerBase::SnapToPosition(const FVector& NewPosition)
{
	const int32 NumAxis = FMath::Min(BlendFilter.FilterPerAxis.Num(), 3);
	for (int32 Idx = 0; Idx < NumAxis; Idx++)
	{
		BlendFilter.FilterPerAxis[Idx].SetToValue(static_cast<float>(NewPosition[Idx]));
	}
}

UAnimationAsset* FAnimNode_BlendSpacePlayerBase::GetAnimAsset() const
{
	return GetBlendSpace();
}

const FBlendSampleData* FAnimNode_BlendSpacePlayerBase::GetHighestWeightedSample() const
{
	if(BlendSampleDataCache.Num() == 0)
	{
		return nullptr;
	}

	const FBlendSampleData* HighestSample = &BlendSampleDataCache[0];

	for(int32 Idx = 1; Idx < BlendSampleDataCache.Num(); ++Idx)
	{
		if(BlendSampleDataCache[Idx].TotalWeight > HighestSample->TotalWeight)
		{
			HighestSample = &BlendSampleDataCache[Idx];
		}
	}

	return HighestSample;
}

void FAnimNode_BlendSpacePlayerBase::Reinitialize(bool bResetTime)
{
	BlendSampleDataCache.Reset();
	if(bResetTime)
	{
		float CurrentStartPosition = GetStartPosition();

		InternalTimeAccumulator = FMath::Clamp(CurrentStartPosition, 0.f, 1.0f);
		if (CurrentStartPosition == 0.f && GetPlayRate() < 0.0f)
		{
			// Blend spaces run between 0 and 1
			InternalTimeAccumulator = 1.0f;
		}

		MarkerTickRecord.Reset();
	}

	UBlendSpace* CurrentBlendSpace = GetBlendSpace();
	if (CurrentBlendSpace != nullptr)
	{
		CurrentBlendSpace->InitializeFilter(&BlendFilter);
	}
}

/////////////////////////////////////////////////////
// FAnimNode_BlendSpacePlayer

FName FAnimNode_BlendSpacePlayer::GetGroupName() const
{
	return GET_ANIM_NODE_DATA(FName, GroupName);
}

EAnimGroupRole::Type FAnimNode_BlendSpacePlayer::GetGroupRole() const
{
	return GET_ANIM_NODE_DATA(TEnumAsByte<EAnimGroupRole::Type>, GroupRole);
}

bool FAnimNode_BlendSpacePlayer::GetOverridePositionWhenJoiningSyncGroupAsLeader() const
{
	return GET_ANIM_NODE_DATA(bool, bOverridePositionWhenJoiningSyncGroupAsLeader);
}

EAnimSyncMethod FAnimNode_BlendSpacePlayer::GetGroupMethod() const
{
	return GET_ANIM_NODE_DATA(EAnimSyncMethod, Method);
}

bool FAnimNode_BlendSpacePlayer::GetIgnoreForRelevancyTest() const
{
	return GET_ANIM_NODE_DATA(bool, bIgnoreForRelevancyTest);
}

bool FAnimNode_BlendSpacePlayer::SetGroupName(FName InGroupName)
{
#if WITH_EDITORONLY_DATA
	GroupName = InGroupName;
#endif

	if(FName* GroupNamePtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(FName, GroupName))
	{
		*GroupNamePtr = InGroupName;
		return true;
	}

	return false;
}

bool FAnimNode_BlendSpacePlayer::SetGroupRole(EAnimGroupRole::Type InRole)
{
#if WITH_EDITORONLY_DATA
	GroupRole = InRole;
#endif
	
	if(TEnumAsByte<EAnimGroupRole::Type>* GroupRolePtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(TEnumAsByte<EAnimGroupRole::Type>, GroupRole))
	{
		*GroupRolePtr = InRole;
		return true;
	}

	return false;
}

bool FAnimNode_BlendSpacePlayer::SetOverridePositionWhenJoiningSyncGroupAsLeader(bool InOverridePositionWhenJoiningSyncGroupAsLeader)
{
#if WITH_EDITORONLY_DATA
	bOverridePositionWhenJoiningSyncGroupAsLeader = InOverridePositionWhenJoiningSyncGroupAsLeader;
#endif

	if (bool* OverridePositionPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(bool, bOverridePositionWhenJoiningSyncGroupAsLeader))
	{
		*OverridePositionPtr = InOverridePositionWhenJoiningSyncGroupAsLeader;
		return true;
	}
	
	return false;
}

bool FAnimNode_BlendSpacePlayer::SetGroupMethod(EAnimSyncMethod InMethod)
{
#if WITH_EDITORONLY_DATA
	Method = InMethod;
#endif

	if(EAnimSyncMethod* MethodPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(EAnimSyncMethod, Method))
	{
		*MethodPtr = InMethod;
		return true;
	}

	return false;
}

bool FAnimNode_BlendSpacePlayer::SetIgnoreForRelevancyTest(bool bInIgnoreForRelevancyTest)
{
#if WITH_EDITORONLY_DATA
	bIgnoreForRelevancyTest = bInIgnoreForRelevancyTest;
#endif

	if(bool* bIgnoreForRelevancyTestPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(bool, bIgnoreForRelevancyTest))
	{
		*bIgnoreForRelevancyTestPtr = bInIgnoreForRelevancyTest;
		return true;
	}

	return false;
}

bool FAnimNode_BlendSpacePlayer::SetResetPlayTimeWhenBlendSpaceChanges(bool bInReset)
{
#if WITH_EDITORONLY_DATA
	bResetPlayTimeWhenBlendSpaceChanges = bInReset;
#endif

	if (bool* bResetPlayTimeWhenBlendSpaceChangesPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(bool, bResetPlayTimeWhenBlendSpaceChanges))
	{
		*bResetPlayTimeWhenBlendSpaceChangesPtr = bInReset;
		return true;
	}

	return false;
}

bool FAnimNode_BlendSpacePlayer::SetBlendSpace(UBlendSpace* InBlendSpace)
{
	BlendSpace = InBlendSpace;
	return true;
}

FVector FAnimNode_BlendSpacePlayer::GetPosition() const
{
	return FVector(GET_ANIM_NODE_DATA(float, X), GET_ANIM_NODE_DATA(float, Y), 0.0f);
}

bool FAnimNode_BlendSpacePlayer::SetPosition(FVector InPosition)
{
#if WITH_EDITORONLY_DATA
	X = static_cast<float>(InPosition[0]);
	Y = static_cast<float>(InPosition[1]);
	GET_MUTABLE_ANIM_NODE_DATA(float, X) = static_cast<float>(InPosition[0]);
	GET_MUTABLE_ANIM_NODE_DATA(float, Y) = static_cast<float>(InPosition[1]);
#endif

	float* XPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(float, X);
	float* YPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(float, Y);

	if (XPtr && YPtr)
	{
		*XPtr = static_cast<float>(InPosition[0]);
		*YPtr = static_cast<float>(InPosition[1]);
		return true;
	}

	return false;
}


float FAnimNode_BlendSpacePlayer::GetPlayRate() const
{
	return GET_ANIM_NODE_DATA(float, PlayRate);
}

bool FAnimNode_BlendSpacePlayer::SetPlayRate(float InPlayRate)
{
#if WITH_EDITORONLY_DATA
	PlayRate = InPlayRate;
	GET_MUTABLE_ANIM_NODE_DATA(float, PlayRate) = InPlayRate;
#endif

	if (float* PlayRatePtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(float, PlayRate))
	{
		*PlayRatePtr = InPlayRate;
		return true;
	}

	return false;
}

bool FAnimNode_BlendSpacePlayer::IsLooping() const
{
	return GET_ANIM_NODE_DATA(bool, bLoop);
}

bool FAnimNode_BlendSpacePlayer::SetLoop(bool bInLoop)
{
#if WITH_EDITORONLY_DATA
	bLoop = bInLoop;
	GET_MUTABLE_ANIM_NODE_DATA(bool, bLoop) = bInLoop;
#endif

	if (bool* bLoopPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(bool, bLoop))
	{
		*bLoopPtr = bInLoop;
		return true;
	}

	return false;
}

bool FAnimNode_BlendSpacePlayer::ShouldResetPlayTimeWhenBlendSpaceChanges() const
{
	return GET_ANIM_NODE_DATA(bool, bResetPlayTimeWhenBlendSpaceChanges);
}

float FAnimNode_BlendSpacePlayer::GetStartPosition() const
{
	return GET_ANIM_NODE_DATA(float, StartPosition);
}

UBlendSpace* FAnimNode_BlendSpacePlayer::GetBlendSpace() const
{
	return BlendSpace;
}
