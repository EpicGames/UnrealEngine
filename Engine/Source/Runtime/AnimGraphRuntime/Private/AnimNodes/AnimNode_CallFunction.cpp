// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_CallFunction.h"
#include "Animation/AnimInstanceProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_CallFunction)

void FAnimNode_CallFunction::OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance)
{
	Counter.Reset();
	CurrentWeight = 0.0f;
}

void FAnimNode_CallFunction::GatherDebugData(FNodeDebugData& DebugData)
{
	DebugData.AddDebugItem(TEXT("CallFunction"));
	
	Source.GatherDebugData(DebugData);
}

void FAnimNode_CallFunction::CallFunctionFromCallSite(EAnimFunctionCallSite InCallSite, const FAnimationBaseContext& InContext) const
{
	if(CallSite == InCallSite)
	{
		GetFunction().Call(InContext.AnimInstanceProxy->GetAnimInstanceObject());
	}
}

void FAnimNode_CallFunction::Update_AnyThread(const FAnimationUpdateContext& InContext)
{
	CallFunctionFromCallSite(EAnimFunctionCallSite::OnUpdate, InContext);

	bool bBecameRelevant = false;
	if(!Counter.HasEverBeenUpdated() || !Counter.WasSynchronizedCounter(InContext.AnimInstanceProxy->GetUpdateCounter()))
	{
		bBecameRelevant = true;
		CurrentWeight = 0.0f;
		CallFunctionFromCallSite(EAnimFunctionCallSite::OnBecomeRelevant, InContext);
	}

	const float NewWeight = InContext.GetFinalBlendWeight();
	if(FAnimWeight::IsFullWeight(CurrentWeight) && !FAnimWeight::IsFullWeight(NewWeight))
	{
		CallFunctionFromCallSite(EAnimFunctionCallSite::OnStartedBlendingOut, InContext);
	}

	if(!FAnimWeight::IsRelevant(CurrentWeight) && FAnimWeight::IsRelevant(NewWeight))
	{
		CallFunctionFromCallSite(EAnimFunctionCallSite::OnStartedBlendingIn, InContext);
	}

	if(!FAnimWeight::IsFullWeight(CurrentWeight) && FAnimWeight::IsFullWeight(NewWeight))
	{
		CallFunctionFromCallSite(EAnimFunctionCallSite::OnFinishedBlendingIn, InContext);
	}

	Source.Update(InContext);

	if(bBecameRelevant)
	{
		CallFunctionFromCallSite(EAnimFunctionCallSite::OnBecomeRelevantPostRecursion, InContext);
	}

	CallFunctionFromCallSite(EAnimFunctionCallSite::OnUpdatePostRecursion, InContext);

	Counter.SynchronizeWith(InContext.AnimInstanceProxy->GetUpdateCounter());
	CurrentWeight = NewWeight;
}

void FAnimNode_CallFunction::Evaluate_AnyThread(FPoseContext& InContext)
{
	CallFunctionFromCallSite(EAnimFunctionCallSite::OnEvaluate, InContext);
	
	Source.Evaluate(InContext);

	CallFunctionFromCallSite(EAnimFunctionCallSite::OnEvaluatePostRecursion, InContext);
}

void FAnimNode_CallFunction::Initialize_AnyThread(const FAnimationInitializeContext& InContext)
{
	CurrentWeight = 0.0f;
	
	CallFunctionFromCallSite(EAnimFunctionCallSite::OnInitialize, InContext);
	
	Source.Initialize(InContext);

	CallFunctionFromCallSite(EAnimFunctionCallSite::OnInitializePostRecursion, InContext);
}

void FAnimNode_CallFunction::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	Source.CacheBones(Context);
}

const FAnimNodeFunctionRef& FAnimNode_CallFunction::GetFunction() const
{
	return GET_ANIM_NODE_DATA(FAnimNodeFunctionRef, Function);
}
