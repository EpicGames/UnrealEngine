// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_CopyPoseFromMesh.h"

#include "Animation/AnimCurveUtils.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimTrace.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_CopyPoseFromMesh)

/////////////////////////////////////////////////////
// FAnimNode_CopyPoseFromMesh

namespace
{
	static bool bTestLeaderPoseEarly = true;
	static FAutoConsoleVariableRef CVarTestLeaderPoseEarly(
		TEXT("a.CopyPose.TestLeaderPoseEarly"),
		bTestLeaderPoseEarly,
		TEXT("In CopyPoseFromMesh, test if we have a Leader Pose early to avoid potential mesh corruption"));
}

FAnimNode_CopyPoseFromMesh::FAnimNode_CopyPoseFromMesh()
	: SourceMeshComponent(nullptr)
	, bUseAttachedParent (false)
	, bCopyCurves (false)
	, bCopyCustomAttributes(false)
	, bUseMeshPose (false)
{
}

void FAnimNode_CopyPoseFromMesh::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_Base::Initialize_AnyThread(Context);

	// Initial update of the node, so we dont have a frame-delay on setup
	GetEvaluateGraphExposedInputs().Execute(Context);
}

void FAnimNode_CopyPoseFromMesh::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)

}

void FAnimNode_CopyPoseFromMesh::RefreshMeshComponent(USkeletalMeshComponent* TargetMeshComponent)
{
	auto ResetMeshComponent = [this](USkeletalMeshComponent* InMeshComponent, USkeletalMeshComponent* InTargetMeshComponent)
	{
		// If our source is running under leader-pose, then get bone data from there
		if (bTestLeaderPoseEarly && InMeshComponent)
		{
			if (USkeletalMeshComponent* LeaderPoseComponent = Cast<USkeletalMeshComponent>(InMeshComponent->LeaderPoseComponent.Get()))
			{
				InMeshComponent = LeaderPoseComponent;
			}
		}

		USkeletalMeshComponent* CurrentMeshComponent = CurrentlyUsedSourceMeshComponent.Get();

		// if current mesh exists, but not same as input mesh
		if (CurrentMeshComponent)
		{
			// if component has been changed, reinitialize
			if (CurrentMeshComponent != InMeshComponent)
			{
				ReinitializeMeshComponent(InMeshComponent, InTargetMeshComponent);
			}
			// if component is still same but mesh has been changed, we have to reinitialize
			else if (CurrentMeshComponent->GetSkeletalMeshAsset() != CurrentlyUsedSourceMesh.Get())
			{
				ReinitializeMeshComponent(InMeshComponent, InTargetMeshComponent);
			}
			else if (InTargetMeshComponent)
			{
				// see if target mesh has changed
				if (InTargetMeshComponent->GetSkeletalMeshAsset() != CurrentlyUsedTargetMesh.Get())
				{
					ReinitializeMeshComponent(InMeshComponent, InTargetMeshComponent);
				}
			}
		}
		// if not valid, but input mesh is
		else if (!CurrentMeshComponent && InMeshComponent)
		{
			ReinitializeMeshComponent(InMeshComponent, InTargetMeshComponent);
		}
	};

	if(SourceMeshComponent.IsValid())
	{
		ResetMeshComponent(SourceMeshComponent.Get(), TargetMeshComponent);
	}
	else if (bUseAttachedParent)
	{
		if (TargetMeshComponent)
		{
			// Walk up the attachment chain until we find a skeletal mesh component
			USkeletalMeshComponent* ParentMeshComponent = nullptr;
			for (USceneComponent* AttachParentComp = TargetMeshComponent->GetAttachParent(); AttachParentComp != nullptr; AttachParentComp = AttachParentComp->GetAttachParent())
			{
				ParentMeshComponent = Cast<USkeletalMeshComponent>(AttachParentComp);
				if (ParentMeshComponent)
				{
					break;
				}
			}

			if (ParentMeshComponent)
			{
				ResetMeshComponent(ParentMeshComponent, TargetMeshComponent);
			}
			else
			{
				CurrentlyUsedSourceMeshComponent.Reset();
			}
		}
		else
		{
			CurrentlyUsedSourceMeshComponent.Reset();
		}
	}
	else
	{
		CurrentlyUsedSourceMeshComponent.Reset();
	}
}

void FAnimNode_CopyPoseFromMesh::PreUpdate(const UAnimInstance* InAnimInstance)
{
	QUICK_SCOPE_CYCLE_COUNTER(FAnimNode_CopyPoseFromMesh_PreUpdate);

	RefreshMeshComponent(InAnimInstance->GetSkelMeshComponent());

	USkeletalMeshComponent* CurrentMeshComponent = CurrentlyUsedSourceMeshComponent.IsValid() ? CurrentlyUsedSourceMeshComponent.Get() : nullptr;

	if (CurrentMeshComponent && CurrentMeshComponent->GetSkeletalMeshAsset() && CurrentMeshComponent->IsRegistered())
	{
		if (!bTestLeaderPoseEarly)
		{
			// If our source is running under leader-pose, then get bone data from there
			if (USkeletalMeshComponent* LeaderPoseComponent = Cast<USkeletalMeshComponent>(CurrentMeshComponent->LeaderPoseComponent.Get()))
			{
				CurrentMeshComponent = LeaderPoseComponent;
			}
		}

		// re-check mesh component validity as it may have changed to leader
		if(CurrentMeshComponent->GetSkeletalMeshAsset() && CurrentMeshComponent->IsRegistered())
		{
			const bool bUROInSync = CurrentMeshComponent->ShouldUseUpdateRateOptimizations() && CurrentMeshComponent->AnimUpdateRateParams != nullptr && CurrentMeshComponent->AnimUpdateRateParams == InAnimInstance->GetSkelMeshComponent()->AnimUpdateRateParams;
			const bool bUsingExternalInterpolation = CurrentMeshComponent->IsUsingExternalInterpolation();
			const TArray<FTransform>& CachedComponentSpaceTransforms = CurrentMeshComponent->GetCachedComponentSpaceTransforms();
			const bool bArraySizesMatch = CachedComponentSpaceTransforms.Num() == CurrentMeshComponent->GetComponentSpaceTransforms().Num();

			// Copy source array from the appropriate location
			SourceMeshTransformArray.Reset();
			SourceMeshTransformArray.Append((bUROInSync || bUsingExternalInterpolation) && bArraySizesMatch ? CachedComponentSpaceTransforms : CurrentMeshComponent->GetComponentSpaceTransforms());

			if (UE::Anim::bUseSafeMeshPoseIndices)
			{
				SourceMeshValidMeshPoseTransforms = CurrentMeshComponent->GetValidMeshPoseTransforms();
			}

			// Ref skeleton is need for parent index lookups later, so store it now
			CurrentlyUsedMesh = CurrentMeshComponent->GetSkeletalMeshAsset();

			if(bCopyCurves)
			{
				if (CurrentMeshComponent->bEnableAnimation == false)
				{
					// Assume this is using the anim next path. Only curves directly in the mesh component are valid.
					// @TODO: Replace the path below with this one after validating performance and behavior.
					SourceCurves.CopyFrom(CurrentMeshComponent->GetAnimCurves());
				}
				else if (UAnimInstance* SourceAnimInstance = CurrentMeshComponent->GetAnimInstance())
				{
					// Potential optimization/tradeoff: If we stored the curve results on the mesh component in non-editor scenarios, this would be
					// much faster (but take more memory). As it is, we need to translate the map stored on the anim instance.
					const TMap<FName, float>& AnimCurveList = SourceAnimInstance->GetAnimationCurveList(EAnimCurveType::AttributeCurve);
					UE::Anim::FCurveUtils::BuildUnsorted(SourceCurves, AnimCurveList);
				}
				else
				{
					SourceCurves.Empty();
				}
			}

			if (bCopyCustomAttributes)
			{
				SourceCustomAttributes.CopyFrom(CurrentMeshComponent->GetCustomAttributes());
			}
		}
		else
		{
			CurrentlyUsedMesh.Reset();
		}
	}
}

void FAnimNode_CopyPoseFromMesh::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)
	// This introduces a frame of latency in setting the pin-driven source component,
	// but we cannot do the work to extract transforms on a worker thread as it is not thread safe.
	GetEvaluateGraphExposedInputs().Execute(Context);

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Component"), *GetNameSafe(CurrentlyUsedSourceMeshComponent.IsValid() ? CurrentlyUsedSourceMeshComponent.Get() : nullptr));
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Mesh"), *GetNameSafe(CurrentlyUsedSourceMeshComponent.IsValid() ? CurrentlyUsedSourceMeshComponent.Get()->GetSkeletalMeshAsset() : nullptr));
}

void FAnimNode_CopyPoseFromMesh::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(CopyPoseFromMesh, !IsInGameThread());

	FCompactPose& OutPose = Output.Pose;
	OutPose.ResetToRefPose();
	USkeletalMesh* CurrentMesh = CurrentlyUsedMesh.IsValid() ? CurrentlyUsedMesh.Get() : nullptr;
	if(SourceMeshTransformArray.Num() > 0 && CurrentMesh)
	{
		const FBoneContainer& RequiredBones = OutPose.GetBoneContainer();

		if (bUseMeshPose)
		{
			FCSPose<FCompactPose> MeshPoses;
			MeshPoses.InitPose(OutPose);

			for (FCompactPoseBoneIndex PoseBoneIndex : OutPose.ForEachBoneIndex())
			{
				const FMeshPoseBoneIndex MeshBoneIndex = RequiredBones.MakeMeshPoseIndex(PoseBoneIndex);
				const int32* Value = BoneMapToSource.Find(MeshBoneIndex.GetInt());
 				if (Value && SourceMeshTransformArray.IsValidIndex(*Value))
				{
					if ((!UE::Anim::bUseSafeMeshPoseIndices || (SourceMeshValidMeshPoseTransforms.IsValidIndex(*Value) && SourceMeshValidMeshPoseTransforms[*Value])))
					{
						const int32 SourceBoneIndex = *Value;
						MeshPoses.SetComponentSpaceTransform(PoseBoneIndex, SourceMeshTransformArray[SourceBoneIndex]);
					}
				}
			}

			FCSPose<FCompactPose>::ConvertComponentPosesToLocalPosesSafe(MeshPoses, OutPose);
		}
		else
		{
			for (FCompactPoseBoneIndex PoseBoneIndex : OutPose.ForEachBoneIndex())
			{
				const FMeshPoseBoneIndex MeshBoneIndex = RequiredBones.MakeMeshPoseIndex(PoseBoneIndex);
				const int32* Value = BoneMapToSource.Find(MeshBoneIndex.GetInt());
				if (Value && SourceMeshTransformArray.IsValidIndex(*Value))
				{
					if ((!UE::Anim::bUseSafeMeshPoseIndices || (SourceMeshValidMeshPoseTransforms.IsValidIndex(*Value) && SourceMeshValidMeshPoseTransforms[*Value])))
					{
						const int32 SourceBoneIndex = *Value;
						const int32 ParentIndex = CurrentMesh->GetRefSkeleton().GetParentIndex(SourceBoneIndex);
						const FCompactPoseBoneIndex MyParentIndex = RequiredBones.GetParentBoneIndex(PoseBoneIndex);
						// only apply if I also have parent, otherwise, it should apply the space bases
						if (SourceMeshTransformArray.IsValidIndex(ParentIndex) && MyParentIndex != INDEX_NONE)
						{
							const FTransform& ParentTransform = SourceMeshTransformArray[ParentIndex];
							const FTransform& ChildTransform = SourceMeshTransformArray[SourceBoneIndex];
							OutPose[PoseBoneIndex] = ChildTransform.GetRelativeTransform(ParentTransform);
						}
						else
						{
							OutPose[PoseBoneIndex] = SourceMeshTransformArray[SourceBoneIndex];
						}
					}
				}
			}
		}
	}

	if (bCopyCurves)
	{
		Output.Curve.CopyFrom(SourceCurves);
	}

	if (bCopyCustomAttributes)
	{	
		const FBoneContainer& RequiredBones = OutPose.GetBoneContainer();
		UE::Anim::Attributes::CopyAndRemapAttributes(SourceCustomAttributes, Output.CustomAttributes, SourceBoneToTarget, RequiredBones);		
	}
}

void FAnimNode_CopyPoseFromMesh::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	DebugLine += FString::Printf(TEXT("('%s')"), *GetNameSafe(CurrentlyUsedSourceMeshComponent.IsValid() ? CurrentlyUsedSourceMeshComponent.Get()->GetSkeletalMeshAsset() : nullptr));
	DebugData.AddDebugItem(DebugLine, true);
}

void FAnimNode_CopyPoseFromMesh::ReinitializeMeshComponent(USkeletalMeshComponent* NewSourceMeshComponent, USkeletalMeshComponent* TargetMeshComponent)
{
	CurrentlyUsedSourceMeshComponent.Reset();
	// reset source mesh
	CurrentlyUsedSourceMesh.Reset();
	CurrentlyUsedTargetMesh.Reset();
	BoneMapToSource.Reset();

	if (TargetMeshComponent && IsValid(NewSourceMeshComponent) && NewSourceMeshComponent->GetSkeletalMeshAsset())
	{
		USkeletalMesh* SourceSkelMesh = NewSourceMeshComponent->GetSkeletalMeshAsset();
		USkeletalMesh* TargetSkelMesh = TargetMeshComponent->GetSkeletalMeshAsset();
		
		if (IsValid(SourceSkelMesh) && !SourceSkelMesh->HasAnyFlags(RF_NeedPostLoad) &&
			IsValid(TargetSkelMesh) && !TargetSkelMesh->HasAnyFlags(RF_NeedPostLoad))
		{
			CurrentlyUsedSourceMeshComponent = NewSourceMeshComponent;
			CurrentlyUsedSourceMesh = SourceSkelMesh;
			CurrentlyUsedTargetMesh = TargetSkelMesh;

			if (SourceSkelMesh == TargetSkelMesh)
			{
				for(int32 ComponentSpaceBoneId = 0; ComponentSpaceBoneId < SourceSkelMesh->GetRefSkeleton().GetNum(); ++ComponentSpaceBoneId)
				{
					BoneMapToSource.Add(ComponentSpaceBoneId, ComponentSpaceBoneId);
				}
			}
			else
			{
				const int32 SplitBoneIndex = (RootBoneToCopy != NAME_Name)? TargetSkelMesh->GetRefSkeleton().FindBoneIndex(RootBoneToCopy) : INDEX_NONE;
				for (int32 ComponentSpaceBoneId = 0; ComponentSpaceBoneId < TargetSkelMesh->GetRefSkeleton().GetNum(); ++ComponentSpaceBoneId)
				{
					if (SplitBoneIndex == INDEX_NONE || ComponentSpaceBoneId == SplitBoneIndex
						|| TargetSkelMesh->GetRefSkeleton().BoneIsChildOf(ComponentSpaceBoneId, SplitBoneIndex))
					{
						FName BoneName = TargetSkelMesh->GetRefSkeleton().GetBoneName(ComponentSpaceBoneId);
						BoneMapToSource.Add(ComponentSpaceBoneId, SourceSkelMesh->GetRefSkeleton().FindBoneIndex(BoneName));
					}
				}
			}

			if (bCopyCustomAttributes)
			{
				SourceBoneToTarget.Reserve(BoneMapToSource.Num());
				Algo::Transform(BoneMapToSource, SourceBoneToTarget, [](const TPair<int32, int32>& Pair)
				{
					return TPair<int32, int32>(Pair.Value, Pair.Key);
				});
			}
		}
	}
}

