// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_CurveSource.h"
#include "AnimationRuntime.h"
#include "Animation/AnimCurveUtils.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimTrace.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_CurveSource)

FAnimNode_CurveSource::FAnimNode_CurveSource()
	: SourceBinding(ICurveSourceInterface::DefaultBinding)
	, Alpha(1.0f)
{
}

void FAnimNode_CurveSource::PreUpdate(const UAnimInstance* AnimInstance)
{
	// re-bind to our named curve source in pre-update
	// we do this here to allow re-binding of the source without reinitializing the whole
	// anim graph. If the source goes away (e.g. if an audio component is destroyed) or the
	// binding changes then we can re-bind to a new object
	if (CurveSource.GetObject() == nullptr || Cast<ICurveSourceInterface>(CurveSource.GetObject())->Execute_GetBindingName(CurveSource.GetObject()) != SourceBinding)
	{
		ICurveSourceInterface* PotentialCurveSource = nullptr;

		auto IsSpecifiedCurveSource = [&PotentialCurveSource](UObject* InObject, const FName& InSourceBinding, TScriptInterface<ICurveSourceInterface>& InOutCurveSource)
		{
			PotentialCurveSource = Cast<ICurveSourceInterface>(InObject);
			if (PotentialCurveSource && PotentialCurveSource->Execute_GetBindingName(InObject) == InSourceBinding)
			{
				InOutCurveSource.SetObject(InObject);
				InOutCurveSource.SetInterface(PotentialCurveSource);
				return true;
			}

			return false;
		};

		AActor* Actor = AnimInstance->GetOwningActor();
		if (Actor)
		{
			// check if our actor implements our interface
			if (IsSpecifiedCurveSource(Actor, SourceBinding, CurveSource))
			{
				return;
			}

			for (TFieldIterator<FObjectProperty> PropertyIt(Actor->GetClass(), EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
			{
				FObjectProperty* ObjProp = *PropertyIt;
				UActorComponent* ActorComponent = Cast<UActorComponent>(ObjProp->GetObjectPropertyValue(ObjProp->ContainerPtrToValuePtr<void>(Actor)));
				if (IsSpecifiedCurveSource(ActorComponent, SourceBinding, CurveSource))
				{
					return;
				}
			}

			const TSet<UActorComponent*>& ActorOwnedComponents = Actor->GetComponents();
			for (UActorComponent* OwnedComponent : ActorOwnedComponents)
			{
				if (IsSpecifiedCurveSource(OwnedComponent, SourceBinding, CurveSource))
				{
					return;
				}
			}
		}
	}
}

class FExternalCurveScratchArea : public TThreadSingleton<FExternalCurveScratchArea>
{
public:
	TArray<FNamedCurveValue> NamedCurveValues;
};

void FAnimNode_CurveSource::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(CurveSource, !IsInGameThread());

	SourcePose.Evaluate(Output);

	if (CurveSource.GetInterface() != nullptr)
	{
		TArray<FNamedCurveValue>& NamedCurveValues = FExternalCurveScratchArea::Get().NamedCurveValues;
		NamedCurveValues.Reset();
		CurveSource->Execute_GetCurves(CurveSource.GetObject(), NamedCurveValues);

		const float ClampedAlpha = FMath::Clamp(Alpha, 0.0f, 1.0f);

		FBlendedCurve Curve;
		UE::Anim::FCurveUtils::BuildUnsorted(Curve, NamedCurveValues.Num(),
			[&NamedCurveValues](int32 InCurveIndex)
			{
				return NamedCurveValues[InCurveIndex].Name;
			},
			[&NamedCurveValues](int32 InCurveIndex)
			{
				return NamedCurveValues[InCurveIndex].Value;
			});

#if ANIM_TRACE_ENABLED
		for (FNamedCurveValue NamedValue : NamedCurveValues)
		{
			TRACE_ANIM_NODE_VALUE(Output, *NamedValue.Name.ToString(), NamedValue.Value);
		}
#endif

		Output.Curve.LerpToValid(Curve, ClampedAlpha);
	}
}

void FAnimNode_CurveSource::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)
	// Evaluate any BP logic plugged into this node
	GetEvaluateGraphExposedInputs().Execute(Context);
	SourcePose.Update(Context);
}

void FAnimNode_CurveSource::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_Base::Initialize_AnyThread(Context);
	SourcePose.Initialize(Context);
}

void FAnimNode_CurveSource::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	FAnimNode_Base::CacheBones_AnyThread(Context);
	SourcePose.CacheBones(Context);
}

void FAnimNode_CurveSource::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FAnimNode_Base::GatherDebugData(DebugData);
	SourcePose.GatherDebugData(DebugData.BranchFlow(1.f));
}
