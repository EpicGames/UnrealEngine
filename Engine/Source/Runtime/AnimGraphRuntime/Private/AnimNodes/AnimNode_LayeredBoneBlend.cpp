// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_LayeredBoneBlend.h"
#include "AnimationRuntime.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimTrace.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimCurveUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_LayeredBoneBlend)

/////////////////////////////////////////////////////
// FAnimNode_LayeredBoneBlend

void FAnimNode_LayeredBoneBlend::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_Base::Initialize_AnyThread(Context);

	const int NumPoses = BlendPoses.Num();
	checkSlow(BlendWeights.Num() == NumPoses);

	// initialize children
	BasePose.Initialize(Context);

	if (NumPoses > 0)
	{
		for (int32 ChildIndex = 0; ChildIndex < NumPoses; ++ChildIndex)
		{
			BlendPoses[ChildIndex].Initialize(Context);
		}
	}
}

void FAnimNode_LayeredBoneBlend::SyncBlendMasksAndLayers()
{
	if (BlendMode == ELayeredBoneBlendMode::BlendMask)
	{
		BlendMasks.SetNum(BlendPoses.Num());
		LayerSetup.Reset();
	}
	else
	{
		BlendMasks.Reset();
		LayerSetup.SetNum(BlendPoses.Num());
	}
}

void FAnimNode_LayeredBoneBlend::RebuildPerBoneBlendWeights(const USkeleton* InSkeleton)
{
	if (InSkeleton)
	{
		if (BlendMode == ELayeredBoneBlendMode::BranchFilter)
		{
			FAnimationRuntime::CreateMaskWeights(PerBoneBlendWeights, LayerSetup, InSkeleton);
		}
		else
		{
			FAnimationRuntime::CreateMaskWeights(PerBoneBlendWeights, BlendMasks, InSkeleton);
		}

		SkeletonGuid = InSkeleton->GetGuid();
		VirtualBoneGuid = InSkeleton->GetVirtualBoneGuid();
	}
}

bool FAnimNode_LayeredBoneBlend::ArePerBoneBlendWeightsValid(const USkeleton* InSkeleton) const
{
	return (InSkeleton != nullptr && InSkeleton->GetGuid() == SkeletonGuid && InSkeleton->GetVirtualBoneGuid() == VirtualBoneGuid);
}

void FAnimNode_LayeredBoneBlend::UpdateCachedBoneData(const FBoneContainer& RequiredBones, const USkeleton* Skeleton)
{
	if(RequiredBones.GetSerialNumber() == RequiredBonesSerialNumber)
	{
		return;
	}

	if (!ArePerBoneBlendWeightsValid(Skeleton))
	{
		RebuildPerBoneBlendWeights(Skeleton);
	}
	
	// build desired bone weights
	const TArray<FBoneIndexType>& RequiredBoneIndices = RequiredBones.GetBoneIndicesArray();
	const int32 NumRequiredBones = RequiredBoneIndices.Num();
	DesiredBoneBlendWeights.SetNumZeroed(NumRequiredBones);
	for (int32 RequiredBoneIndex=0; RequiredBoneIndex<NumRequiredBones; RequiredBoneIndex++)
	{
		const int32 SkeletonBoneIndex = RequiredBones.GetSkeletonIndex(FCompactPoseBoneIndex(RequiredBoneIndex));
		if (ensure(SkeletonBoneIndex != INDEX_NONE))
		{
			DesiredBoneBlendWeights[RequiredBoneIndex] = PerBoneBlendWeights[SkeletonBoneIndex];
		}
	}
	
	CurrentBoneBlendWeights.Reset(DesiredBoneBlendWeights.Num());
	CurrentBoneBlendWeights.AddZeroed(DesiredBoneBlendWeights.Num());

	//Reinitialize bone blend weights now that we have cleared them
	FAnimationRuntime::UpdateDesiredBoneWeight(DesiredBoneBlendWeights, CurrentBoneBlendWeights, BlendWeights);

	// Build curve source indices
	{
		// Get the original Reserve value
		int32 OriginalReserve = CurvePoseSourceIndices.Max();
		CurvePoseSourceIndices.Empty();

		Skeleton->ForEachCurveMetaData([this, &RequiredBones](const FName& InCurveName, const FCurveMetaData& InMetaData)
		{
			for (const FBoneReference& LinkedBone : InMetaData.LinkedBones)
			{
				FCompactPoseBoneIndex CompactPoseIndex = LinkedBone.GetCompactPoseIndex(RequiredBones);
				if (CompactPoseIndex != INDEX_NONE)
				{
					if (DesiredBoneBlendWeights[CompactPoseIndex.GetInt()].BlendWeight > 0.f)
					{
						CurvePoseSourceIndices.Add(InCurveName, DesiredBoneBlendWeights[CompactPoseIndex.GetInt()].SourceIndex);
						break;
					}
				}
			}
		});

		// Shrink afterwards to exactly what was used if the Reserve increased, to save memory.  Eventually the reserve will
		// stabilize at the maximum number of nodes actually used in practice for this specific anim node.
		if (CurvePoseSourceIndices.Num() > OriginalReserve)
		{
			CurvePoseSourceIndices.Shrink();
		}
	}

	RequiredBonesSerialNumber = RequiredBones.GetSerialNumber();
}

void FAnimNode_LayeredBoneBlend::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	BasePose.CacheBones(Context);
	int32 NumPoses = BlendPoses.Num();
	for(int32 ChildIndex=0; ChildIndex<NumPoses; ChildIndex++)
	{
		BlendPoses[ChildIndex].CacheBones(Context);
	}

	UpdateCachedBoneData(Context.AnimInstanceProxy->GetRequiredBones(), Context.AnimInstanceProxy->GetSkeleton());
}

void FAnimNode_LayeredBoneBlend::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)
	bHasRelevantPoses = false;
	int32 RootMotionBlendPose = -1;
	float RootMotionWeight = 0.f;
	const float RootMotionClearWeight = bBlendRootMotionBasedOnRootBone ? 0.f : 1.f;

	TArray<float, TInlineAllocator<64>> ChildRootMotionWeight;
	if (IsLODEnabled(Context.AnimInstanceProxy))
	{
		GetEvaluateGraphExposedInputs().Execute(Context);

		if (bUpdateBasePoseFirst)
		{
			ChildRootMotionWeight.SetNumZeroed(BlendPoses.Num());
		}

		for (int32 ChildIndex = 0; ChildIndex < BlendPoses.Num(); ++ChildIndex)
		{
			const float ChildWeight = BlendWeights[ChildIndex];
			if (FAnimWeight::IsRelevant(ChildWeight))
			{
				if (bHasRelevantPoses == false)
				{
					// Update cached data now we know we might be valid
					UpdateCachedBoneData(Context.AnimInstanceProxy->GetRequiredBones(), Context.AnimInstanceProxy->GetSkeleton());

					// Update weights
					FAnimationRuntime::UpdateDesiredBoneWeight(DesiredBoneBlendWeights, CurrentBoneBlendWeights, BlendWeights);
					bHasRelevantPoses = true;

					if(bBlendRootMotionBasedOnRootBone && !CurrentBoneBlendWeights.IsEmpty())
					{
						const float NewRootMotionWeight = CurrentBoneBlendWeights[0].BlendWeight;
						if(NewRootMotionWeight > ZERO_ANIMWEIGHT_THRESH)
						{
							RootMotionWeight = NewRootMotionWeight;
							RootMotionBlendPose = CurrentBoneBlendWeights[0].SourceIndex;
						}
					}
				}

				const float ThisPoseRootMotionWeight = (ChildIndex == RootMotionBlendPose) ? RootMotionWeight : RootMotionClearWeight;
				if (bUpdateBasePoseFirst)
				{
					ChildRootMotionWeight[ChildIndex] = ThisPoseRootMotionWeight;
				}
				else
				{
					BlendPoses[ChildIndex].Update(Context.FractionalWeightAndRootMotion(ChildWeight, ThisPoseRootMotionWeight));
				}
			}
		}
	}

	// initialize children
	const float BaseRootMotionWeight = 1.f - RootMotionWeight;

	if (BaseRootMotionWeight < ZERO_ANIMWEIGHT_THRESH)
	{
		BasePose.Update(Context.FractionalWeightAndRootMotion(1.f, BaseRootMotionWeight));
	}
	else
	{
		BasePose.Update(Context);
	}

	if (!ChildRootMotionWeight.IsEmpty())
	{
		check(bUpdateBasePoseFirst);
		check(ChildRootMotionWeight.Num() == BlendPoses.Num());

		for (int32 ChildIndex = 0; ChildIndex < BlendPoses.Num(); ++ChildIndex)
		{
			const float ChildWeight = BlendWeights[ChildIndex];
			if (FAnimWeight::IsRelevant(ChildWeight))
			{
				const float ThisPoseRootMotionWeight = ChildRootMotionWeight[ChildIndex];
				BlendPoses[ChildIndex].Update(Context.FractionalWeightAndRootMotion(ChildWeight, ThisPoseRootMotionWeight));
			}
		}
	}

#if ANIM_TRACE_ENABLED
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Num Poses"), BlendPoses.Num());
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Has Relevant Poses"), bHasRelevantPoses);
	for (int32 LayerIndex = 0; LayerIndex < BlendWeights.Num(); ++LayerIndex)
	{
		TRACE_ANIM_NODE_VALUE(Context, *FString::Printf(TEXT("Layer %d Weight"), LayerIndex), BlendWeights[LayerIndex]);
	}
#endif
}

void FAnimNode_LayeredBoneBlend::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER(BlendPosesInGraph, !IsInGameThread());

	const int NumPoses = BlendPoses.Num();
	if ((NumPoses == 0) || !bHasRelevantPoses)
	{
		BasePose.Evaluate(Output);
	}
	else
	{
		FPoseContext BasePoseContext(Output);

		// evaluate children
		BasePose.Evaluate(BasePoseContext);

		TArray<FCompactPose, TMemStackAllocator<>> TargetBlendPoses;
		TargetBlendPoses.SetNum(NumPoses);

		TArray<FBlendedCurve, TMemStackAllocator<>> TargetBlendCurves;
		TargetBlendCurves.SetNum(NumPoses);

		TArray<UE::Anim::FStackAttributeContainer, TMemStackAllocator<>> TargetBlendAttributes;
		TargetBlendAttributes.SetNum(NumPoses);

		for (int32 ChildIndex = 0; ChildIndex < NumPoses; ++ChildIndex)
		{
			if (FAnimWeight::IsRelevant(BlendWeights[ChildIndex]))
			{
				FPoseContext CurrentPoseContext(Output);
				BlendPoses[ChildIndex].Evaluate(CurrentPoseContext);

				TargetBlendPoses[ChildIndex].MoveBonesFrom(CurrentPoseContext.Pose);
				TargetBlendCurves[ChildIndex].MoveFrom(CurrentPoseContext.Curve);
				TargetBlendAttributes[ChildIndex].MoveFrom(CurrentPoseContext.CustomAttributes);
			}
			else
			{
				TargetBlendPoses[ChildIndex].ResetToRefPose(BasePoseContext.Pose.GetBoneContainer());
				TargetBlendCurves[ChildIndex].InitFrom(Output.Curve);
			}
		}

		// filter to make sure it only includes curves that are linked to the correct bone filter
		UE::Anim::FNamedValueArrayUtils::RemoveByPredicate(BasePoseContext.Curve, CurvePoseSourceIndices,
			[](const UE::Anim::FCurveElement& InOutBasePoseElement, const UE::Anim::FCurveElementIndexed& InSourceIndexElement)
			{
				// if source index is set, remove base pose curve value
				return (InSourceIndexElement.Index != INDEX_NONE);
			});

		// Filter child pose curves
		for (int32 ChildIndex = 0; ChildIndex < NumPoses; ++ChildIndex)
		{
			UE::Anim::FNamedValueArrayUtils::RemoveByPredicate(TargetBlendCurves[ChildIndex], CurvePoseSourceIndices,
				[ChildIndex](const UE::Anim::FCurveElement& InOutBasePoseElement, const UE::Anim::FCurveElementIndexed& InSourceIndexElement)
				{
					// if not source, remove it
					return (InSourceIndexElement.Index != INDEX_NONE) && (InSourceIndexElement.Index != ChildIndex);
				});
		}

		FAnimationRuntime::EBlendPosesPerBoneFilterFlags BlendFlags = FAnimationRuntime::EBlendPosesPerBoneFilterFlags::None;
		if (bMeshSpaceRotationBlend)
		{
			BlendFlags |= FAnimationRuntime::EBlendPosesPerBoneFilterFlags::MeshSpaceRotation;

			if (bRootSpaceRotationBlend)
			{
				BlendFlags |= FAnimationRuntime::EBlendPosesPerBoneFilterFlags::RootSpaceRotation;
			}
		}
		if (bMeshSpaceScaleBlend)
		{
			BlendFlags |= FAnimationRuntime::EBlendPosesPerBoneFilterFlags::MeshSpaceScale;
		}

		FAnimationPoseData AnimationPoseData(Output);
		FAnimationRuntime::BlendPosesPerBoneFilter(BasePoseContext.Pose, TargetBlendPoses, BasePoseContext.Curve, TargetBlendCurves, BasePoseContext.CustomAttributes, TargetBlendAttributes, AnimationPoseData, CurrentBoneBlendWeights, BlendFlags, CurveBlendOption);
	}
}


void FAnimNode_LayeredBoneBlend::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	const int NumPoses = BlendPoses.Num();

	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("(Num Poses: %i, Has Relevant Poses: %s)"), NumPoses, bHasRelevantPoses ? TEXT("true") : TEXT("false"));
	DebugData.AddDebugItem(DebugLine);

	BasePose.GatherDebugData(DebugData.BranchFlow(1.f));

	for (int32 ChildIndex = 0; ChildIndex < NumPoses; ++ChildIndex)
	{
		BlendPoses[ChildIndex].GatherDebugData(DebugData.BranchFlow(BlendWeights[ChildIndex]));
	}
}

void FAnimNode_LayeredBoneBlend::SetBlendMask(int32 InPoseIndex, UBlendProfile* InBlendMask)
{
	check(BlendMode == ELayeredBoneBlendMode::BlendMask);
	check(BlendPoses.IsValidIndex(InPoseIndex));
	check(BlendMasks.IsValidIndex(InPoseIndex));

	BlendMasks[InPoseIndex] = InBlendMask;

	InvalidatePerBoneBlendWeights();
}
