// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_MakeDynamicAdditive.h"
#include "AnimationRuntime.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_MakeDynamicAdditive)

/////////////////////////////////////////////////////
// FAnimNode_MakeDynamicAdditive

FAnimNode_MakeDynamicAdditive::FAnimNode_MakeDynamicAdditive()
	: bMeshSpaceAdditive(false)
{
}

void FAnimNode_MakeDynamicAdditive::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_Base::Initialize_AnyThread(Context);

	Base.Initialize(Context);
	Additive.Initialize(Context);
}

void FAnimNode_MakeDynamicAdditive::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	Base.CacheBones(Context);
	Additive.CacheBones(Context);
}

void FAnimNode_MakeDynamicAdditive::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)
	Base.Update(Context.FractionalWeight(1.f));
	Additive.Update(Context.FractionalWeight(1.f));

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Mesh Space Additive"), bMeshSpaceAdditive);
}

void FAnimNode_MakeDynamicAdditive::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(MakeDynamicAdditive, !IsInGameThread());

	FScopedExpectsAdditiveOverride ScopedExpectsAdditiveOverride(Output, false);
	FPoseContext BaseEvalContext(Output);

	Base.Evaluate(BaseEvalContext);
	Additive.Evaluate(Output);

	if (bMeshSpaceAdditive)
	{
		FAnimationRuntime::ConvertPoseToMeshRotation(Output.Pose);
		FAnimationRuntime::ConvertPoseToMeshRotation(BaseEvalContext.Pose);
	}

	FAnimationRuntime::ConvertPoseToAdditive(Output.Pose, BaseEvalContext.Pose);
	Output.Curve.ConvertToAdditive(BaseEvalContext.Curve);

	UE::Anim::Attributes::ConvertToAdditive(BaseEvalContext.CustomAttributes, Output.CustomAttributes);
}

void FAnimNode_MakeDynamicAdditive::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("(Mesh Space Additive: %s)"), bMeshSpaceAdditive ? TEXT("true") : TEXT("false"));

	DebugData.AddDebugItem(DebugLine);
	Base.GatherDebugData(DebugData.BranchFlow(1.f));
	Additive.GatherDebugData(DebugData.BranchFlow(1.f));
}

