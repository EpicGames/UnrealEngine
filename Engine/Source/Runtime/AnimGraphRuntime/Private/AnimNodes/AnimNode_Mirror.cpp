// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_Mirror.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNode_Inertialization.h"
#include "Animation/AnimTrace.h"
#include "AnimationRuntime.h"
#include "Animation/AnimInertializationSyncScope.h"
#include "Animation/AttributesRuntime.h"
#include "Animation/MirrorSyncScope.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_Mirror)

#define LOCTEXT_NAMESPACE "AnimNode_Mirror"

FAnimNode_MirrorBase::FAnimNode_MirrorBase()
	: bMirrorState(false)
	, bMirrorStateIsValid(false)
{
}

FAnimNode_MirrorBase::FAnimNode_MirrorBase(const FAnimNode_MirrorBase&) = default;
FAnimNode_MirrorBase::~FAnimNode_MirrorBase() = default;

UMirrorDataTable* FAnimNode_MirrorBase::GetMirrorDataTable() const
{
	return nullptr;
}

bool FAnimNode_MirrorBase::SetMirrorDataTable(UMirrorDataTable* MirrorTable)
{
	return false;
}

EMirrorBlendType FAnimNode_MirrorBase::GetBlendType() const
{
	return EMirrorBlendType::Inertialization;
}

bool FAnimNode_MirrorBase::GetMirror() const
{
	return false;
}


float FAnimNode_MirrorBase::GetBlendTimeOnMirrorStateChange() const
{
	return 0.0;
}

bool FAnimNode_MirrorBase::GetBoneMirroring() const
{
	return false;
}

bool FAnimNode_MirrorBase::GetCurveMirroring() const
{
	return false;
}

bool FAnimNode_MirrorBase::GetAttributeMirroring() const
{
	return false;
}

bool FAnimNode_MirrorBase::GetResetChildOnMirrorStateChange() const
{
	return false;
}

bool FAnimNode_MirrorBase::SetMirror(bool bInMirror)
{
	return false;
}
bool FAnimNode_MirrorBase::SetBlendTimeOnMirrorStateChange(float InBlendTime)
{
	return false;
}

bool FAnimNode_MirrorBase::SetBoneMirroring(bool bInBoneMirroring)
{
	return false;
}

bool FAnimNode_MirrorBase::SetCurveMirroring(bool bInCurveMirroring)
{
	return false;
}

bool FAnimNode_MirrorBase::SetAttributeMirroring(bool bInAttributeMirroring)
{
	return false;
}

bool FAnimNode_MirrorBase::SetResetChildOnMirrorStateChange(bool bInResetChildOnMirrorStateChange)
{
	return false;
}

void FAnimNode_MirrorBase::SetSourceLinkNode(FAnimNode_Base* NewLinkNode)
{
	Source.SetLinkNode(NewLinkNode);
}

FAnimNode_Base* FAnimNode_MirrorBase::GetSourceLinkNode()
{
	return Source.GetLinkNode(); 
}

void FAnimNode_MirrorBase::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	FAnimNode_Base::Initialize_AnyThread(Context);
	Source.Initialize(Context);

	// Invalidate mirror state to avoid triggering blends on first frame after a re-init.
	bMirrorStateIsValid = false;
}

void FAnimNode_MirrorBase::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	Source.CacheBones(Context);
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	Super::CacheBones_AnyThread(Context);
	const FBoneContainer& BoneContainer = Context.AnimInstanceProxy->GetRequiredBones();
	FillCompactPoseAndComponentRefRotations(BoneContainer);
}

void FAnimNode_MirrorBase::MirrorPoseInPlace(FPoseContext& InPose)
{
	UMirrorDataTable* MirrorDataTable = GetMirrorDataTable();
	if (MirrorDataTable)
	{
		if (GetBoneMirroring())
		{
			if (InPose.ExpectsAdditivePose())
			{
				FText Message = FText::Format(LOCTEXT("AdditiveMirrorWarning", "Trying to mirror an additive animation pose is not supported in anim instance '{0}'"), FText::FromString(InPose.AnimInstanceProxy->GetAnimInstanceName()));
				InPose.LogMessage(EMessageSeverity::Warning, Message);

				// Force a bind pose to make it obvious
				InPose.Pose.ResetToAdditiveIdentity();
			}
			else
			{
				const FBoneContainer& BoneContainer = InPose.Pose.GetBoneContainer();
				const TArray<FBoneIndexType>& RequiredBoneIndices = BoneContainer.GetBoneIndicesArray();
				int32 NumReqBones = RequiredBoneIndices.Num();
				if (CompactPoseMirrorBones.Num() != NumReqBones)
				{
					FillCompactPoseAndComponentRefRotations(BoneContainer);
				}

				FAnimationRuntime::MirrorPose(InPose.Pose, MirrorDataTable->MirrorAxis, CompactPoseMirrorBones, ComponentSpaceRefRotations);
			}
		}

		if (GetCurveMirroring())
		{
			FAnimationRuntime::MirrorCurves(InPose.Curve, *MirrorDataTable);
		}

		if (GetAttributeMirroring())
		{
			UE::Anim::Attributes::MirrorAttributes(InPose.CustomAttributes, *MirrorDataTable, CompactPoseMirrorBones);
		}
	}
}

void FAnimNode_MirrorBase::FillCompactPoseAndComponentRefRotations(const FBoneContainer& BoneContainer)
{
	UMirrorDataTable* MirrorDataTable = GetMirrorDataTable();
	if (MirrorDataTable)
	{
		MirrorDataTable->FillCompactPoseAndComponentRefRotations(
			BoneContainer, 
			CompactPoseMirrorBones, 
			ComponentSpaceRefRotations);
	}
	else
	{
		CompactPoseMirrorBones.Reset();
		ComponentSpaceRefRotations.Reset();
	}
}

void FAnimNode_MirrorBase::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	GetEvaluateGraphExposedInputs().Execute(Context);

	bool bMirror = GetMirror();
	bool bRequestedInertialization = false;
	
	if (bMirrorStateIsValid)
	{
		if (bMirrorState != bMirror)
		{
			Blend.SetDesiredValue(bMirror ? 1.0f : 0.0f);
			if ((GetBlendType() == EMirrorBlendType::Inertialization))
			{
				Blend.SetBlendTime(0.0f);
				if (GetBlendTimeOnMirrorStateChange() > SMALL_NUMBER)
				{
					// Inertialize when switching between mirrored and unmirrored states to smooth out the pose discontinuity
					UE::Anim::IInertializationRequester* InertializationRequester = Context.GetMessage<UE::Anim::IInertializationRequester>();
					if (InertializationRequester)
					{
						FInertializationRequest Request;
						Request.Duration = GetBlendTimeOnMirrorStateChange();
#if ANIM_TRACE_ENABLED
						Request.NodeId = Context.GetCurrentNodeId();
						Request.AnimInstance = Context.AnimInstanceProxy->GetAnimInstanceObject();
#endif

						InertializationRequester->RequestInertialization(Request);
						InertializationRequester->AddDebugRecord(*Context.AnimInstanceProxy, Context.GetCurrentNodeId());

						bRequestedInertialization = true;
					}
					else
					{
						FAnimNode_Inertialization::LogRequestError(Context, Source);
					}
				}
			}
			else
			{
				Blend.SetBlendTime(GetBlendTimeOnMirrorStateChange());
			}

			// Optionally reinitialize the source when the mirror state changes
			if (GetResetChildOnMirrorStateChange())
			{
				FAnimationInitializeContext ReinitializeContext(Context.AnimInstanceProxy, Context.SharedContext);
				Source.Initialize(ReinitializeContext);
			}
		}

		Blend.Update(Context.GetDeltaTime());
	}
	else
	{
		const float DesiredAlpha = bMirror ? 1.0f : 0.0f;
		Blend.SetAlpha(DesiredAlpha);
		Blend.SetDesiredValue(DesiredAlpha);
	}

	UMirrorDataTable* MirrorDataTable = GetMirrorDataTable();
	UE::Anim::TOptionalScopedGraphMessage<UE::Anim::FMirrorSyncScope> Message(bMirror, Context, Context, MirrorDataTable);
	UE::Anim::TOptionalScopedGraphMessage<UE::Anim::FAnimInertializationSyncScope> InertializationSync(bRequestedInertialization, Context);

	bMirrorState = bMirror;
	bMirrorStateIsValid = true;

	Source.Update(Context);

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Mirrored"), bMirrorState);
}

void FAnimNode_MirrorBase::Evaluate_AnyThread(FPoseContext& Output)
{

	const float BlendAlpha = Blend.GetBlendedValue();
	if (FAnimWeight::IsFullWeight(BlendAlpha))
	{
		Source.Evaluate(Output);
		MirrorPoseInPlace(Output);
	}
	else if (FAnimWeight::IsRelevant(BlendAlpha))
	{
		// Evaluate directly into Output (unmirrored), copy into MirroredPose and mirror it in place
		Source.Evaluate(Output);

		FPoseContext MirroredPose(Output);
		MirroredPose.Pose.CopyBonesFrom(Output.Pose);
		MirroredPose.Curve.CopyFrom(Output.Curve);
		MirroredPose.CustomAttributes.CopyFrom(Output.CustomAttributes);
		MirrorPoseInPlace(MirroredPose);

		FAnimationPoseData InOutPoseData(Output);
		const FAnimationPoseData MirroredPoseData(MirroredPose);
		FAnimationRuntime::BlendTwoPosesTogetherInPlace(InOutPoseData, MirroredPoseData, (1.0f - BlendAlpha));
	}
	else
	{
		Source.Evaluate(Output);
	}
}

void FAnimNode_MirrorBase::GatherDebugData(FNodeDebugData& DebugData)
{
	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("(Mirrored: %s)"), (bMirrorState) ? TEXT("true") : TEXT("false"));
	DebugData.AddDebugItem(DebugLine);

	Source.GatherDebugData(DebugData);
}

FAnimNode_Mirror::FAnimNode_Mirror() = default;
FAnimNode_Mirror::~FAnimNode_Mirror() = default;

UMirrorDataTable* FAnimNode_Mirror::GetMirrorDataTable() const 
{ 
	return GET_ANIM_NODE_DATA(TObjectPtr<UMirrorDataTable>, MirrorDataTable);
}

bool FAnimNode_Mirror::SetMirrorDataTable(UMirrorDataTable* InMirrorTable)
{
#if WITH_EDITORONLY_DATA
	MirrorDataTable = InMirrorTable;
	GET_MUTABLE_ANIM_NODE_DATA(TObjectPtr<UMirrorDataTable>, MirrorDataTable) = InMirrorTable;
#endif

	if (TObjectPtr<UMirrorDataTable>* InMirrorTablePtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(TObjectPtr<UMirrorDataTable>, MirrorDataTable))
	{
		*InMirrorTablePtr = InMirrorTable;
		return true;
	}

	return false;
}

EMirrorBlendType FAnimNode_Mirror::GetBlendType() const
{
	return GET_ANIM_NODE_DATA(EMirrorBlendType, BlendType);

}

bool FAnimNode_Mirror::GetMirror() const
{
	return GET_ANIM_NODE_DATA(bool, bMirror);
}

float FAnimNode_Mirror::GetBlendTimeOnMirrorStateChange() const
{
	return GET_ANIM_NODE_DATA(float, BlendTime);
}

bool FAnimNode_Mirror::GetBoneMirroring() const
{
	return GET_ANIM_NODE_DATA(bool, bBoneMirroring);
}

bool FAnimNode_Mirror::GetCurveMirroring() const
{
	return GET_ANIM_NODE_DATA(bool, bCurveMirroring);
}

bool FAnimNode_Mirror::GetAttributeMirroring() const
{
	return GET_ANIM_NODE_DATA(bool, bAttributeMirroring);
}

bool FAnimNode_Mirror::GetResetChildOnMirrorStateChange() const
{
	return GET_ANIM_NODE_DATA(bool, bResetChild);
}

bool FAnimNode_Mirror::SetMirror(bool bInMirror)
{
#if WITH_EDITORONLY_DATA
	bMirror = bInMirror;
#endif

	if (bool* bMirrorPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(bool, bMirror))
	{
		*bMirrorPtr = bInMirror;
		return true;
	}
	return false;
}
bool FAnimNode_Mirror::SetBlendTimeOnMirrorStateChange(float InBlendTime)
{
#if WITH_EDITORONLY_DATA
	BlendTime = InBlendTime;
#endif

	if (float* BlendTimePtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(float, BlendTime))
	{
		*BlendTimePtr = InBlendTime;
		return true;
	}
	return false;
}

bool FAnimNode_Mirror::SetBoneMirroring(bool bInBoneMirroring)
{
#if WITH_EDITORONLY_DATA
	bBoneMirroring = bInBoneMirroring;
#endif

	if (bool* bBoneMirroringPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(bool, bBoneMirroring))
	{
		*bBoneMirroringPtr = bInBoneMirroring;
		return true;
	}

	return false;
}

bool FAnimNode_Mirror::SetCurveMirroring(bool bInCurveMirroring)
{
#if WITH_EDITORONLY_DATA
	bCurveMirroring = bInCurveMirroring;
#endif

	if (bool* bCurveMirroringPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(bool, bCurveMirroring))
	{
		*bCurveMirroringPtr = bInCurveMirroring;
		return true;
	}

	return false;
}

bool FAnimNode_Mirror::SetAttributeMirroring(bool bInAttributeMirroring)
{
#if WITH_EDITORONLY_DATA
	bAttributeMirroring = bInAttributeMirroring;
#endif

	if (bool* bAttributeMirroringPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(bool, bAttributeMirroring))
	{
		*bAttributeMirroringPtr = bInAttributeMirroring;
		return true;
	}

	return false;
}

bool FAnimNode_Mirror::SetResetChildOnMirrorStateChange(bool bInResetChildOnMirrorStateChange)
{
#if WITH_EDITORONLY_DATA
	bResetChild = bInResetChildOnMirrorStateChange;
#endif

	if (bool* bResetChildPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(bool, bResetChild))
	{
		*bResetChildPtr = bInResetChildOnMirrorStateChange;
		return true;
	}

	return false;
}


FAnimNode_Mirror_Standalone::FAnimNode_Mirror_Standalone() = default;
FAnimNode_Mirror_Standalone::FAnimNode_Mirror_Standalone(const FAnimNode_Mirror_Standalone&) = default;
FAnimNode_Mirror_Standalone::~FAnimNode_Mirror_Standalone() = default;

UMirrorDataTable* FAnimNode_Mirror_Standalone::GetMirrorDataTable() const
{
	return MirrorDataTable.Get();
}

bool FAnimNode_Mirror_Standalone::SetMirrorDataTable(UMirrorDataTable* MirrorTable)
{
	MirrorDataTable = MirrorTable;
	return true;
}

EMirrorBlendType FAnimNode_Mirror_Standalone::GetBlendType() const
{
	return BlendType;
}

bool FAnimNode_Mirror_Standalone::GetMirror() const
{
	return bMirror;
}

float FAnimNode_Mirror_Standalone::GetBlendTimeOnMirrorStateChange() const
{
	return BlendTime;
}

bool FAnimNode_Mirror_Standalone::GetBoneMirroring() const
{
	return bBoneMirroring;
}

bool FAnimNode_Mirror_Standalone::GetCurveMirroring() const
{
	return bCurveMirroring;
}

bool FAnimNode_Mirror_Standalone::GetAttributeMirroring() const
{
	return bAttributeMirroring;
}

bool FAnimNode_Mirror_Standalone::GetResetChildOnMirrorStateChange() const
{
	return bResetChild;
}

bool FAnimNode_Mirror_Standalone::SetMirror(bool bInMirror)
{
	bMirror = bInMirror;
	return true;
}
bool FAnimNode_Mirror_Standalone::SetBlendTimeOnMirrorStateChange(float InBlendTime)
{
	BlendTime = InBlendTime;
	return true;
}

bool FAnimNode_Mirror_Standalone::SetBoneMirroring(bool bInBoneMirroring)
{
	bBoneMirroring = bInBoneMirroring;
	return true;

}

bool FAnimNode_Mirror_Standalone::SetCurveMirroring(bool bInCurveMirroring)
{
	bCurveMirroring = bInCurveMirroring;
	return true;
}

bool FAnimNode_Mirror_Standalone::SetAttributeMirroring(bool bInAttributeMirroring)
{
	bAttributeMirroring = bInAttributeMirroring;
	return true;
}

bool FAnimNode_Mirror_Standalone::SetResetChildOnMirrorStateChange(bool bInResetChildOnMirrorStateChange)
{
	bResetChild = bInResetChildOnMirrorStateChange;
	return true;
}

#undef LOCTEXT_NAMESPACE

