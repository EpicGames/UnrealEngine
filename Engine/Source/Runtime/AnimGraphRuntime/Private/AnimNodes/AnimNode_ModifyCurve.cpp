// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_ModifyCurve.h"
#include "AnimationRuntime.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimCurveUtils.h"
#include "Animation/AnimStats.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_ModifyCurve)

FAnimNode_ModifyCurve::FAnimNode_ModifyCurve()
{
	ApplyMode = EModifyCurveApplyMode::Blend;
	Alpha = 1.f;
}

void FAnimNode_ModifyCurve::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	Super::Initialize_AnyThread(Context);
	SourcePose.Initialize(Context);

	// Init our last values array to be the right size
	if (ApplyMode == EModifyCurveApplyMode::WeightedMovingAverage)
	{
		LastCurve.Empty();
	}
}

void FAnimNode_ModifyCurve::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	Super::CacheBones_AnyThread(Context);
	SourcePose.CacheBones(Context);
}

void FAnimNode_ModifyCurve::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(ModifyCurve, !IsInGameThread());

	FPoseContext SourceData(Output);
	SourcePose.Evaluate(SourceData);

	Output = SourceData;

	if (ApplyMode == EModifyCurveApplyMode::WeightedMovingAverage)
	{
		// WMA acts as a filter on incoming curves - it ignores the values that are provided on pins and just uses those
		// curves as a mask to use when applying the filter

		UE::Anim::FCurveFilter Filter;
		Filter.SetFilterMode(UE::Anim::ECurveFilterMode::AllowOnlyFiltered);
		Filter.Reserve(CurveNames.Num());
		for(int32 CurveIndex = 0; CurveIndex < CurveNames.Num(); ++CurveIndex)
		{
			Filter.Add(CurveNames[CurveIndex], UE::Anim::ECurveFilterFlags::Filtered);
		}

		if(CurveMap.Num() > 0)
		{
			UE::Anim::FCurveFilter MapFilter;
			MapFilter.Reserve(CurveNames.Num());
			for(const TPair<FName, float>& NameValuePair : CurveMap)
			{
				MapFilter.Add(NameValuePair.Key, UE::Anim::ECurveFilterFlags::Filtered);
			}

			// Combine pin & map curves in case they overlap
			UE::Anim::FNamedValueArrayUtils::Union(Filter, MapFilter,
				[](UE::Anim::FCurveFilterElement& InOutElement, const UE::Anim::FCurveFilterElement& InElement, UE::Anim::ENamedValueUnionFlags InFlags)
				{
					if(EnumHasAnyFlags(InFlags, UE::Anim::ENamedValueUnionFlags::ValidArg1))
					{
						InOutElement.Flags = InElement.Flags;
					}
				});
		}

		// Mask off last curves by pin-exposed curves
		UE::Anim::FCurveUtils::Filter(LastCurve, Filter);

		// Perform WMA on output
		UE::Anim::FNamedValueArrayUtils::Union(Output.Curve, LastCurve,
			[this](UE::Anim::FCurveElement& InOutResult, const UE::Anim::FCurveElement& InCurveElement, UE::Anim::ENamedValueUnionFlags InFlags)
			{
				// Only apply curves that we are overriding
				if(EnumHasAnyFlags(InFlags, UE::Anim::ENamedValueUnionFlags::ValidArg1))
				{
					InOutResult.Value = ProcessCurveWMAOperation(InOutResult.Value, InCurveElement.Value);
				}
			});

		// Copy current to last
		LastCurve.CopyFrom(Output.Curve);
	}
	else
	{
		// Build internal curves from array & map
		FBlendedCurve Curve;
		Curve.Reserve(CurveNames.Num());
		for(int32 CurveIndex = 0; CurveIndex < CurveNames.Num(); ++CurveIndex)
		{
			Curve.Add(CurveNames[CurveIndex], CurveValues[CurveIndex]);
		}

		if(CurveMap.Num() > 0)
		{
			FBlendedCurve MapCurve;
			MapCurve.Reserve(CurveNames.Num());
			for(const TPair<FName, float>& NameValuePair : CurveMap)
			{
				MapCurve.Add(NameValuePair.Key, NameValuePair.Value);
			}

			// Combine pin & map curves in case they overlap
			Curve.Combine(MapCurve);
		}
		
		UE::Anim::FNamedValueArrayUtils::Union(Output.Curve, Curve,
			[this](UE::Anim::FCurveElement& InOutResult, const UE::Anim::FCurveElement& InCurveElement, UE::Anim::ENamedValueUnionFlags InFlags)
			{
				// Only apply curves that we are overriding
				if(EnumHasAnyFlags(InFlags, UE::Anim::ENamedValueUnionFlags::ValidArg1))
				{
					InOutResult.Value = ProcessCurveOperation(InOutResult.Value, InCurveElement.Value);
				}
			});
	}
}

float FAnimNode_ModifyCurve::ProcessCurveOperation(float CurrentValue, float NewValue) const
{
	float UseNewValue = CurrentValue;

	// Use ApplyMode enum to decide how to apply
	if (ApplyMode == EModifyCurveApplyMode::Add)
	{
		UseNewValue = CurrentValue + NewValue;
	}
	else if (ApplyMode == EModifyCurveApplyMode::Scale)
	{
		UseNewValue = CurrentValue * NewValue;
	}
	else if (ApplyMode == EModifyCurveApplyMode::RemapCurve)
	{
		const float RemapScale = 1.f / FMath::Max(1.f - NewValue, 0.01f);
		UseNewValue = FMath::Min(FMath::Max(CurrentValue - NewValue, 0.f) * RemapScale, 1.f);
	}
	else if (ApplyMode == EModifyCurveApplyMode::Blend)
	{
		UseNewValue = NewValue;
	}

	const float UseAlpha = FMath::Clamp(Alpha, 0.f, 1.f);
	return FMath::Lerp(CurrentValue, UseNewValue, UseAlpha);
}

float FAnimNode_ModifyCurve::ProcessCurveWMAOperation(float CurrentValue, float LastValue) const
{
	return FMath::WeightedMovingAverage(CurrentValue, LastValue, Alpha);
}

void FAnimNode_ModifyCurve::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)

	// Run update on input pose nodes
	SourcePose.Update(Context);

	// Evaluate any BP logic plugged into this node
	GetEvaluateGraphExposedInputs().Execute(Context);
}

void FAnimNode_ModifyCurve::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("(Alpha: %.2f)"), Alpha);
	DebugData.AddDebugItem(DebugLine);

	SourcePose.GatherDebugData(DebugData);
}

#if WITH_EDITOR

void FAnimNode_ModifyCurve::AddCurve(const FName& InName, float InValue)
{
	CurveValues.Add(InValue);
	CurveNames.Add(InName);
}

void FAnimNode_ModifyCurve::RemoveCurve(int32 PoseIndex)
{
	CurveValues.RemoveAt(PoseIndex);
	CurveNames.RemoveAt(PoseIndex);
}

#endif // WITH_EDITOR

