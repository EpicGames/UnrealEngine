// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_MultiWayBlend.h"
#include "AnimationRuntime.h"
#include "Animation/AnimStats.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_MultiWayBlend)


struct FMultiBlendData : public TThreadSingleton<FMultiBlendData>
{
	TArray<FCompactPose, TInlineAllocator<8>> SourcePoses;
	TArray<float, TInlineAllocator<8>> SourceWeights;
	TArray<FBlendedCurve, TInlineAllocator<8>> SourceCurves;
	TArray<UE::Anim::FStackAttributeContainer, TInlineAllocator<8>> SourceAttributes;
};

/////////////////////////////////////////////////////
// FAnimNode_MultiWayBlend

void FAnimNode_MultiWayBlend::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_Base::Initialize_AnyThread(Context);

	// this should be consistent all the time by editor node
	if (!ensure(Poses.Num() == DesiredAlphas.Num()))
	{
		DesiredAlphas.Init(0.f, Poses.Num());
	}

	UpdateCachedAlphas();

	for (FPoseLink& Pose : Poses)
	{
		Pose.Initialize(Context);
	}
}

void FAnimNode_MultiWayBlend::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	for (FPoseLink& Pose : Poses)
	{
		Pose.CacheBones(Context);
	}
}

void FAnimNode_MultiWayBlend::UpdateCachedAlphas()
{
	float TotalAlpha = GetTotalAlpha();

	if (DesiredAlphas.Num() > 0)
	{
		if (DesiredAlphas.Num() != CachedAlphas.Num())
		{
			CachedAlphas.Init(0.f, DesiredAlphas.Num());
		}
		else
		{
			FMemory::Memzero(CachedAlphas.GetData(), CachedAlphas.GetAllocatedSize());
		}
	}
	else
	{
		CachedAlphas.Reset();
	}

	const float ActualTotalAlpha = AlphaScaleBias.ApplyTo(TotalAlpha);
	if (ActualTotalAlpha > ZERO_ANIMWEIGHT_THRESH)
	{
		if (bNormalizeAlpha)
		{
			// normalize by total alpha
			for (int32 AlphaIndex = 0; AlphaIndex < DesiredAlphas.Num(); ++AlphaIndex)
			{
				// total alpha shouldn't be zero
				CachedAlphas[AlphaIndex] = AlphaScaleBias.ApplyTo(DesiredAlphas[AlphaIndex] / TotalAlpha);
			}
		}
		else
		{
			for (int32 AlphaIndex = 0; AlphaIndex < DesiredAlphas.Num(); ++AlphaIndex)
			{
				// total alpha shouldn't be zero
				CachedAlphas[AlphaIndex] = AlphaScaleBias.ApplyTo(DesiredAlphas[AlphaIndex]);
			}
		}
	}

	ensure(Poses.Num() == CachedAlphas.Num());
}

void FAnimNode_MultiWayBlend::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FAnimationNode_MultiWayBlend_Update);
	GetEvaluateGraphExposedInputs().Execute(Context);
	UpdateCachedAlphas();

	for (int32 PoseIndex = 0; PoseIndex < Poses.Num(); ++PoseIndex)
	{
		// total alpha shouldn't be zero
		float CurrentAlpha = CachedAlphas[PoseIndex];
		if (CurrentAlpha > ZERO_ANIMWEIGHT_THRESH)
		{
			Poses[PoseIndex].Update(Context.FractionalWeight(CurrentAlpha));
		}
	}
}

void FAnimNode_MultiWayBlend::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(MultiWayBlend, !IsInGameThread());

	// this function may be reentrant when multiple MultiWayBlend nodes are chained together
	// these scratch arrays are treated as stacks below
	FMultiBlendData& BlendData = FMultiBlendData::Get();
	TArray<FCompactPose, TInlineAllocator<8>>& SourcePoses = BlendData.SourcePoses;
	TArray<FBlendedCurve, TInlineAllocator<8>>& SourceCurves = BlendData.SourceCurves;
	TArray<UE::Anim::FStackAttributeContainer, TInlineAllocator<8>>& SourceAttributes = BlendData.SourceAttributes;
	TArray<float, TInlineAllocator<8>>& SourceWeights = BlendData.SourceWeights;

	const int32 SourcePosesInitialNum = SourcePoses.Num();
	int32 SourcePosesAdded = 0;
	if (ensure(Poses.Num() == CachedAlphas.Num()))
	{
		for (int32 PoseIndex = 0; PoseIndex < Poses.Num(); ++PoseIndex)
		{
			const float CurrentAlpha = CachedAlphas[PoseIndex];
			if (CurrentAlpha > ZERO_ANIMWEIGHT_THRESH)
			{
				// evaluate input pose, potentially reentering this function and pushing/popping more poses
				FPoseContext PoseContext(Output);
				Poses[PoseIndex].Evaluate(PoseContext);

				// push source pose data
				FCompactPose& SourcePose = SourcePoses.AddDefaulted_GetRef();
				SourcePose.MoveBonesFrom(PoseContext.Pose);

				FBlendedCurve& SourceCurve = SourceCurves.AddDefaulted_GetRef();
				SourceCurve.MoveFrom(PoseContext.Curve);

				UE::Anim::FStackAttributeContainer& SourceAttribute = SourceAttributes.AddDefaulted_GetRef();
				SourceAttribute.MoveFrom(PoseContext.CustomAttributes);

				SourceWeights.Add(CurrentAlpha);

				++SourcePosesAdded;
			}
		}
	}

	if (SourcePosesAdded > 0)
	{
		// obtain views onto the ends of our stacks
		TArrayView<FCompactPose> SourcePosesView = MakeArrayView(&SourcePoses[SourcePosesInitialNum], SourcePosesAdded);
		TArrayView<FBlendedCurve> SourceCurvesView = MakeArrayView(&SourceCurves[SourcePosesInitialNum], SourcePosesAdded);
		TArrayView<UE::Anim::FStackAttributeContainer> SourceAttributesView = MakeArrayView(&SourceAttributes[SourcePosesInitialNum], SourcePosesAdded);
		TArrayView<float> SourceWeightsView = MakeArrayView(&SourceWeights[SourcePosesInitialNum], SourcePosesAdded);

		FAnimationPoseData AnimationPoseData(Output);

		FAnimationRuntime::BlendPosesTogether(SourcePosesView, SourceCurvesView, SourceAttributesView, SourceWeightsView, AnimationPoseData);
		
		// normalize rotation - some cases, where additive is applied less than 1, it will use non normalized rotation
		Output.Pose.NormalizeRotations();
		
		// pop the poses we added
		SourcePoses.SetNum(SourcePosesInitialNum, EAllowShrinking::No);
		SourceCurves.SetNum(SourcePosesInitialNum, EAllowShrinking::No);
		SourceWeights.SetNum(SourcePosesInitialNum, EAllowShrinking::No);
		SourceAttributes.SetNum(SourcePosesInitialNum, EAllowShrinking::No);
	}
	else
	{
		if (bAdditiveNode)
		{
			Output.ResetToAdditiveIdentity();
		}
		else
		{
			Output.ResetToRefPose();
		}
	}
}

void FAnimNode_MultiWayBlend::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);
	DebugData.AddDebugItem(DebugLine);

	for (int32 PoseIndex=0; PoseIndex <Poses.Num(); ++PoseIndex)
	{
		Poses[PoseIndex].GatherDebugData(DebugData.BranchFlow(CachedAlphas[PoseIndex]));
	}
}
