// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_PoseBlendNode.h"
#include "AnimationRuntime.h"
#include "Animation/AnimCurveUtils.h"
#include "Animation/AnimInstanceProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_PoseBlendNode)

/////////////////////////////////////////////////////
// FAnimPoseByNameNode

FAnimNode_PoseBlendNode::FAnimNode_PoseBlendNode()
	: CustomCurve(nullptr)
{
	BlendOption = UE::Anim::DefaultBlendOption;
}

void FAnimNode_PoseBlendNode::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_PoseHandler::Initialize_AnyThread(Context);
	
	SourcePose.Initialize(Context);
}

void FAnimNode_PoseBlendNode::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	FAnimNode_PoseHandler::CacheBones_AnyThread(Context);
	SourcePose.CacheBones(Context);
}

void FAnimNode_PoseBlendNode::UpdateAssetPlayer(const FAnimationUpdateContext& Context)
{
	FAnimNode_PoseHandler::UpdateAssetPlayer(Context);
	SourcePose.Update(Context);
}

void FAnimNode_PoseBlendNode::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER(PoseBlendNodeEvaluate, !IsInGameThread());

	FPoseContext SourceData(Output);
	SourcePose.Evaluate(SourceData);

	bool bValidPose = false;

	const UPoseAsset* CachedPoseAsset = CurrentPoseAsset.Get();
	if (CachedPoseAsset && PoseExtractContext.PoseCurves.Num() > 0 && CachedPoseAsset->GetSkeleton() != nullptr)
	{
		FPoseContext CurrentPose(Output);

		// Remap incoming curve
		UE::Anim::FCurveUtils::BulkGet(SourceData.Curve, BulkCurves, [this](const UE::Anim::FNamedIndexElement& InBulkElement, float InValue)
		{
			// Remap using chosen BlendOption
			const float RemappedValue = FAlphaBlend::AlphaToBlendOption(InValue, BlendOption, CustomCurve);
			PoseExtractContext.PoseCurves[InBulkElement.Index].Value = RemappedValue;
		});

		FAnimationPoseData CurrentAnimationPoseData(CurrentPose);
		if (CachedPoseAsset->GetAnimationPose(CurrentAnimationPoseData, PoseExtractContext, bIgnorePoseAssetSourceAnimationCurves))
		{
			// once we get it, we have to blend by weight
			if (CachedPoseAsset->IsValidAdditive())
			{
				Output = SourceData;

				FAnimationPoseData BaseAnimationPoseData(Output);
				FAnimationRuntime::AccumulateAdditivePose(BaseAnimationPoseData, CurrentAnimationPoseData, 1.f, EAdditiveAnimationType::AAT_LocalSpaceBase);
			}
			else
			{

				FAnimationPoseData OutputAnimationPoseData(Output);
				const FAnimationPoseData SourceAnimationPoseData(SourceData);

				FAnimationRuntime::BlendTwoPosesTogetherPerBone(SourceAnimationPoseData, CurrentAnimationPoseData, BoneBlendWeights, OutputAnimationPoseData);
			}

			bValidPose = true;
		}
	}

	// If we didn't create a valid pose, just copy SourcePose to output (pass through)
	if(!bValidPose)
	{
		Output = SourceData;
	}
}

void FAnimNode_PoseBlendNode::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FAnimNode_PoseHandler::GatherDebugData(DebugData);
	SourcePose.GatherDebugData(DebugData.BranchFlow(1.f));
}

void FAnimNode_PoseBlendNode::RebuildPoseList(const FBoneContainer& InBoneContainer, const UPoseAsset* InPoseAsset)
{
	FAnimNode_PoseHandler::RebuildPoseList(InBoneContainer, InPoseAsset);
	
	BulkCurves.Empty();
	for (int32 PoseIdx = 0; PoseIdx < PoseExtractContext.PoseCurves.Num(); ++PoseIdx)
	{
		FPoseCurve& PoseCurve = PoseExtractContext.PoseCurves[PoseIdx];
		BulkCurves.Add(PoseCurve.Name, BulkCurves.Num());
	}
}