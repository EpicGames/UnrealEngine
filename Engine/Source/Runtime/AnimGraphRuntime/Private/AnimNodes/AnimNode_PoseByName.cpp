// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_PoseByName.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_PoseByName)

/////////////////////////////////////////////////////
// FAnimPoseByNameNode

void FAnimNode_PoseByName::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_PoseHandler::Initialize_AnyThread(Context);
}

void FAnimNode_PoseByName::RebuildPoseList(const FBoneContainer& InBoneContainer, const UPoseAsset* InPoseAsset)
{
	PoseExtractContext.PoseCurves.Reset();
	const TArray<FName>& PoseNames = InPoseAsset->GetPoseFNames();
	const int32 PoseIndex = InPoseAsset->GetPoseIndexByName(PoseName);
	if (PoseIndex != INDEX_NONE)
	{
		// we keep pose index as that is the fastest way to search when extracting pose asset
		PoseExtractContext.PoseCurves.Add(FPoseCurve(PoseIndex, PoseNames[PoseIndex], 0.f));
	}
}

void FAnimNode_PoseByName::UpdateAssetPlayer(const FAnimationUpdateContext& Context)
{
	FAnimNode_PoseHandler::UpdateAssetPlayer(Context);

	// update pose extraction context if the name differs
	if (CurrentPoseName != PoseName)
	{
		RebuildPoseList(Context.AnimInstanceProxy->GetRequiredBones(), PoseAsset);
		CurrentPoseName = PoseName;
	}

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Pose Asset"), CurrentPoseAsset.IsValid() ? *CurrentPoseAsset.Get()->GetName() : TEXT("None"));
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Pose"), *PoseName.ToString());
}

void FAnimNode_PoseByName::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	// make sure we have curve to eval
	const UPoseAsset* CachedPoseAsset = CurrentPoseAsset.Get();
	if (CachedPoseAsset && PoseExtractContext.PoseCurves.Num() > 0 && CurrentPoseAsset->GetSkeleton() != nullptr)
	{
		// we only have one 
		PoseExtractContext.PoseCurves[0].Value = PoseWeight;
		// only give pose curve, we don't set any more curve here

		FAnimationPoseData OutputAnimationPoseData(Output);
		CurrentPoseAsset->GetAnimationPose(OutputAnimationPoseData, PoseExtractContext);
	}
	else
	{
		Output.ResetToRefPose();
	}
}

void FAnimNode_PoseByName::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);
	
	DebugLine += FString::Printf(TEXT("('%s' Pose: %s)"), CurrentPoseAsset.IsValid()? *CurrentPoseAsset.Get()->GetName() : TEXT("None"), *PoseName.ToString());
	DebugData.AddDebugItem(DebugLine, true);
}

