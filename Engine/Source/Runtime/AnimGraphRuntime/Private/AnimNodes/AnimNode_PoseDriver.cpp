// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_PoseDriver.h"
#include "AnimationRuntime.h"
#include "Animation/AnimCurveUtils.h"
#include "Serialization/CustomVersion.h"
#include "Animation/AnimInstanceProxy.h"
#include "RBF/RBFSolver.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_PoseDriver)


FAnimNode_PoseDriver::FAnimNode_PoseDriver()
	: DriveSource(EPoseDriverSource::Rotation)
	, DriveOutput(EPoseDriverOutput::DrivePoses)
	, LODThreshold(INDEX_NONE)
{
	RBFParams.DistanceMethod = ERBFDistanceMethod::SwingAngle;

#if WITH_EDITORONLY_DATA
	SoloTargetIndex = INDEX_NONE;
	bSoloDrivenOnly = false;

	RadialScaling_DEPRECATED = 0.25f;
	Type_DEPRECATED = EPoseDriverType::SwingOnly;
	TwistAxis_DEPRECATED = BA_X;
#endif
}

void FAnimNode_PoseDriver::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_PoseHandler::Initialize_AnyThread(Context);

	SourcePose.Initialize(Context);
}

void FAnimNode_PoseDriver::RebuildPoseList(const FBoneContainer& InBoneContainer, const UPoseAsset* InPoseAsset)
{
	// Cache UIDs for driving curves
	PoseExtractContext.PoseCurves.Reset();
	const USkeleton* Skeleton = InPoseAsset->GetSkeleton();
	if (Skeleton)
	{
		const TArray<FName>& PoseNames = InPoseAsset->GetPoseFNames();
		for (FPoseDriverTarget& PoseTarget : PoseTargets)
		{
			const int32 PoseIndex = InPoseAsset->GetPoseIndexByName(PoseTarget.DrivenName);
			if (PoseIndex != INDEX_NONE)
			{
				// we keep pose index as that is the fastest way to search when extracting pose asset
				PoseTarget.PoseCurveIndex = PoseExtractContext.PoseCurves.Add(FPoseCurve(PoseIndex, PoseNames[PoseIndex], 0.f));
			}
			else
			{
				PoseTarget.PoseCurveIndex = INDEX_NONE;
			}
		}
	}
}

void FAnimNode_PoseDriver::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	FAnimNode_PoseHandler::CacheBones_AnyThread(Context);

	// Init pose input
	SourcePose.CacheBones(Context);

	const FBoneContainer& BoneContainer = Context.AnimInstanceProxy->GetRequiredBones();

	// Init bone refs
	for (FBoneReference& SourceBoneRef : SourceBones)
	{
		SourceBoneRef.Initialize(BoneContainer);
	}

	for (FBoneReference& OnlyDriveBoneRef : OnlyDriveBones)
	{
		OnlyDriveBoneRef.Initialize(BoneContainer);
	}

	EvalSpaceBone.Initialize(BoneContainer);

	// Don't want to modify SourceBones, set weight to zero (if weight array is allocated)
	for (FBoneReference& SourceBoneRef : SourceBones)
	{
		const FCompactPoseBoneIndex SourceCompactIndex = SourceBoneRef.GetCompactPoseIndex(BoneContainer);
		if (BoneBlendWeights.IsValidIndex(SourceCompactIndex.GetInt()))
		{
			BoneBlendWeights[SourceCompactIndex.GetInt()] = 0.f;
		}
	}

	// Check if there are valid OnlyDriveBones, since there might be a None entry.
	bHasOnlyDriveBones = false;
	for (const FBoneReference& BoneRef : OnlyDriveBones)
	{
		if (!BoneRef.BoneName.IsNone())
		{
			bHasOnlyDriveBones = true;
			break;
		}
	}

	// If we are filtering for only specific bones, set blend weight to zero for unwanted bones, and remember which bones to filter
	BonesToFilter.Reset();
	if (bHasOnlyDriveBones && CurrentPoseAsset.IsValid())
	{
		// Super call above should init BoneBlendWeights to compact pose size if CurrentPoseAsset is valid
		check(BoneBlendWeights.Num() == BoneContainer.GetBoneIndicesArray().Num());

		const TArray<FName> TrackNames = CurrentPoseAsset.Get()->GetTrackNames();
		for (const auto& TrackName : TrackNames)
		{
			// Check if bone in OnlyDriveBones
			if (!IsBoneDriven(TrackName))
			{
				int32 MeshBoneIndex = BoneContainer.GetPoseBoneIndexForBoneName(TrackName);
				FCompactPoseBoneIndex CompactBoneIndex = BoneContainer.MakeCompactPoseIndex(FMeshPoseBoneIndex(MeshBoneIndex));
				if (CompactBoneIndex != INDEX_NONE)
				{
					BoneBlendWeights[CompactBoneIndex.GetInt()] = 0.f; // Set blend weight for non-additive 
					BonesToFilter.Add(CompactBoneIndex); // Remember bones to filter out for additive
				}
			}
		}
	}

	PoseExtractContext.BonesRequired.SetNumZeroed(BoneBlendWeights.Num());
	for (int32 BoneIndex = 0; BoneIndex < BoneBlendWeights.Num(); BoneIndex++)
	{
		PoseExtractContext.BonesRequired[BoneIndex] = BoneBlendWeights[BoneIndex] > SMALL_NUMBER;
	}
}

void FAnimNode_PoseDriver::UpdateAssetPlayer(const FAnimationUpdateContext& Context)
{
	FAnimNode_PoseHandler::UpdateAssetPlayer(Context);
	SourcePose.Update(Context);
}

void FAnimNode_PoseDriver::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FAnimNode_PoseHandler::GatherDebugData(DebugData);
	SourcePose.GatherDebugData(DebugData.BranchFlow(1.f));
}

float FAnimNode_PoseDriver::GetRadiusForTarget(const FRBFTarget& Target) const
{
	return FRBFSolver::GetRadiusForTarget(Target, RBFParams);
}

bool FAnimNode_PoseDriver::IsBoneDriven(FName BoneName) const
{
	for (const FBoneReference& BoneRef : OnlyDriveBones)
	{
		if (BoneRef.BoneName == BoneName)
		{
			return true;
		}
	}

	return false;
}


void FAnimNode_PoseDriver::GetRBFTargets(TArray<FRBFTarget>& OutTargets, const FBoneContainer* BoneContainer) const
{
	OutTargets.Reset();
	OutTargets.AddZeroed(PoseTargets.Num());

	// Create entry for each target
	for (int32 TargetIdx = 0; TargetIdx < PoseTargets.Num(); TargetIdx++)
	{
		FRBFTarget& RBFTarget = OutTargets[TargetIdx];
		const FPoseDriverTarget& PoseTarget = PoseTargets[TargetIdx];

		// We want to make sure we always have the right number of Values in our RBFTarget. 
		// If bone entries are missing, we fill with zeroes
		for (int32 SourceIdx = 0; SourceIdx < SourceBones.Num(); SourceIdx++)
		{
			if (PoseTarget.BoneTransforms.IsValidIndex(SourceIdx))
			{
				const FPoseDriverTransform& BoneTransform = PoseTarget.BoneTransforms[SourceIdx];

				// Get Ref Transform
				FTransform RefBoneTransform = FTransform::Identity;
				if (bEvalFromRefPose && BoneContainer)
				{
					const FBoneReference& BoneReference = SourceBones[SourceIdx];
					if (BoneReference.HasValidSetup())
					{
						const FCompactPoseBoneIndex CompactPoseIndex = BoneReference.CachedCompactPoseIndex;
						if (CompactPoseIndex < BoneContainer->GetCompactPoseNumBones())
						{
							RefBoneTransform = BoneContainer->GetRefPoseTransform(CompactPoseIndex);
						}
					}
				}

				// Target Translation
				if (DriveSource == EPoseDriverSource::Translation)
				{
					// Make translation relative to its Ref
					if (bEvalFromRefPose)
					{
						RBFTarget.AddFromVector(RefBoneTransform.Inverse().TransformPosition(BoneTransform.TargetTranslation));
					}
					else
					{
						RBFTarget.AddFromVector(BoneTransform.TargetTranslation);
					}
				}

				// Target Rotation
				else
				{
					// Make rotation relative to its Ref
					if (bEvalFromRefPose)
					{	
						const FQuat TargetRotation = BoneTransform.TargetRotation.Quaternion();
						RBFTarget.AddFromRotator(RefBoneTransform.Inverse().TransformRotation(TargetRotation).Rotator());
					}
					else
					{
						RBFTarget.AddFromRotator(BoneTransform.TargetRotation);
					}
				}
			}
			else
			{
				RBFTarget.AddFromVector(FVector::ZeroVector);
			}
		}

		RBFTarget.ScaleFactor = PoseTarget.TargetScale;
		RBFTarget.bApplyCustomCurve = PoseTarget.bApplyCustomCurve;
		RBFTarget.CustomCurve = PoseTarget.CustomCurve;
		RBFTarget.DistanceMethod = PoseTarget.DistanceMethod;
		RBFTarget.FunctionType = PoseTarget.FunctionType;
	}
}


void FAnimNode_PoseDriver::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PoseDriver_Eval);

	if (!IsLODEnabled(Output.AnimInstanceProxy))
	{
		SourcePose.Evaluate(Output);
		return;
	}

	FPoseContext SourceData(Output);
	SourcePose.Evaluate(SourceData);

	// Udpate DrivenIDs if needed
	if (bCachedDrivenIDsAreDirty)
	{
		if (CurrentPoseAsset.IsValid())
		{
			RebuildPoseList(Output.AnimInstanceProxy->GetRequiredBones(), CurrentPoseAsset.Get());
		}
	}

	// Get the index of the source bone
	const FBoneContainer& BoneContainer = SourceData.Pose.GetBoneContainer();

	RBFInput.Values.Reset();

	SourceBoneTMs.Reset();
	bool bFoundAnyBone = false;
	for (const FBoneReference& SourceBoneRef : SourceBones)
	{
		FTransform SourceBoneTM = FTransform::Identity;

		const FCompactPoseBoneIndex SourceCompactIndex = SourceBoneRef.GetCompactPoseIndex(BoneContainer);
		if (SourceCompactIndex.GetInt() != INDEX_NONE)
		{
			// If evaluating in alternative bone space, have to build component space pose
			if (EvalSpaceBone.IsValidToEvaluate(BoneContainer))
			{
				FCSPose<FCompactPose> CSPose;
				CSPose.InitPose(SourceData.Pose);

				const FCompactPoseBoneIndex EvalSpaceCompactIndex = EvalSpaceBone.GetCompactPoseIndex(BoneContainer);
				FTransform EvalSpaceCompSpace = CSPose.GetComponentSpaceTransform(EvalSpaceCompactIndex);
				FTransform SourceBoneCompSpace = CSPose.GetComponentSpaceTransform(SourceCompactIndex);

				SourceBoneTM = SourceBoneCompSpace.GetRelativeTransform(EvalSpaceCompSpace);
			}
			// If just evaluating in local space, just grab from local space pose
			else
			{
				// Relative to Ref Pose
				if (bEvalFromRefPose && SourceCompactIndex.GetInt() < BoneContainer.GetCompactPoseNumBones())
				{
					SourceBoneTM = SourceData.Pose[SourceCompactIndex].GetRelativeTransform(BoneContainer.GetRefPoseTransform(SourceCompactIndex));
				}
				else
				{
					SourceBoneTM = SourceData.Pose[SourceCompactIndex];
				}
			}

			bFoundAnyBone = true;
		}


		// Build RBFInput entry
		if (DriveSource == EPoseDriverSource::Translation)
		{
			RBFInput.AddFromVector(SourceBoneTM.GetTranslation());
		}
		else
		{
			RBFInput.AddFromRotator(SourceBoneTM.Rotator());
		}

		// Record this so we can use it for drawing in edit mode
		SourceBoneTMs.Add(SourceBoneTM);
	}

	// Do nothing if bone is no bones are found/all LOD-ed out
	if (!bFoundAnyBone)
	{
		Output = SourceData;
		return;
	}

	RBFParams.TargetDimensions = SourceBones.Num() * 3;

	OutputWeights.Reset();
	
	// Use SoloTarget, no need to Solve
#if WITH_EDITORONLY_DATA
	if (PoseTargets.IsValidIndex(SoloTargetIndex))
	{
		OutputWeights.Add(FRBFOutputWeight(SoloTargetIndex, 1.0f));
	}
	else
#endif
	// Solve Weights
	{
		// Get target array as RBF types
		GetRBFTargets(RBFTargets, &BoneContainer);

		if (!SolverData.IsValid() || !FRBFSolver::IsSolverDataValid(*SolverData, RBFParams, RBFTargets))
		{
			SolverData = FRBFSolver::InitSolver(RBFParams, RBFTargets);
		}

		// Run RBF solver
		FRBFSolver::Solve(*SolverData, RBFParams, RBFTargets, RBFInput, OutputWeights);
	}

	// Track if we have filled Output with valid pose
	bool bHaveValidPose = false;

	// Process active targets (if any)
	if (OutputWeights.Num() > 0)
	{
		// Drive Poses from PoseAsset
		if (DriveOutput == EPoseDriverOutput::DrivePoses)
		{
			const UPoseAsset* CachedPoseAsset = CurrentPoseAsset.Get();
				
			// Check if PoseAsset is assignedand and compatible
			if (CachedPoseAsset && CachedPoseAsset->GetSkeleton() != nullptr)
			{
				// clear the value before setting it. 
				for (int32 PoseIndex = 0; PoseIndex < PoseExtractContext.PoseCurves.Num(); ++PoseIndex)
				{
					PoseExtractContext.PoseCurves[PoseIndex].Value = 0.f;
				}

				// Then fill in weight for any driven poses
				for (const FRBFOutputWeight& Weight : OutputWeights)
				{
					const FPoseDriverTarget& PoseTarget = PoseTargets[Weight.TargetIndex];
					const int32 PoseIndex = PoseTarget.PoseCurveIndex;
					if (PoseIndex != INDEX_NONE)
					{
						PoseExtractContext.PoseCurves[PoseIndex].Value = Weight.TargetWeight;
					}
				}

				FPoseContext CurrentPose(Output);
				FAnimationPoseData CurrentAnimationPoseData(CurrentPose);

				// Evaluate PoseAsset
				if (CachedPoseAsset->GetAnimationPose(CurrentAnimationPoseData, PoseExtractContext))
				{
					// If Additive, Set Source and OnlyDrive Bones to Zero and Accumulate Pose
					if (CurrentPoseAsset->IsValidAdditive())
					{
						const FTransform AdditiveIdentity(FQuat::Identity, FVector::ZeroVector, FVector::ZeroVector);

						// Don't want to modify SourceBones, set additive offset to zero (not identity transform, as need zero scale)
						for (const FBoneReference& SourceBoneRef : SourceBones)
						{
							const FCompactPoseBoneIndex SourceCompactIndex = SourceBoneRef.GetCompactPoseIndex(BoneContainer);
							CurrentPose.Pose[SourceCompactIndex] = AdditiveIdentity;
						}

						// If filtering for specific bones, filter out bones using BonesToFilter array
						if (bHasOnlyDriveBones)
						{
							for (FCompactPoseBoneIndex BoneIndex : BonesToFilter)
							{
								CurrentPose.Pose[BoneIndex] = AdditiveIdentity;
							}
						}

						// Start by copying input
						Output = SourceData;

						FAnimationPoseData AccumulatedAnimationPoseData(Output); // out
						FAnimationRuntime::AccumulateAdditivePose(AccumulatedAnimationPoseData, CurrentAnimationPoseData, 1.f, EAdditiveAnimationType::AAT_LocalSpaceBase);
					}

					// If Non-Additive, Blend between Source and Current filtering with OnlyDrive Bones
					else
					{
						const FAnimationPoseData SourceAnimationPoseData(SourceData);
						FAnimationPoseData BlendedAnimationPoseData(Output); // out
						FAnimationRuntime::BlendTwoPosesTogetherPerBone(SourceAnimationPoseData, CurrentAnimationPoseData, BoneBlendWeights, BlendedAnimationPoseData);
					}

					bHaveValidPose = true;
				}
			}
		}

		// Drive curves (morphs, materials etc)
		else if (DriveOutput == EPoseDriverOutput::DriveCurves)
		{
			// Start by copying input
			Output = SourceData;

			// Then set curves based on target weights
			FBlendedCurve DrivenCurves;
			for (const FRBFOutputWeight& Weight : OutputWeights)
			{
				FPoseDriverTarget& PoseTarget = PoseTargets[Weight.TargetIndex];
				if (PoseTarget.DrivenName != NAME_None)
				{
					DrivenCurves.Add(PoseTarget.DrivenName, Weight.TargetWeight);
				}
			}

			// Merge driven curves into output
			UE::Anim::FNamedValueArrayUtils::Union(Output.Curve, DrivenCurves, [](UE::Anim::FCurveElement& InOutElement, const UE::Anim::FCurveElement& InElement0, UE::Anim::ENamedValueUnionFlags InFlags)
			{
				if(EnumHasAnyFlags(InFlags, UE::Anim::ENamedValueUnionFlags::ValidArg1))
				{
					InOutElement.Value = InElement0.Value;
					InOutElement.Flags |= InElement0.Flags;
				}
			});

			bHaveValidPose = true;
		}
	}

	// No valid pose, just pass through
	if (!bHaveValidPose)
	{
		Output = SourceData;
	}

#if WITH_EDITORONLY_DATA
	else if (!bSoloDrivenOnly && PoseTargets.IsValidIndex(SoloTargetIndex))
	{
		SourceBoneTMs.Reset();
		const FPoseDriverTarget& PoseTarget = PoseTargets[SoloTargetIndex];

		for (int32 SourceIdx = 0; SourceIdx < SourceBones.Num(); SourceIdx++)
		{
			const FBoneReference& SourceBoneRef = SourceBones[SourceIdx];
			const FCompactPoseBoneIndex SourceCompactIndex = SourceBoneRef.GetCompactPoseIndex(BoneContainer);

			if (PoseTarget.BoneTransforms.IsValidIndex(SourceIdx) && SourceCompactIndex.GetInt() != INDEX_NONE)
			{
				FTransform& TargetTransform = Output.Pose[SourceCompactIndex];
				const FPoseDriverTransform& SourceTransform = PoseTarget.BoneTransforms[SourceIdx];

				if (DriveSource == EPoseDriverSource::Translation)
				{
					TargetTransform.SetTranslation(SourceTransform.TargetTranslation);
				}
				else
				{
					TargetTransform.SetRotation(SourceTransform.TargetRotation.Quaternion());
				}
				SourceBoneTMs.Add(TargetTransform);
			}
		}
	}
#endif
}

