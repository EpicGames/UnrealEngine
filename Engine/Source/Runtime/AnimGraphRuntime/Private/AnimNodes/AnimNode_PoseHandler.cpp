// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_PoseHandler.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_PoseHandler)

/////////////////////////////////////////////////////
// FAnimPoseByNameNode

void FAnimNode_PoseHandler::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_AssetPlayerBase::Initialize_AnyThread(Context);

	UpdatePoseAssetProperty(Context.AnimInstanceProxy);
}

void FAnimNode_PoseHandler::CacheBoneBlendWeights(FAnimInstanceProxy* InstanceProxy)
{
	BoneBlendWeights.Reset();

	const FBoneContainer& BoneContainer = InstanceProxy->GetRequiredBones();

	// this has to update bone blending weight
	if (CurrentPoseAsset.IsValid() && BoneContainer.IsValid())
	{
		const UPoseAsset* CurrentAsset = CurrentPoseAsset.Get();
		const TArray<FName>& TrackNames = CurrentAsset->GetTrackNames();
		const TArray<FBoneIndexType>& RequiredBoneIndices = BoneContainer.GetBoneIndicesArray();
		BoneBlendWeights.AddZeroed(RequiredBoneIndices.Num());

		for (const auto& TrackName : TrackNames)
		{
			int32 MeshBoneIndex = BoneContainer.GetPoseBoneIndexForBoneName(TrackName);
			FCompactPoseBoneIndex CompactBoneIndex = BoneContainer.MakeCompactPoseIndex(FMeshPoseBoneIndex(MeshBoneIndex));
			if (CompactBoneIndex != INDEX_NONE)
			{
				BoneBlendWeights[CompactBoneIndex.GetInt()] = 1.f;
			}
		}

		RebuildPoseList(BoneContainer, CurrentAsset);
	}
	else
	{
		PoseExtractContext.PoseCurves.Reset();
	}
}

void FAnimNode_PoseHandler::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	FAnimNode_AssetPlayerBase::CacheBones_AnyThread(Context);

	CacheBoneBlendWeights(Context.AnimInstanceProxy);
}

void FAnimNode_PoseHandler::RebuildPoseList(const FBoneContainer& InBoneContainer, const UPoseAsset* InPoseAsset)
{
	PoseExtractContext.PoseCurves.Reset();
	const TArray<FName>& PoseNames = InPoseAsset->GetPoseFNames();
	const int32 TotalPoseNum = PoseNames.Num();
	if (TotalPoseNum > 0)
	{
		for (int32 PoseIndex = 0; PoseIndex < PoseNames.Num(); ++PoseIndex)
		{
			const FName& PoseName = PoseNames[PoseIndex];

			// we keep pose index as that is the fastest way to search when extracting pose asset
			PoseExtractContext.PoseCurves.Add(FPoseCurve(PoseIndex, PoseName, 0.f));
		}
	}
}

void FAnimNode_PoseHandler::UpdateAssetPlayer(const FAnimationUpdateContext& Context)
{
	GetEvaluateGraphExposedInputs().Execute(Context);

	// update pose asset if it's not valid
	if (CurrentPoseAsset.IsValid() == false || CurrentPoseAsset.Get() != PoseAsset)
	{
		UpdatePoseAssetProperty(Context.AnimInstanceProxy);
	}

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Name"), CurrentPoseAsset.IsValid() ? *CurrentPoseAsset.Get()->GetName() : TEXT("None"));
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Pose Asset"), CurrentPoseAsset.IsValid() ? *CurrentPoseAsset.Get()->GetName() : TEXT("None"));
}

#if WITH_EDITORONLY_DATA
void FAnimNode_PoseHandler::SetPoseAsset(UPoseAsset* InPoseAsset)
{
	PoseAsset = InPoseAsset;
}
#endif

void FAnimNode_PoseHandler::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);
	
	DebugLine += FString::Printf(TEXT("('%s')"), *GetNameSafe(PoseAsset));
	DebugData.AddDebugItem(DebugLine, true);
}

void FAnimNode_PoseHandler::UpdatePoseAssetProperty(struct FAnimInstanceProxy* InstanceProxy)
{
	CurrentPoseAsset = PoseAsset;
	CacheBoneBlendWeights(InstanceProxy);
}


