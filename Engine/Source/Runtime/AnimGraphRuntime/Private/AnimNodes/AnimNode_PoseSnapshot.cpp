// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_PoseSnapshot.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimTrace.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_PoseSnapshot)

/////////////////////////////////////////////////////
// FAnimNode_PoseSnapshot

FAnimNode_PoseSnapshot::FAnimNode_PoseSnapshot()
	: SnapshotName(NAME_None)
	, Mode(ESnapshotSourceMode::NamedSnapshot)
	, MappedSourceMeshName(NAME_None)
	, MappedTargetMeshName(NAME_None)
	, TargetBoneNameMesh(NAME_None)
{
}

void FAnimNode_PoseSnapshot::PreUpdate(const UAnimInstance* InAnimInstance)
{
	// cache the currently used skeletal mesh's bone names
	USkeletalMesh* CurrentSkeletalMesh = nullptr;
	if (InAnimInstance->GetSkelMeshComponent() && InAnimInstance->GetSkelMeshComponent()->IsRegistered())
	{
		CurrentSkeletalMesh = InAnimInstance->GetSkelMeshComponent()->GetSkeletalMeshAsset();
	}

	if (CurrentSkeletalMesh)
	{
		if (TargetBoneNameMesh != CurrentSkeletalMesh->GetFName())
		{
			// cache bone names for the target mesh
			TargetBoneNames.Reset();
			TargetBoneNames.AddDefaulted(CurrentSkeletalMesh->GetRefSkeleton().GetNum());

			for (int32 BoneIndex = 0; BoneIndex < CurrentSkeletalMesh->GetRefSkeleton().GetNum(); ++BoneIndex)
			{
				TargetBoneNames[BoneIndex] = CurrentSkeletalMesh->GetRefSkeleton().GetBoneName(BoneIndex);
			}
		}

		TargetBoneNameMesh = CurrentSkeletalMesh->GetFName();
	}
	else
	{
		TargetBoneNameMesh = NAME_None;
	}
}

void FAnimNode_PoseSnapshot::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)
	// Evaluate any BP logic plugged into this node
	GetEvaluateGraphExposedInputs().Execute(Context);

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Snapshot Name"), Snapshot.SnapshotName);
}

void FAnimNode_PoseSnapshot::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	FCompactPose& OutPose = Output.Pose;
	OutPose.ResetToRefPose();

	switch (Mode)
	{
		case ESnapshotSourceMode::NamedSnapshot:
		{
			if (const FPoseSnapshot* PoseSnapshotData = Output.AnimInstanceProxy->GetPoseSnapshot(SnapshotName))
			{
				ApplyPose(*PoseSnapshotData, OutPose);
			}
		}
		break;
		case ESnapshotSourceMode::SnapshotPin:
		{
			ApplyPose(Snapshot, OutPose);
		}
		break;
	}
}

void FAnimNode_PoseSnapshot::ApplyPose(const FPoseSnapshot& PoseSnapshot, FCompactPose& OutPose)
{
	const TArray<FTransform>& LocalTMs = PoseSnapshot.LocalTransforms;
	const FBoneContainer& RequiredBones = OutPose.GetBoneContainer();

	if (TargetBoneNameMesh == PoseSnapshot.SkeletalMeshName)
	{
		for (FCompactPoseBoneIndex PoseBoneIndex : OutPose.ForEachBoneIndex())
		{
			const FMeshPoseBoneIndex MeshBoneIndex = RequiredBones.MakeMeshPoseIndex(PoseBoneIndex);
			const int32 Index = MeshBoneIndex.GetInt();

			if (LocalTMs.IsValidIndex(Index))
			{
				OutPose[PoseBoneIndex] = LocalTMs[Index];
			}
		}
	}
	else
	{
		// per-bone matching required
		CacheBoneMapping(PoseSnapshot.SkeletalMeshName, TargetBoneNameMesh, PoseSnapshot.BoneNames, TargetBoneNames);

		for (FCompactPoseBoneIndex PoseBoneIndex : OutPose.ForEachBoneIndex())
		{
			const FMeshPoseBoneIndex MeshBoneIndex = RequiredBones.MakeMeshPoseIndex(PoseBoneIndex);
			const int32 Index = MeshBoneIndex.GetInt();

			if (SourceBoneMapping.IsValidIndex(Index))
			{
				const int32 SourceBoneIndex = SourceBoneMapping[Index];
				if(LocalTMs.IsValidIndex(SourceBoneIndex))
				{
					OutPose[PoseBoneIndex] = LocalTMs[SourceBoneIndex];
				}
			}
		}
	}
}

void FAnimNode_PoseSnapshot::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this) + " Snapshot Name:" + SnapshotName.ToString();
	DebugData.AddDebugItem(DebugLine, true);
}

void FAnimNode_PoseSnapshot::CacheBoneMapping(FName InSourceMeshName, FName InTargetMeshName, const TArray<FName>& InSourceBoneNames, const TArray<FName>& InTargetBoneNames)
{
	if (InSourceMeshName != MappedSourceMeshName || InTargetMeshName != MappedTargetMeshName)
	{
		SourceBoneMapping.Reset();

		for (int32 BoneIndex = 0; BoneIndex < InTargetBoneNames.Num(); ++BoneIndex)
		{
			SourceBoneMapping.Add(InSourceBoneNames.IndexOfByKey(InTargetBoneNames[BoneIndex]));
		}

		MappedSourceMeshName = InSourceMeshName;
		MappedTargetMeshName = InTargetMeshName;
	}
}

