// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_RandomPlayer.h"

#include "Algo/BinarySearch.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimTrace.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimSyncScope.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_RandomPlayer)

FAnimNode_RandomPlayer::FAnimNode_RandomPlayer()
    : CurrentPlayDataIndex(0)
    , bShuffleMode(false)
{
}

void FAnimNode_RandomPlayer::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_Base::Initialize_AnyThread(Context);
	GetEvaluateGraphExposedInputs().Execute(Context);

	// Create a sanitized list of valid entries and only operate on those from here on in.
	ValidEntries.Empty(Entries.Num());
	for (int32 EntryIndex = 0; EntryIndex < Entries.Num(); EntryIndex++)
	{
		FRandomPlayerSequenceEntry* Entry = &Entries[EntryIndex];

		if (Entry->Sequence == nullptr)
		{
			continue;
		}

		// If the likelihood of this entry playing is nil, then skip it as well.
		if (!bShuffleMode && Entry->ChanceToPlay <= SMALL_NUMBER)
		{
			continue;
		}

		ValidEntries.Push(Entry);
	}

	const int32 NumValidEntries = ValidEntries.Num();

	if (NumValidEntries == 0)
	{
		// early out here, no need to do anything at all if we're not playing anything
		return;
	}

	NormalizedPlayChances.Empty(NormalizedPlayChances.Num());
	NormalizedPlayChances.AddUninitialized(NumValidEntries);

	// Sanitize the data and sum up the range of the random chances so that
	// we can normalize the individual chances below.
	float SumChances = 0.0f;
	for (FRandomPlayerSequenceEntry* Entry : ValidEntries)
	{
		SumChances += Entry->ChanceToPlay;

		if (Entry->MaxLoopCount < Entry->MinLoopCount)
		{
			Swap(Entry->MaxLoopCount, Entry->MinLoopCount);
		}

		if (Entry->MaxPlayRate < Entry->MinPlayRate)
		{
			Swap(Entry->MaxPlayRate, Entry->MinPlayRate);
		}

		Entry->BlendIn.Reset();
	}

	if (bShuffleMode)
	{
		// Seed the shuffle list, ignoring all last entry checks, since we're doing the
		// initial build and don't care about the non-repeatability property (yet).
		BuildShuffleList(INDEX_NONE);
	}
	else
	{
		// Ensure that our chance sum is non-"zero" and non-negative.
		check(SumChances > SMALL_NUMBER);

		// Construct a cumulative distribution function so that we can look up the
		// index of the sequence using binary search on the [0-1) random number.
		float CurrentChance = 0.0f;
		for (int32 Idx = 0; Idx < NumValidEntries; ++Idx)
		{
			CurrentChance += ValidEntries[Idx]->ChanceToPlay / SumChances;
			NormalizedPlayChances[Idx] = CurrentChance;
		}
		// Remove rounding errors (possibly slightly padding out the chance of the last item)
		NormalizedPlayChances[NumValidEntries - 1] = 1.0f;
	}

	// Initialize random stream and pick first entry
	RandomStream.Initialize(FPlatformTime::Cycles());

	PlayData.Empty(2);
	PlayData.AddDefaulted(2);

	int32 CurrentEntry = GetNextValidEntryIndex();
	int32 NextEntry = GetNextValidEntryIndex();

	// Initialize the animation data for the first and the next sequence so that we can properly
	// blend between them.
	FRandomAnimPlayData& CurrentData = GetPlayData(ERandomDataIndexType::Current);
	InitPlayData(CurrentData, CurrentEntry, 1.0f);

	FRandomAnimPlayData& NextData = GetPlayData(ERandomDataIndexType::Next);
	InitPlayData(NextData, NextEntry, 0.0f);
}

void FAnimNode_RandomPlayer::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	BlendWeight = Context.GetFinalBlendWeight();

	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)
	GetEvaluateGraphExposedInputs().Execute(Context);

	if (ValidEntries.Num() == 0)
	{
		// We don't have any entries, play data will be invalid - early out
		return;
	}

	FRandomAnimPlayData* CurrentData = &GetPlayData(ERandomDataIndexType::Current);
	FRandomAnimPlayData* NextData = &GetPlayData(ERandomDataIndexType::Next);

	const UAnimSequenceBase* CurrentSequence = CurrentData->Entry->Sequence;

	// If we looped around, adjust the previous play time to always be before the current playtime,
	// since we can assume modulo. This makes the crossing check for the start time a lot simpler.
	float AdjustedPreviousPlayTime = CurrentData->DeltaTimeRecord.GetPrevious();
	if (CurrentData->CurrentPlayTime < AdjustedPreviousPlayTime)
	{
		AdjustedPreviousPlayTime -= CurrentSequence->GetPlayLength();
	}

	// Did we cross the play start time? Decrement the loop counter. Once we're on the last loop, we can
	// start blending into the next animation.
	bool bHasLooped = AdjustedPreviousPlayTime < CurrentData->PlayStartTime && CurrentData->PlayStartTime <= CurrentData->CurrentPlayTime;
	if (bHasLooped)
	{
		// We've looped, update remaining
		--CurrentData->RemainingLoops;
	}

	bool bAdvanceToNextEntry = false;

	if (CurrentData->RemainingLoops <= 0)
	{
		const bool bNextAnimIsDifferent = CurrentData->Entry != NextData->Entry;

		// If we're in the blend window start blending, but only if we're moving to a new animation,
		// otherwise just keep looping.
		FRandomPlayerSequenceEntry& NextSequenceEntry = *NextData->Entry;

		// If the next animation is different, then smoothly blend between them. Otherwise
		// we do a hard transition to the same play point. The next animation might play at
		// a different rate, so we have to switch.
		if (bNextAnimIsDifferent)
		{
			bool bDoBlending = false;

			// Are we already blending? Continue to do so. Special case for zero blend time as alpha will always be 1.
			if (NextSequenceEntry.BlendIn.GetBlendTime() > 0.0f && FAnimationRuntime::HasWeight(NextSequenceEntry.BlendIn.GetAlpha()))
			{
				bDoBlending = true;
			}
			else
			{
				// Check to see if we need to start the blending process.
				float AmountPlayedSoFar = CurrentData->CurrentPlayTime - CurrentData->PlayStartTime;
				if (AmountPlayedSoFar < 0.0f)
				{
					AmountPlayedSoFar += CurrentSequence->GetPlayLength();
				}

				float TimeRemaining = CurrentSequence->GetPlayLength() - AmountPlayedSoFar;

				if (TimeRemaining <= NextSequenceEntry.BlendIn.GetBlendTime() || bHasLooped)
				{
					bDoBlending = true;
				}
			}

			if (bDoBlending)
			{
				// Blending to next
				NextSequenceEntry.BlendIn.Update(Context.GetDeltaTime());

				if (NextSequenceEntry.BlendIn.IsComplete())
				{
					// Set the play start time to be the current play time so that loop counts are properly
					// maintained.
					NextData->PlayStartTime = NextData->CurrentPlayTime;
					bAdvanceToNextEntry = true;
				}
				else
				{
					float BlendedAlpha = NextSequenceEntry.BlendIn.GetBlendedValue();

					if (BlendedAlpha < 1.0f)
					{
						NextData->BlendWeight = BlendedAlpha;
						CurrentData->BlendWeight = 1.0f - BlendedAlpha;
					}
				}
			}
		}
		else if (!bNextAnimIsDifferent && CurrentData->RemainingLoops < 0)
		{
			NextData->CurrentPlayTime = CurrentData->CurrentPlayTime;

			// Set the play start time to be the current play time so that loop counts are properly
			// maintained.
			NextData->PlayStartTime = NextData->CurrentPlayTime;
			bAdvanceToNextEntry = true;
		}
	}

	// Cache time to detect loops
	CurrentData->DeltaTimeRecord.SetPrevious(CurrentData->CurrentPlayTime);
	NextData->DeltaTimeRecord.SetPrevious(NextData->CurrentPlayTime);

	if (bAdvanceToNextEntry)
	{
		AdvanceToNextSequence();

		// Re-get data as we've switched over
		CurrentData = &GetPlayData(ERandomDataIndexType::Current);
		NextData = &GetPlayData(ERandomDataIndexType::Next);
	}

	FAnimTickRecord TickRecord(CurrentData->Entry->Sequence, true, CurrentData->PlayRate, false, CurrentData->BlendWeight * BlendWeight, CurrentData->CurrentPlayTime, CurrentData->MarkerTickRecord);
	TickRecord.DeltaTimeRecord = &CurrentData->DeltaTimeRecord;
#if ANIM_TRACE_ENABLED
	TickRecord.SourceProxy = Context.AnimInstanceProxy;
#endif
	TickRecord.GatherContextData(Context);

	UE::Anim::FAnimSyncGroupScope& SyncScope = Context.GetMessageChecked<UE::Anim::FAnimSyncGroupScope>();
	SyncScope.AddTickRecord(TickRecord, UE::Anim::FAnimSyncParams(), UE::Anim::FAnimSyncDebugInfo(Context));

	if (FAnimationRuntime::HasWeight(NextData->BlendWeight))
	{
		FAnimTickRecord NextTickRecord(NextData->Entry->Sequence, true, NextData->PlayRate, false, NextData->BlendWeight * BlendWeight, NextData->CurrentPlayTime, NextData->MarkerTickRecord);
		NextTickRecord.DeltaTimeRecord = &NextData->DeltaTimeRecord;
#if ANIM_TRACE_ENABLED
		NextTickRecord.SourceProxy = Context.AnimInstanceProxy;
#endif
		NextTickRecord.GatherContextData(Context);

		SyncScope.AddTickRecord(NextTickRecord, UE::Anim::FAnimSyncParams(), UE::Anim::FAnimSyncDebugInfo(Context));
	}

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Current Sequence"), CurrentData ? CurrentData->Entry->Sequence : nullptr);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Current Weight"), CurrentData ? CurrentData->BlendWeight : 0.0f);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Next Sequence"), NextData ? NextData->Entry->Sequence : nullptr);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Next Weight"), NextData ? NextData->BlendWeight : 0.0f);
}

void FAnimNode_RandomPlayer::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(RandomPlayer, !IsInGameThread());

	if (ValidEntries.Num() == 0)
	{
		Output.ResetToRefPose();
		return;
	}

	FRandomAnimPlayData& CurrentData = GetPlayData(ERandomDataIndexType::Current);
	FRandomAnimPlayData& NextData = GetPlayData(ERandomDataIndexType::Next);

	UAnimSequenceBase* CurrentSequence = CurrentData.Entry->Sequence;

	if (!FMath::IsNearlyEqualByULP(CurrentData.BlendWeight, 1.0f))
	{
		FAnimInstanceProxy* AnimProxy = Output.AnimInstanceProxy;

		// Start Blending
		FCompactPose Poses[2];
		FBlendedCurve Curves[2];
		UE::Anim::FStackAttributeContainer Attributes[2];
		float Weights[2];

		const FBoneContainer& RequiredBone = AnimProxy->GetRequiredBones();
		Poses[0].SetBoneContainer(&RequiredBone);
		Poses[1].SetBoneContainer(&RequiredBone);

		Curves[0].InitFrom(RequiredBone);
		Curves[1].InitFrom(RequiredBone);

		Weights[0] = CurrentData.BlendWeight;
		Weights[1] = NextData.BlendWeight;

		UAnimSequenceBase* NextSequence = NextData.Entry->Sequence;

		FAnimationPoseData CurrentPoseData(Poses[0], Curves[0], Attributes[0]);
		FAnimationPoseData NextPoseData(Poses[1], Curves[1], Attributes[1]);

		CurrentSequence->GetAnimationPose(CurrentPoseData, FAnimExtractContext(static_cast<double>(CurrentData.CurrentPlayTime), AnimProxy->ShouldExtractRootMotion(), CurrentData.DeltaTimeRecord, CurrentData.RemainingLoops > 0));
		NextSequence->GetAnimationPose(NextPoseData, FAnimExtractContext(static_cast<double>(NextData.CurrentPlayTime), AnimProxy->ShouldExtractRootMotion(), NextData.DeltaTimeRecord, NextData.RemainingLoops > 0));

		FAnimationPoseData AnimationPoseData(Output);
		FAnimationRuntime::BlendPosesTogether(Poses, Curves, Attributes, Weights, AnimationPoseData);
	}
	else
	{
		// Single animation, no blending needed.
		FAnimationPoseData AnimationPoseData(Output);
		CurrentSequence->GetAnimationPose(AnimationPoseData, FAnimExtractContext(static_cast<double>(CurrentData.CurrentPlayTime), Output.AnimInstanceProxy->ShouldExtractRootMotion(), CurrentData.DeltaTimeRecord, CurrentData.RemainingLoops > 0));
	}
}

void FAnimNode_RandomPlayer::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	DebugData.AddDebugItem(DebugLine, true);
}

UAnimationAsset* FAnimNode_RandomPlayer::GetAnimAsset() const
{
	UAnimationAsset* AnimationAsset = nullptr;

	if (ValidEntries.Num() > 0)
	{
		const FRandomAnimPlayData& CurrentPlayData = GetPlayData(ERandomDataIndexType::Current);
		AnimationAsset = (CurrentPlayData.Entry != nullptr) ? CurrentPlayData.Entry->Sequence.Get() : nullptr;
	}

	return AnimationAsset;
}

float FAnimNode_RandomPlayer::GetAccumulatedTime() const
{
	float AccumulatedTime = 0.f;

	if (ValidEntries.Num() > 0)
	{
		const FRandomAnimPlayData& CurrentPlayData = GetPlayData(ERandomDataIndexType::Current);

		return CurrentPlayData.CurrentPlayTime;
	}

	return AccumulatedTime;
}

bool FAnimNode_RandomPlayer::GetIgnoreForRelevancyTest() const
{
	return GET_ANIM_NODE_DATA(bool, bIgnoreForRelevancyTest);
}

bool FAnimNode_RandomPlayer::SetIgnoreForRelevancyTest(bool bInIgnoreForRelevancyTest)
{
#if WITH_EDITORONLY_DATA
	bIgnoreForRelevancyTest = bInIgnoreForRelevancyTest;
#endif

	if (bool* bIgnoreForRelevancyTestPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(bool, bIgnoreForRelevancyTest))
	{
		*bIgnoreForRelevancyTestPtr = bInIgnoreForRelevancyTest;
		return true;
	}

	return false;
}

float FAnimNode_RandomPlayer::GetCachedBlendWeight() const
{
	return BlendWeight;
}

void FAnimNode_RandomPlayer::ClearCachedBlendWeight()
{
	BlendWeight = 0.f;
}

const FDeltaTimeRecord* FAnimNode_RandomPlayer::GetDeltaTimeRecord() const
{
	if (ValidEntries.Num() > 0)
	{
		const FRandomAnimPlayData& CurrentPlayData = GetPlayData(ERandomDataIndexType::Current);
		return &CurrentPlayData.DeltaTimeRecord;
	}
	return nullptr;
}

int32 FAnimNode_RandomPlayer::GetNextValidEntryIndex()
{
	check(ValidEntries.Num() > 0);

	if (bShuffleMode)
	{
		// Get the top value, don't allow realloc
		int32 Index = ShuffleList.Pop(EAllowShrinking::No);

		// If we cleared the shuffles, rebuild for the next round, indicating
		// the current value so that we don't pop that one off again next time.
		if (ShuffleList.Num() == 0)
		{
			BuildShuffleList(Index);
		}

		return Index;
	}
	else
	{
		float RandomVal = RandomStream.GetFraction();

		// Search the cumulative distribution array for the last entry that's
		// smaller or equal to the random value. That becomes our new animation.
		return Algo::UpperBound(NormalizedPlayChances, RandomVal);
	}
}

FRandomAnimPlayData& FAnimNode_RandomPlayer::GetPlayData(ERandomDataIndexType Type)
{
	// PlayData only holds two entries. We swap between them in AdvanceToNextSequence
	// by setting CUrrentPlayDataIndex to either 0 or 1. Hence the modulo 2 magic below.
	if (Type == ERandomDataIndexType::Current)
	{
		return PlayData[CurrentPlayDataIndex];
	}
	else
	{
		return PlayData[(CurrentPlayDataIndex + 1) % 2];
	}
}

const FRandomAnimPlayData& FAnimNode_RandomPlayer::GetPlayData(ERandomDataIndexType Type) const
{
	return const_cast<FAnimNode_RandomPlayer*>(this)->GetPlayData(Type);
}

void FAnimNode_RandomPlayer::InitPlayData(FRandomAnimPlayData& Data, int32 InValidEntryIndex, float InBlendWeight)
{
	FRandomPlayerSequenceEntry* Entry = ValidEntries[InValidEntryIndex];

	Data.Entry = Entry;
	Data.BlendWeight = InBlendWeight;
	Data.PlayRate = static_cast<float>(RandomStream.FRandRange(Entry->MinPlayRate, Entry->MaxPlayRate));
	Data.RemainingLoops = FMath::Clamp(RandomStream.RandRange(Entry->MinLoopCount, Entry->MaxLoopCount), 0, MAX_int32);

	Data.PlayStartTime = 0.0f;
	Data.CurrentPlayTime = 0.0f;
	Data.DeltaTimeRecord = FDeltaTimeRecord();
	Data.MarkerTickRecord.Reset();
}

void FAnimNode_RandomPlayer::AdvanceToNextSequence()
{
	// Get the next sequence entry to use.
	int32 NextEntry = GetNextValidEntryIndex();

	// Switch play data by flipping it between 0 and 1.
	CurrentPlayDataIndex = (CurrentPlayDataIndex + 1) % 2;

	// Get our play data
	FRandomAnimPlayData& CurrentData = GetPlayData(ERandomDataIndexType::Current);
	FRandomAnimPlayData& NextData = GetPlayData(ERandomDataIndexType::Next);

	// Reset blend weights
	CurrentData.BlendWeight = 1.0f;
	CurrentData.Entry->BlendIn.Reset();

	// Set up data for next switch
	InitPlayData(NextData, NextEntry, 0.0f);
}

void FAnimNode_RandomPlayer::BuildShuffleList(int32 LastEntry)
{
	ShuffleList.Reset(ValidEntries.Num());

	// Build entry index list
	const int32 NumValidEntries = ValidEntries.Num();
	for (int32 i = 0; i < NumValidEntries; ++i)
	{
		ShuffleList.Add(i);
	}

	// Shuffle the list
	const int32 NumShuffles = ShuffleList.Num() - 1;
	for (int32 i = 0; i < NumShuffles; ++i)
	{
		int32 SwapIdx = RandomStream.RandRange(i, NumShuffles);
		ShuffleList.Swap(i, SwapIdx);
	}

	// Make sure we don't play the same thing twice in a row
	if (ShuffleList.Num() > 1 && ShuffleList.Last() == LastEntry)
	{
		// Swap the last with a random entry.
		ShuffleList.Swap(RandomStream.RandRange(0, ShuffleList.Num() - 2), ShuffleList.Num() - 1);
	}
}

