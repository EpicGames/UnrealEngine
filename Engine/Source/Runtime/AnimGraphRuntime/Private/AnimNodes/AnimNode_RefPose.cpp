// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_RefPose.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimTrace.h"

/////////////////////////////////////////////////////
// FAnimRefPoseNode

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_RefPose)

/** Helper for enum output... */
#ifndef CASE_ENUM_TO_TEXT
#define CASE_ENUM_TO_TEXT(txt) case txt: return TEXT(#txt);
#endif

const TCHAR* GetRefPostTypeText(ERefPoseType RefPose)
{
	switch (RefPose)
	{
		FOREACH_ENUM_EREFPOSETYPE(CASE_ENUM_TO_TEXT)
	}
	return TEXT("Unknown Ref Pose Type");
}

void FAnimNode_RefPose::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	// I don't have anything to evaluate. Should this be even here?
	// EvaluateGraphExposedInputs.Execute(Context);

    ERefPoseType CurrentRefPoseType = GetRefPoseType(); 
	switch (CurrentRefPoseType) 
	{
	case EIT_LocalSpace:
		Output.ResetToRefPose();
		break;

	case EIT_Additive:
	default:
		Output.ResetToAdditiveIdentity();
		break;
	}

	TRACE_ANIM_NODE_VALUE(Output, TEXT("Ref Pose Type"), GetRefPostTypeText(CurrentRefPoseType));
}

ERefPoseType FAnimNode_RefPose::GetRefPoseType() const
{
	return GET_ANIM_NODE_DATA(TEnumAsByte<ERefPoseType>, RefPoseType);
}

void FAnimNode_MeshSpaceRefPose::EvaluateComponentSpace_AnyThread(FComponentSpacePoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateComponentSpace_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(MeshSpaceRefPose, !IsInGameThread());

	Output.ResetToRefPose();
}

void FAnimNode_RefPose::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("(Ref Pose Type: %s)"), GetRefPostTypeText(GetRefPoseType()));
	DebugData.AddDebugItem(DebugLine, true);
}
