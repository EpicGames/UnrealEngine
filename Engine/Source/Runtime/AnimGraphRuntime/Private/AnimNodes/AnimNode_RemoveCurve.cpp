// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_RemoveCurve.h"
#include "Animation/AnimInstanceProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_RemoveCurve)

FAnimNode_RemoveCurve::FAnimNode_RemoveCurve()
{	
	bEnabled = true;
	Curves = {};
	RemoveMode = ERemoveCurveMode::RemoveSpecifiedCurves;
	bAffectMorphTargetCurves = true;
	bAffectMaterialCurves = true;
	bAffectRegularCurves = true;
}

void FAnimNode_RemoveCurve::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	Super::Initialize_AnyThread(Context);
	SourcePose.Initialize(Context);
}

void FAnimNode_RemoveCurve::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	Super::CacheBones_AnyThread(Context);
	SourcePose.CacheBones(Context);

	// Store curve types on the skel to look up in Eval state
	MorphTargetCurveNames.Reset();
	MaterialCurveNames.Reset();

	// get skeleton's curves and store curves of different types to different lists
	const USkeleton* Skel = Context.AnimInstanceProxy ? Context.AnimInstanceProxy->GetSkeleton() : nullptr;
	if (!Skel)
	{
		return;
	}
	
	const bool bAffectsAllCurves = bAffectMorphTargetCurves && bAffectMaterialCurves && bAffectRegularCurves;
	// No need to fill curves array if we filter regardless of type
	if (!bAffectsAllCurves)
	{
		TArray<FName> MetaNames;
		Skel->GetCurveMetaDataNames(MetaNames);
	
		for (const FName& N : MetaNames)
		{
			if (const FCurveMetaData* Meta = Skel->GetCurveMetaData(N))
			{
				if (Meta->Type.bMorphtarget)
				{
					MorphTargetCurveNames.Add(N);
				}

				if (Meta->Type.bMaterial)
				{
					MaterialCurveNames.Add(N);
				}
			}
		}
	}
}

void FAnimNode_RemoveCurve::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)
	SourcePose.Update(Context);
	GetEvaluateGraphExposedInputs().Execute(Context);
}

void FAnimNode_RemoveCurve::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(RemoveCurve, !IsInGameThread());

	// Evaluate upstream pose first
	SourcePose.Evaluate(Output);

	if (!bEnabled)
	{
		return;
	}

	const bool bRemoveAllCurves = RemoveMode == ERemoveCurveMode::KeepOnlySpecifiedCurves ? true : false;

	if (!bRemoveAllCurves && Curves.Num() == 0)
	{
		return;
	}

	// If we want to remove all curves, just do that directly
	const bool bAffectsAllCurves = bAffectMorphTargetCurves && bAffectMaterialCurves && bAffectRegularCurves;
	if (bRemoveAllCurves && bAffectsAllCurves && Curves.IsEmpty())
	{
		Output.Curve.Empty();
		return;
	}
	
	CurvesForRemoval = {};

	// helper func to check if curve is not allowed to be removed	
	auto CurveIsOfAllowedToRemoveType = [this](const FName& CurveName) -> bool
	{
		// not allowed to be removed cause the type is checked off on the node
		if (!bAffectRegularCurves)
		{
			const bool bIsMorphTargetCurve = MorphTargetCurveNames.Contains(CurveName);
			const bool bIsMaterialCurve = MaterialCurveNames.Contains(CurveName);

			return (bIsMorphTargetCurve || bIsMaterialCurve) &&			// Regular
				(bAffectMorphTargetCurves || !bIsMorphTargetCurve) &&	// MorphTarget
				(bAffectMaterialCurves || !bIsMaterialCurve);			// Material

		}
		else
		{
			return (bAffectMorphTargetCurves || !MorphTargetCurveNames.Contains(CurveName)) &&	// Morph target
				(bAffectMaterialCurves || !MaterialCurveNames.Contains(CurveName));				// Material
		}
	};

	// If Removal of all curves is requested
	if (bRemoveAllCurves)
	{
		CurvesForRemoval.Reserve(Output.Curve.Num());

		Output.Curve.ForEachElement([this, bAffectsAllCurves, CurveIsOfAllowedToRemoveType](const UE::Anim::FCurveElement& Elem)
			{

				// is in exceptions list
				bool bIsInExceptions = Curves.Contains(Elem.Name);

				if (!bIsInExceptions && (bAffectsAllCurves || CurveIsOfAllowedToRemoveType(Elem.Name)))
				{
					CurvesForRemoval.Add(Elem.Name);
				}
			});
	}
	else
	{
		CurvesForRemoval.Reserve(Curves.Num());

		// iterate through user provided list of curve names and remove
		for (const FName& CurveName : Curves)
		{
			if (bAffectsAllCurves || CurveIsOfAllowedToRemoveType(CurveName))
			{
				CurvesForRemoval.Add(CurveName);
			}
		};
	}

	// Remove each collected and approved curve from this pose's curve container	
	UE::Anim::FNamedValueArrayUtils::RemoveByPredicate(
		Output.Curve,
		CurvesForRemoval,
		[](const UE::Anim::FCurveElement& OutputCurve, const UE::Anim::FCurveElement& CurveToRemove)
		{
			return true;
		}
	);
}

void FAnimNode_RemoveCurve::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	DebugLine += bEnabled ? TEXT("True") : TEXT("False");
	DebugData.AddDebugItem(DebugLine);

	DebugLine = TEXT("Affects MorphTarget Type curves: ");
	DebugLine += bAffectMorphTargetCurves ? TEXT("True") : TEXT("False");
	DebugData.AddDebugItem(DebugLine);

	DebugLine = TEXT("Affects Material Type curves: ");
	DebugLine += bAffectMaterialCurves ? TEXT("True") : TEXT("False");
	DebugData.AddDebugItem(DebugLine);

	DebugLine = TEXT("Affects Regular Type curves: ");
	DebugLine += bAffectMaterialCurves ? TEXT("True") : TEXT("False");
	DebugData.AddDebugItem(DebugLine);

	DebugLine = TEXT(" Removed=[");
	CurvesForRemoval.ForEachElement([&DebugLine](const UE::Anim::FCurveElement& Elem)
		{
			DebugLine += Elem.Name.ToString();
			DebugLine += TEXT(", ");
		}
	);

	DebugLine += TEXT("]");
	DebugData.AddDebugItem(DebugLine);

	SourcePose.GatherDebugData(DebugData);
}
