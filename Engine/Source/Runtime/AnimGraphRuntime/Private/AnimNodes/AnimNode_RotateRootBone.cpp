// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_RotateRootBone.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimTrace.h"
#include "Animation/AnimRootMotionProvider.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_RotateRootBone)

/////////////////////////////////////////////////////
// FAnimNode_RotateRootBone

void FAnimNode_RotateRootBone::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_Base::Initialize_AnyThread(Context);

	BasePose.Initialize(Context);

	PitchScaleBiasClamp.Reinitialize();
	YawScaleBiasClamp.Reinitialize();
}

void FAnimNode_RotateRootBone::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	BasePose.CacheBones(Context);
}

void FAnimNode_RotateRootBone::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)
	GetEvaluateGraphExposedInputs().Execute(Context);
	BasePose.Update(Context);

	ActualPitch = PitchScaleBiasClamp.ApplyTo(Pitch, Context.GetDeltaTime());
	ActualYaw = YawScaleBiasClamp.ApplyTo(Yaw, Context.GetDeltaTime());

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Pitch"), ActualPitch);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Yaw"), ActualYaw);
}

void FAnimNode_RotateRootBone::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(RotateRootBone, !IsInGameThread());

	// Evaluate the input
	BasePose.Evaluate(Output);

	checkSlow(!FMath::IsNaN(ActualYaw) && FMath::IsFinite(ActualYaw));
	checkSlow(!FMath::IsNaN(ActualPitch) && FMath::IsFinite(ActualPitch));

	if (!FMath::IsNearlyZero(ActualPitch, KINDA_SMALL_NUMBER) || !FMath::IsNearlyZero(ActualYaw, KINDA_SMALL_NUMBER))
	{
		// Build our desired rotation
		const FRotator DeltaRotation(ActualPitch, ActualYaw, 0.f);
		const FQuat DeltaQuat(DeltaRotation);
		const FQuat MeshToComponentQuat(MeshToComponent);

		// Convert our rotation from Component Space to Mesh Space.
		const FQuat MeshSpaceDeltaQuat = MeshToComponentQuat.Inverse() * DeltaQuat * MeshToComponentQuat;

		// Apply rotation to root bone.
		FCompactPoseBoneIndex RootBoneIndex(0);
		Output.Pose[RootBoneIndex].SetRotation(Output.Pose[RootBoneIndex].GetRotation() * MeshSpaceDeltaQuat);
		Output.Pose[RootBoneIndex].NormalizeRotation();

		if (bRotateRootMotionAttribute)
		{
			// Rotate our root motion attribute by the same rotation we apply to the root
			const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get();
			if (RootMotionProvider && RootMotionProvider->HasRootMotion(Output.CustomAttributes))
			{
				FTransform RootMotionTransformDelta;
				if (RootMotionProvider->ExtractRootMotion(Output.CustomAttributes, RootMotionTransformDelta))
				{
					RootMotionTransformDelta.SetTranslation(MeshSpaceDeltaQuat.RotateVector(RootMotionTransformDelta.GetTranslation()));
					RootMotionProvider->OverrideRootMotion(RootMotionTransformDelta, Output.CustomAttributes);
				}
			}
		}
	}
}


void FAnimNode_RotateRootBone::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	DebugLine += FString::Printf(TEXT("Pitch(%.2f) Yaw(%.2f)"), ActualPitch, ActualYaw);
	DebugData.AddDebugItem(DebugLine);

	BasePose.GatherDebugData(DebugData);
}

FAnimNode_RotateRootBone::FAnimNode_RotateRootBone()
	: Pitch(0.0f)
	, Yaw(0.0f)
	, MeshToComponent(FRotator::ZeroRotator)
	, bRotateRootMotionAttribute(false)
	, ActualPitch(0.f)
	, ActualYaw(0.f)
{
}

