// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_RotationOffsetBlendSpace.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimInstance.h"
#include "AnimationRuntime.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_RotationOffsetBlendSpace)

/////////////////////////////////////////////////////
// FAnimNode_RotationOffsetBlendSpace

FAnimNode_RotationOffsetBlendSpace::FAnimNode_RotationOffsetBlendSpace()
	: LODThreshold(INDEX_NONE)
	, Alpha(1.f)
	, AlphaCurveName(NAME_None)
	, ActualAlpha(0.0f)
	, AlphaInputType(EAnimAlphaInputType::Float)
	, bAlphaBoolEnabled(false)
	, bIsLODEnabled(false)
{
}

void FAnimNode_RotationOffsetBlendSpace::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_BlendSpacePlayer::Initialize_AnyThread(Context);
	BasePose.Initialize(Context);
}

void FAnimNode_RotationOffsetBlendSpace::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	FAnimNode_BlendSpacePlayer::CacheBones_AnyThread(Context);
	BasePose.CacheBones(Context);
}

void FAnimNode_RotationOffsetBlendSpace::UpdateAssetPlayer(const FAnimationUpdateContext& Context)
{
	ActualAlpha = 0.f;
	bIsLODEnabled = IsLODEnabled(Context.AnimInstanceProxy);
	if (bIsLODEnabled)
	{
		GetEvaluateGraphExposedInputs().Execute(Context);

		// Determine Actual Alpha.
		switch (AlphaInputType)
		{
		case EAnimAlphaInputType::Float:
			ActualAlpha = AlphaScaleBias.ApplyTo(AlphaScaleBiasClamp.ApplyTo(Alpha, Context.GetDeltaTime()));
			break;
		case EAnimAlphaInputType::Bool:
			ActualAlpha = AlphaBoolBlend.ApplyTo(bAlphaBoolEnabled, Context.GetDeltaTime());
			break;
		case EAnimAlphaInputType::Curve:
			if (UAnimInstance* AnimInstance = Cast<UAnimInstance>(Context.AnimInstanceProxy->GetAnimInstanceObject()))
			{
				ActualAlpha = AlphaScaleBiasClamp.ApplyTo(AnimInstance->GetCurveValue(AlphaCurveName), Context.GetDeltaTime());
			}
			break;
		};

		// Make sure Alpha is clamped between 0 and 1.
		ActualAlpha = FMath::Clamp<float>(ActualAlpha, 0.f, 1.f);

		if (FAnimWeight::IsRelevant(ActualAlpha))
		{
			UpdateInternal(Context);
		}
	}

	BasePose.Update(Context);

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Alpha"), ActualAlpha);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Playback Time"), InternalTimeAccumulator);
}

void FAnimNode_RotationOffsetBlendSpace::Evaluate_AnyThread(FPoseContext& Context)
{
	// Evaluate base pose
	BasePose.Evaluate(Context);

	if (bIsLODEnabled && FAnimWeight::IsRelevant(ActualAlpha) && GetBlendSpace())
	{
		// Evaluate MeshSpaceRotation additive blendspace
		FPoseContext MeshSpaceRotationAdditivePoseContext(Context);
		FAnimNode_BlendSpacePlayer::Evaluate_AnyThread(MeshSpaceRotationAdditivePoseContext);

		// Accumulate poses together
		FAnimationPoseData BaseAnimationPoseData(Context);
		const FAnimationPoseData AdditiveAnimationPoseData(MeshSpaceRotationAdditivePoseContext);

		if (bApplyAdditiveInRootSpace)
		{
			FAnimationRuntime::AccumulateRootSpaceRotationAdditiveToLocalPose(BaseAnimationPoseData, AdditiveAnimationPoseData, ActualAlpha);
		}
		else
		{
			FAnimationRuntime::AccumulateMeshSpaceRotationAdditiveToLocalPose(BaseAnimationPoseData, AdditiveAnimationPoseData, ActualAlpha);
		}

		// Resulting rotations are not normalized, so normalize here.
		Context.Pose.NormalizeRotations();
	}
}

void FAnimNode_RotationOffsetBlendSpace::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);
	
	DebugLine += FString::Printf(TEXT("Alpha (%.1f%%) PlayTime (%.3f)"), ActualAlpha * 100.f, InternalTimeAccumulator);
	DebugData.AddDebugItem(DebugLine);
	
	BasePose.GatherDebugData(DebugData);
}



