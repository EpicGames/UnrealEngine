// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_RotationOffsetBlendSpaceGraph.h"
#include "AnimationRuntime.h"
#include "Animation/AnimTrace.h"
#include "Animation/AnimNodeAlphaOptions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_RotationOffsetBlendSpaceGraph)

void FAnimNode_RotationOffsetBlendSpaceGraph::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_BlendSpaceGraphBase::Initialize_AnyThread(Context);
	BasePose.Initialize(Context);
}

void FAnimNode_RotationOffsetBlendSpaceGraph::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	FAnimNode_BlendSpaceGraphBase::CacheBones_AnyThread(Context);
	BasePose.CacheBones(Context);
}

void FAnimNode_RotationOffsetBlendSpaceGraph::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	ActualAlpha = 0.f;

	bIsLODEnabled = IsLODEnabled(Context.AnimInstanceProxy);
	if (bIsLODEnabled)
	{
		GetEvaluateGraphExposedInputs().Execute(Context);

		if (FAnimNodeAlphaOptions::Update(*this, Context))
		{
			FAnimNode_BlendSpaceGraphBase::UpdateInternal(Context);
		}
	}

	BasePose.Update(Context);

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Alpha"), ActualAlpha);
}

void FAnimNode_RotationOffsetBlendSpaceGraph::Evaluate_AnyThread(FPoseContext& Context)
{
	// Evaluate base pose
	BasePose.Evaluate(Context);

	if (bIsLODEnabled && FAnimWeight::IsRelevant(ActualAlpha))
	{
		// Evaluate MeshSpaceRotation additive blendspace
		FPoseContext MeshSpaceRotationAdditivePoseContext(Context);
		FAnimNode_BlendSpaceGraphBase::Evaluate_AnyThread(MeshSpaceRotationAdditivePoseContext);

		// Accumulate poses together
		FAnimationPoseData BaseAnimationPoseData(Context);
		const FAnimationPoseData AdditiveAnimationPoseData(MeshSpaceRotationAdditivePoseContext);

		if (bApplyAdditiveInRootSpace)
		{
			FAnimationRuntime::AccumulateRootSpaceRotationAdditiveToLocalPose(BaseAnimationPoseData, AdditiveAnimationPoseData, ActualAlpha);
		}
		else
		{
			FAnimationRuntime::AccumulateMeshSpaceRotationAdditiveToLocalPose(BaseAnimationPoseData, AdditiveAnimationPoseData, ActualAlpha);
		}

		// Resulting rotations are not normalized, so normalize here.
		Context.Pose.NormalizeRotations();
	}
}

void FAnimNode_RotationOffsetBlendSpaceGraph::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);
	
	DebugLine += FString::Printf(TEXT("Alpha (%.1f%%)"), ActualAlpha * 100.f);
	DebugData.AddDebugItem(DebugLine);
	
	BasePose.GatherDebugData(DebugData);
}



