// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_SequenceEvaluator.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_SequenceEvaluator)

float FAnimNode_SequenceEvaluatorBase::GetCurrentAssetTime() const
{
	return GetExplicitTime();
}

float FAnimNode_SequenceEvaluatorBase::GetCurrentAssetLength() const
{
	UAnimSequenceBase* CurrentSequence = GetSequence();
	return CurrentSequence ? CurrentSequence->GetPlayLength() : 0.0f;
}

void FAnimNode_SequenceEvaluatorBase::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_AssetPlayerBase::Initialize_AnyThread(Context);
	GetEvaluateGraphExposedInputs().Execute(Context);
	bReinitialized = true;
}

void FAnimNode_SequenceEvaluatorBase::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
}

void FAnimNode_SequenceEvaluatorBase::UpdateAssetPlayer(const FAnimationUpdateContext& Context)
{
	GetEvaluateGraphExposedInputs().Execute(Context);
	// We split this off so subclasses can insert code between the two to set the explicit time.
	PostExposedInputsUpdateAssetPlayer(Context);
}

void FAnimNode_SequenceEvaluatorBase::PostExposedInputsUpdateAssetPlayer(const FAnimationUpdateContext& Context)
{
	float CurrentExplicitTime = GetExplicitTime();

	UAnimSequenceBase* CurrentSequence = GetSequence();
	if (CurrentSequence)
	{
		// Clamp input to a valid position on this sequence's time line.
		CurrentExplicitTime = FMath::Clamp(CurrentExplicitTime, 0.f, CurrentSequence->GetPlayLength());

		if ((!GetTeleportToExplicitTime() || (GetGroupName() != NAME_None) || (GetGroupMethod() == EAnimSyncMethod::Graph)) && CurrentSequence->GetSkeleton() != nullptr)
		{
			if (bReinitialized)
			{
				switch (GetReinitializationBehavior())
				{
					case ESequenceEvalReinit::StartPosition: InternalTimeAccumulator = GetStartPosition(); break;
					case ESequenceEvalReinit::ExplicitTime: InternalTimeAccumulator = CurrentExplicitTime; break;
				}

				InternalTimeAccumulator = FMath::Clamp(InternalTimeAccumulator, 0.f, CurrentSequence->GetPlayLength());
			}

			const float TimeJump = GetEffectiveDeltaTime(CurrentExplicitTime, InternalTimeAccumulator);

			// if you jump from front to end or end to front, your time jump is 0.f, so nothing moves
			// to prevent that from happening, we set current accumulator to explicit time
			if (TimeJump == 0.f)
			{
				InternalTimeAccumulator = CurrentExplicitTime;
			}

			const float DeltaTime = Context.GetDeltaTime();
			const float RateScale = CurrentSequence->RateScale;
			const float PlayRate = FMath::IsNearlyZero(DeltaTime) || FMath::IsNearlyZero(RateScale) ? 0.f : (TimeJump / (DeltaTime * RateScale));
			CreateTickRecordForNode(Context, CurrentSequence, IsLooping(), PlayRate, true);
		}
		else
		{
			InternalTimeAccumulator = CurrentExplicitTime;
			CreateTickRecordForNode(Context, CurrentSequence, IsLooping(), 0, true);
		}
	}

	bReinitialized = false;

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Name"), CurrentSequence != nullptr ? CurrentSequence->GetFName() : NAME_None);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Sequence"), CurrentSequence);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("InputTime"), CurrentExplicitTime);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Time"), InternalTimeAccumulator);
}

void FAnimNode_SequenceEvaluatorBase::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	check(Output.AnimInstanceProxy != nullptr);
	UAnimSequenceBase* CurrentSequence = GetSequence();
	if (CurrentSequence != nullptr && CurrentSequence->GetSkeleton() != nullptr)
	{
		FAnimationPoseData AnimationPoseData(Output);
		CurrentSequence->GetAnimationPose(AnimationPoseData, FAnimExtractContext(static_cast<double>(InternalTimeAccumulator), Output.AnimInstanceProxy->ShouldExtractRootMotion(), DeltaTimeRecord, IsLooping()));
	}
	else
	{
		Output.ResetToRefPose();
	}
}

void FAnimNode_SequenceEvaluatorBase::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);
	
	DebugLine += FString::Printf(TEXT("('%s' InputTime: %.3f, Time: %.3f)"), *GetNameSafe(GetSequence()), GetExplicitTime(), InternalTimeAccumulator);
	DebugData.AddDebugItem(DebugLine, true);
}

float FAnimNode_SequenceEvaluatorBase::GetEffectiveDeltaTime(float ExplicitTime, float PrevExplicitTime) const 
{
	float DeltaTime = ExplicitTime - PrevExplicitTime;

	if (IsLooping())
	{
		if (FMath::Abs(DeltaTime) > (GetSequence()->GetPlayLength() * 0.5f))
		{
			if (DeltaTime > 0.f)
			{
				DeltaTime -= GetSequence()->GetPlayLength();
			}
			else
			{
				DeltaTime += GetSequence()->GetPlayLength();
			}
		}
	}

	return DeltaTime;
}

bool FAnimNode_SequenceEvaluator::SetSequence(UAnimSequenceBase* InSequence)
{
#if WITH_EDITORONLY_DATA
	Sequence = InSequence;
	GET_MUTABLE_ANIM_NODE_DATA(TObjectPtr<UAnimSequenceBase>, Sequence) = InSequence;
#endif

	if(TObjectPtr<UAnimSequenceBase>* SequencePtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(TObjectPtr<UAnimSequenceBase>, Sequence))
	{
		*SequencePtr = InSequence;
		return true;
	}

	return false;
}

UAnimSequenceBase* FAnimNode_SequenceEvaluator::GetSequence() const
{
	return GET_ANIM_NODE_DATA(TObjectPtr<UAnimSequenceBase>, Sequence);
}

bool FAnimNode_SequenceEvaluator::SetShouldUseExplicitFrame(bool bFlag)
{	
#if WITH_EDITORONLY_DATA
	bUseExplicitFrame = bFlag;
#endif

	if (bool* bUseExplicitFrameState = GET_INSTANCE_ANIM_NODE_DATA_PTR(bool, bUseExplicitFrame))
	{
		*bUseExplicitFrameState = bFlag;
		return true;
	}

	return false;
}

bool FAnimNode_SequenceEvaluator::ShouldUseExplicitFrame() const
{
	return GET_ANIM_NODE_DATA(bool, bUseExplicitFrame);
}

int32 FAnimNode_SequenceEvaluator::GetExplicitFrame() const
{
	return GET_ANIM_NODE_DATA(int32, ExplicitFrame);
}

bool FAnimNode_SequenceEvaluator::SetExplicitFrame(int32 InFrame)
{	
#if WITH_EDITORONLY_DATA
	ExplicitFrame = InFrame;
#endif

	if (int32* ExplicitFramePtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(int32, ExplicitFrame))
	{
		*ExplicitFramePtr = InFrame;
		return true;
	}

	return false;
}

float FAnimNode_SequenceEvaluator::GetExplicitTime() const
{
	if(ShouldUseExplicitFrame())
	{
		if(const UAnimSequenceBase* SequenceBase = GetSequence())
		{
			const int32 Frame = GET_ANIM_NODE_DATA(int32, ExplicitFrame);
			return SequenceBase->GetSamplingFrameRate().AsSeconds(Frame);
		}

		return 0.f;
	}
	  
	 return GET_ANIM_NODE_DATA(float, ExplicitTime);
}

bool FAnimNode_SequenceEvaluator::SetExplicitTime(float InTime)
{
#if WITH_EDITORONLY_DATA
	ExplicitTime = InTime;
#endif

	if (float* ExplicitTimePtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(float, ExplicitTime))
	{
		*ExplicitTimePtr = InTime;
		return true;
	}

	return false;
}

bool FAnimNode_SequenceEvaluator::SetShouldLoop(bool bInShouldLoop)
{
#if WITH_EDITORONLY_DATA
	bShouldLoop = bInShouldLoop;
#endif

	if (bool* bShouldLoopPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(bool, bShouldLoop))
	{
		*bShouldLoopPtr = bInShouldLoop;
		return true;
	}

	return false;
}

bool FAnimNode_SequenceEvaluator::IsLooping() const
{
	return GET_ANIM_NODE_DATA(bool, bShouldLoop);
}

bool FAnimNode_SequenceEvaluator::GetTeleportToExplicitTime() const
{
	return GET_ANIM_NODE_DATA(bool, bTeleportToExplicitTime);
}

TEnumAsByte<ESequenceEvalReinit::Type> FAnimNode_SequenceEvaluator::GetReinitializationBehavior() const
{
	return GET_ANIM_NODE_DATA(TEnumAsByte<ESequenceEvalReinit::Type>, ReinitializationBehavior);
}

float FAnimNode_SequenceEvaluator::GetStartPosition() const
{
	return GET_ANIM_NODE_DATA(float, StartPosition);
}

FName FAnimNode_SequenceEvaluator::GetGroupName() const
{
	return GET_ANIM_NODE_DATA(FName, GroupName);
}

EAnimGroupRole::Type FAnimNode_SequenceEvaluator::GetGroupRole() const
{
	return GET_ANIM_NODE_DATA(TEnumAsByte<EAnimGroupRole::Type>, GroupRole);
}

EAnimSyncMethod FAnimNode_SequenceEvaluator::GetGroupMethod() const
{
	return GET_ANIM_NODE_DATA(EAnimSyncMethod, Method);
}

bool FAnimNode_SequenceEvaluator::GetIgnoreForRelevancyTest() const
{
	return GET_ANIM_NODE_DATA(bool, bIgnoreForRelevancyTest);
}

bool FAnimNode_SequenceEvaluator::SetGroupName(FName InGroupName)
{
#if WITH_EDITORONLY_DATA	
	GroupName = InGroupName;
#endif

	if(FName* GroupNamePtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(FName, GroupName))
	{
		*GroupNamePtr = InGroupName;
		return true;
	}

	return false;
}

bool FAnimNode_SequenceEvaluator::SetGroupRole(EAnimGroupRole::Type InRole)
{
#if WITH_EDITORONLY_DATA
	GroupRole = InRole;
#endif

	if(TEnumAsByte<EAnimGroupRole::Type>* GroupRolePtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(TEnumAsByte<EAnimGroupRole::Type>, GroupRole))
	{
		*GroupRolePtr = InRole;
		return true;
	}

	return false;	
}

bool FAnimNode_SequenceEvaluator::SetGroupMethod(EAnimSyncMethod InMethod)
{
#if WITH_EDITORONLY_DATA
	Method = InMethod;
#endif

	if(EAnimSyncMethod* MethodPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(EAnimSyncMethod, Method))
	{
		*MethodPtr = InMethod;
		return true;
	}

	return false;
}

bool FAnimNode_SequenceEvaluator::SetIgnoreForRelevancyTest(bool bInIgnoreForRelevancyTest)
{
#if WITH_EDITORONLY_DATA
	bIgnoreForRelevancyTest = bInIgnoreForRelevancyTest;
#endif

	if(bool* bIgnoreForRelevancyTestPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(bool, bIgnoreForRelevancyTest))
	{
		*bIgnoreForRelevancyTestPtr = bInIgnoreForRelevancyTest;
		return true;
	}

	return false;
}