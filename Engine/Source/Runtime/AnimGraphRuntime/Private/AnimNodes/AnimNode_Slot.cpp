// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_Slot.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimTrace.h"
#include "Animation/AnimNode_Inertialization.h"
#include "IAnimGraphRuntime_SequencerMixerTargetConnector.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_Slot)

namespace MontageCVars
{
	bool bMarkMontageSourceAsBlendingOut = true;
	FAutoConsoleVariableRef CVarMarkMontageSourceAsBlendingOut(
		TEXT("a.Montage.MarkMontageSourceAsBlendingOut"),
		bMarkMontageSourceAsBlendingOut,
		TEXT("Set true to make montages mark their source graph as blending out when enabled")
	);
} // end namespace MontageCVars

/////////////////////////////////////////////////////
// FAnimNode_Slot

void FAnimNode_Slot::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_Base::Initialize_AnyThread(Context);

	Source.Initialize(Context);
	WeightData.Reset();

	// If this node has not already been registered with the AnimInstance, do it.
	if (!SlotNodeInitializationCounter.IsSynchronized_Counter(Context.AnimInstanceProxy->GetSlotNodeInitializationCounter()))
	{
		SlotNodeInitializationCounter.SynchronizeWith(Context.AnimInstanceProxy->GetSlotNodeInitializationCounter());
		Context.AnimInstanceProxy->RegisterSlotNodeWithAnimInstance(SlotName);
	}
}

void FAnimNode_Slot::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	Source.CacheBones(Context);
}

void FAnimNode_Slot::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)

	// Update weights.
	const float PreviousSourceWeight = WeightData.SourceWeight;
	Context.AnimInstanceProxy->GetSlotWeight(SlotName, OUT WeightData.SlotNodeWeight, OUT WeightData.SourceWeight, OUT WeightData.TotalNodeWeight);

	// Update cache in AnimInstance.
	Context.AnimInstanceProxy->UpdateSlotNodeWeight(SlotName, WeightData.SlotNodeWeight, Context.GetFinalBlendWeight());

	FInertializationRequest InertializationRequest;
	if (Context.AnimInstanceProxy->GetSlotInertializationRequestData(SlotName, InertializationRequest))
	{
		UE::Anim::IInertializationRequester* InertializationRequester = Context.GetMessage<UE::Anim::IInertializationRequester>();
		if (InertializationRequester)
		{
#if ANIM_TRACE_ENABLED
			InertializationRequest.NodeId = Context.GetCurrentNodeId();
			InertializationRequest.AnimInstance = Context.AnimInstanceProxy->GetAnimInstanceObject();
#endif

			InertializationRequester->RequestInertialization(InertializationRequest);
		}
		else
		{
			FAnimNode_Inertialization::LogRequestError(Context, Source);
		}
	}

	const bool bUpdateSource = bAlwaysUpdateSourcePose || FAnimWeight::IsRelevant(WeightData.SourceWeight);
	if (bUpdateSource)
	{
		const float SourceWeight = FMath::Max(FAnimWeight::GetSmallestRelevantWeight(), WeightData.SourceWeight);
		const bool bBlendingOutSource = PreviousSourceWeight > WeightData.SourceWeight || FAnimWeight::IsFullWeight(WeightData.SlotNodeWeight);
		const FAnimationUpdateContext SourceContext = MontageCVars::bMarkMontageSourceAsBlendingOut && bBlendingOutSource ? Context.AsInactive() : Context;
		Source.Update(SourceContext.FractionalWeight(SourceWeight));
	}

#if ANIM_TRACE_ENABLED
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Name"), SlotName);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Slot Weight"), WeightData.SlotNodeWeight);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Pose Source"), (WeightData.SourceWeight <= ZERO_ANIMWEIGHT_THRESH));

	Context.AnimInstanceProxy->TraceMontageEvaluationData(Context, SlotName);
#endif
}

void FAnimNode_Slot::Evaluate_AnyThread(FPoseContext & Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(Slot, !IsInGameThread());

	// If not playing a montage, just pass through
	if (WeightData.SlotNodeWeight <= ZERO_ANIMWEIGHT_THRESH)
	{
		Source.Evaluate(Output);
		PostEvaluateSourcePose(Output);
	}
	else
	{
		FPoseContext SourceContext(Output);
		if (WeightData.SourceWeight > ZERO_ANIMWEIGHT_THRESH)
		{
			Source.Evaluate(SourceContext);
		}

		PostEvaluateSourcePose(SourceContext);
		const FAnimationPoseData SourcePoseData(SourceContext);
		FAnimationPoseData OutputPoseData(Output);
		Output.AnimInstanceProxy->SlotEvaluatePose(SlotName, SourcePoseData, WeightData.SourceWeight, OutputPoseData, WeightData.SlotNodeWeight, WeightData.TotalNodeWeight);

		checkSlow(!Output.ContainsNaN());
		checkSlow(Output.IsNormalized());
	}

	// Backwards compatibility with Sequencer mixer
	if (IModularFeatures::Get().IsModularFeatureAvailable(IAnimGraphRuntime_SequencerMixerTargetConnector::GetModularFeatureName()))
	{
		IModularFeatures::Get().GetModularFeature<IAnimGraphRuntime_SequencerMixerTargetConnector>(IAnimGraphRuntime_SequencerMixerTargetConnector::GetModularFeatureName()).ApplySequencerMixedPose(Output, SlotName);
	}
}

void FAnimNode_Slot::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("(Slot Name: '%s' Weight:%.1f%%)"), *SlotName.ToString(), WeightData.SlotNodeWeight*100.f);
	
	bool const bIsPoseSource = (WeightData.SourceWeight <= ZERO_ANIMWEIGHT_THRESH);
	DebugData.AddDebugItem(DebugLine, bIsPoseSource);
	Source.GatherDebugData(DebugData.BranchFlow(WeightData.SourceWeight));

	for (FAnimMontageInstance* MontageInstance : DebugData.AnimInstance->MontageInstances)
	{
		if (MontageInstance->IsValid() && MontageInstance->Montage->IsValidSlot(SlotName))
		{
			if (const FAnimTrack* const Track = MontageInstance->Montage->GetAnimationData(SlotName))
			{
				if (const FAnimSegment* const Segment = Track->GetSegmentAtTime(MontageInstance->GetPosition()))
				{
					float CurrentAnimPos;
					if (UAnimSequenceBase* Anim = Segment->GetAnimationData(MontageInstance->GetPosition(), CurrentAnimPos))
					{
						FString MontageLine = FString::Printf(TEXT("Montage('%s') Anim('%s') P(%.2f) W(%.0f%%)"), *MontageInstance->Montage->GetName(), *Anim->GetName(), CurrentAnimPos, WeightData.SlotNodeWeight*100.f);
						DebugData.BranchFlow(1.0f).AddDebugItem(MontageLine, true);
						break;
					}
				}
			}
		}
	}
}

FAnimNode_Slot::FAnimNode_Slot()
	: SlotName(FAnimSlotGroup::DefaultSlotName)
	, bAlwaysUpdateSourcePose(false)
{
}

