// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_Sync.h"
#include "Animation/AnimSyncScope.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_Sync)

void FAnimNode_Sync::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	Source.Initialize(Context);
}

void FAnimNode_Sync::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	const bool bApplySyncing = GroupName != NAME_None;
	UE::Anim::TOptionalScopedGraphMessage<UE::Anim::FAnimSyncGroupScope> Message(bApplySyncing, Context, Context, GroupName, GroupRole);

	Source.Update(Context);
}

void FAnimNode_Sync::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	Source.CacheBones(Context);
}

void FAnimNode_Sync::Evaluate_AnyThread(FPoseContext& Output)
{
	Source.Evaluate(Output);
}

void FAnimNode_Sync::GatherDebugData(FNodeDebugData& DebugData)
{
	DebugData.AddDebugItem(DebugData.GetNodeName(this));

	Source.GatherDebugData(DebugData);
}

