// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_TwoWayBlend.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimTrace.h"
#include "AnimationRuntime.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_TwoWayBlend)

/////////////////////////////////////////////////////
// FAnimNode_TwoWayBlend

void FAnimNode_TwoWayBlend::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_Base::Initialize_AnyThread(Context);

	A.Initialize(Context);
	B.Initialize(Context);

	bAIsRelevant = false;
	bBIsRelevant = false;

	AlphaBoolBlend.Reinitialize();
	AlphaScaleBiasClamp.Reinitialize();
}

void FAnimNode_TwoWayBlend::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	A.CacheBones(Context);
	B.CacheBones(Context);
}

void FAnimNode_TwoWayBlend::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FAnimationNode_TwoWayBlend_Update);
	GetEvaluateGraphExposedInputs().Execute(Context);

	InternalBlendAlpha = 0.f;
	switch (AlphaInputType)
	{
	case EAnimAlphaInputType::Float:
		InternalBlendAlpha = AlphaScaleBias.ApplyTo(AlphaScaleBiasClamp.ApplyTo(Alpha, Context.GetDeltaTime()));
		break;
	case EAnimAlphaInputType::Bool:
		InternalBlendAlpha = AlphaBoolBlend.ApplyTo(bAlphaBoolEnabled, Context.GetDeltaTime());
		break;
	case EAnimAlphaInputType::Curve:
		if (UAnimInstance* AnimInstance = Cast<UAnimInstance>(Context.AnimInstanceProxy->GetAnimInstanceObject()))
		{
			InternalBlendAlpha = AlphaScaleBiasClamp.ApplyTo(AnimInstance->GetCurveValue(AlphaCurveName), Context.GetDeltaTime());
		}
		break;
	};

	// Make sure Alpha is clamped between 0 and 1.
	InternalBlendAlpha = FMath::Clamp<float>(InternalBlendAlpha, 0.f, 1.f);

	const bool bNewAIsRelevant = !FAnimWeight::IsFullWeight(InternalBlendAlpha);
	const bool bNewBIsRelevant = FAnimWeight::IsRelevant(InternalBlendAlpha);

	// when this flag is true, we'll reinitialize the children
	if (bResetChildOnActivation)
	{
		if (bNewAIsRelevant && !bAIsRelevant)
		{
			FAnimationInitializeContext ReinitializeContext(Context.AnimInstanceProxy, Context.SharedContext);

			// reinitialize
			A.Initialize(ReinitializeContext);
		}

		if (bNewBIsRelevant && !bBIsRelevant)
		{
			FAnimationInitializeContext ReinitializeContext(Context.AnimInstanceProxy, Context.SharedContext);

			// reinitialize
			B.Initialize(ReinitializeContext);
		}
	}

	bAIsRelevant = bNewAIsRelevant;
	bBIsRelevant = bNewBIsRelevant;

	if (bBIsRelevant)
	{
		if (bAIsRelevant)
		{
			// Blend A and B together
			A.Update(Context.FractionalWeight(1.0f - InternalBlendAlpha));
			B.Update(Context.FractionalWeight(InternalBlendAlpha));
		}
		else
		{
			if (bAlwaysUpdateChildren)
			{
				A.Update(Context.FractionalWeight(FAnimWeight::GetSmallestRelevantWeight()));
			}

			// Take all of B
			B.Update(Context);
		}
	}
	else
	{
		// Take all of A
		A.Update(Context);

		if (bAlwaysUpdateChildren)
		{
			B.Update(Context.FractionalWeight(FAnimWeight::GetSmallestRelevantWeight()));
		}
	}

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Alpha"), InternalBlendAlpha);
}

void FAnimNode_TwoWayBlend::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(TwoWayBlend, !IsInGameThread());

	if (bBIsRelevant)
	{
		if (bAIsRelevant)
		{
			// Evaluate A directly into Output to avoid allocating a third pose
			A.Evaluate(Output);

			FPoseContext Pose2(Output);
			B.Evaluate(Pose2);

			FAnimationPoseData InOutPoseData(Output);
			const FAnimationPoseData PoseTwoData(Pose2);
			FAnimationRuntime::BlendTwoPosesTogetherInPlace(InOutPoseData, PoseTwoData, (1.0f - InternalBlendAlpha));
		}
		else
		{
			B.Evaluate(Output);
		}
	}
	else
	{
		A.Evaluate(Output);
	}
}


void FAnimNode_TwoWayBlend::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("(Alpha: %.1f%%)"), InternalBlendAlpha *100);
	DebugData.AddDebugItem(DebugLine);

	A.GatherDebugData(DebugData.BranchFlow(1.f - InternalBlendAlpha));
	B.GatherDebugData(DebugData.BranchFlow(InternalBlendAlpha));
}

