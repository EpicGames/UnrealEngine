// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNotifies/AnimNotify_PlayMontageNotify.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSequenceBase.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotify_PlayMontageNotify)

//////////////////////////////////////////////////////////////////////////
// UAnimNotify_PlayMontageNotify
//////////////////////////////////////////////////////////////////////////

UAnimNotify_PlayMontageNotify::UAnimNotify_PlayMontageNotify(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsNativeBranchingPoint = true;
}


void UAnimNotify_PlayMontageNotify::BranchingPointNotify(FBranchingPointNotifyPayload& BranchingPointPayload)
{
	Super::BranchingPointNotify(BranchingPointPayload);

	if (USkeletalMeshComponent* MeshComp = BranchingPointPayload.SkelMeshComponent)
	{
		if (UAnimInstance* AnimInstance = MeshComp->GetAnimInstance())
		{
			AnimInstance->OnPlayMontageNotifyBegin.Broadcast(NotifyName, BranchingPointPayload);
		}
	}
}

FString UAnimNotify_PlayMontageNotify::GetNotifyName_Implementation() const
{
	return NotifyName.ToString();
}

#if WITH_EDITOR
bool UAnimNotify_PlayMontageNotify::CanBePlaced(UAnimSequenceBase* Animation) const
{
	return (Animation && Animation->IsA(UAnimMontage::StaticClass()));
}
#endif // WITH_EDITOR


//////////////////////////////////////////////////////////////////////////
// UAnimNotify_PlayMontageNotifyWindow
//////////////////////////////////////////////////////////////////////////

UAnimNotify_PlayMontageNotifyWindow::UAnimNotify_PlayMontageNotifyWindow(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsNativeBranchingPoint = true;
}


void UAnimNotify_PlayMontageNotifyWindow::BranchingPointNotifyBegin(FBranchingPointNotifyPayload& BranchingPointPayload)
{
	Super::BranchingPointNotifyBegin(BranchingPointPayload);

	if (USkeletalMeshComponent* MeshComp = BranchingPointPayload.SkelMeshComponent)
	{
		if (UAnimInstance* AnimInstance = MeshComp->GetAnimInstance())
		{
			AnimInstance->OnPlayMontageNotifyBegin.Broadcast(NotifyName, BranchingPointPayload);
		}
	}
}


void UAnimNotify_PlayMontageNotifyWindow::BranchingPointNotifyEnd(FBranchingPointNotifyPayload& BranchingPointPayload)
{
	Super::BranchingPointNotifyEnd(BranchingPointPayload);

	if (USkeletalMeshComponent* MeshComp = BranchingPointPayload.SkelMeshComponent)
	{
		if (UAnimInstance* AnimInstance = MeshComp->GetAnimInstance())
		{
			AnimInstance->OnPlayMontageNotifyEnd.Broadcast(NotifyName, BranchingPointPayload);
		}
	}
}

FString UAnimNotify_PlayMontageNotifyWindow::GetNotifyName_Implementation() const
{
	return NotifyName.ToString();
}

#if WITH_EDITOR
bool UAnimNotify_PlayMontageNotifyWindow::CanBePlaced(UAnimSequenceBase* Animation) const
{
	return (Animation && Animation->IsA(UAnimMontage::StaticClass()));
}
#endif // WITH_EDITOR

