// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UAnimSequencerInstance.cpp: Single Node Tree Instance 
	Only plays one animation at a time. 
=============================================================================*/ 

#include "AnimSequencerInstance.h"
#include "AnimSequencerInstanceProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimSequencerInstance)

/////////////////////////////////////////////////////
// UAnimSequencerInstance
/////////////////////////////////////////////////////

const FName UAnimSequencerInstance::SequencerPoseName(TEXT("Sequencer_Pose_Name"));

UAnimSequencerInstance::UAnimSequencerInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUseMultiThreadedAnimationUpdate = false;
}

FAnimInstanceProxy* UAnimSequencerInstance::CreateAnimInstanceProxy()
{
	return new FAnimSequencerInstanceProxy(this);
}

void UAnimSequencerInstance::UpdateAnimTrack(UAnimSequenceBase* InAnimSequence, int32 SequenceId, float InPosition, float Weight, bool bFireNotifies)
{
	GetProxyOnGameThread<FAnimSequencerInstanceProxy>().UpdateAnimTrack(InAnimSequence, SequenceId, InPosition, Weight, bFireNotifies);
}

void UAnimSequencerInstance::UpdateAnimTrack(UAnimSequenceBase* InAnimSequence, int32 SequenceId, float InFromPosition, float InToPosition, float Weight, bool bFireNotifies)
{
	GetProxyOnGameThread<FAnimSequencerInstanceProxy>().UpdateAnimTrack(InAnimSequence, SequenceId, InFromPosition, InToPosition, Weight, bFireNotifies);
}

void UAnimSequencerInstance::UpdateAnimTrackWithRootMotion(const FAnimSequencerData& InAnimSequencerData)
{
	GetProxyOnGameThread<FAnimSequencerInstanceProxy>().UpdateAnimTrackWithRootMotion(InAnimSequencerData);
}

void UAnimSequencerInstance::ConstructNodes()
{
	GetProxyOnGameThread<FAnimSequencerInstanceProxy>().ConstructNodes();
}

void UAnimSequencerInstance::ResetNodes()
{
	GetProxyOnGameThread<FAnimSequencerInstanceProxy>().ResetNodes();
}

void UAnimSequencerInstance::ResetPose()
{
	GetProxyOnGameThread<FAnimSequencerInstanceProxy>().ResetPose();
}

void UAnimSequencerInstance::SavePose()
{
	if (USkeletalMeshComponent* SkeletalMeshComponent = GetSkelMeshComponent())
	{
		if (SkeletalMeshComponent->GetSkeletalMeshAsset() && SkeletalMeshComponent->GetComponentSpaceTransforms().Num() > 0)
		{
			SavePoseSnapshot(UAnimSequencerInstance::SequencerPoseName);
		}
	}
}

