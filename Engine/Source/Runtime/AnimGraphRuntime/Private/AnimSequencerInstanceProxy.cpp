// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimSequencerInstanceProxy.h"
#include "AnimSequencerInstance.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimSequencerInstanceProxy)

void FAnimSequencerInstanceProxy::Initialize(UAnimInstance* InAnimInstance)
{
	FAnimInstanceProxy::Initialize(InAnimInstance);
	ConstructNodes();
	FullBodyBlendNode.bAdditiveNode = false;
	FullBodyBlendNode.bNormalizeAlpha = true;

	AdditiveBlendNode.bAdditiveNode = true;
	AdditiveBlendNode.bNormalizeAlpha = false;

	FullBodyBlendNode.ResetPoses();
	AdditiveBlendNode.ResetPoses();


	SnapshotNode.SnapshotName = UAnimSequencerInstance::SequencerPoseName;
	ClearSequencePlayerAndMirrorMaps();
	UpdateCounter.Reset();
	RootMotionOverride.Reset();
}

bool FAnimSequencerInstanceProxy::Evaluate(FPoseContext& Output)
{
	SequencerRootNode.Evaluate_AnyThread(Output);
	if (RootMotionOverride.IsSet())
	{
		if (!RootMotionOverride.GetValue().bBlendFirstChildOfRoot)
		{
			for (const FCompactPoseBoneIndex BoneIndex : Output.Pose.ForEachBoneIndex())
			{
				if (BoneIndex.IsRootBone())
				{
					Output.Pose[BoneIndex] = RootMotionOverride.GetValue().RootMotion;
					break;
				}
			}
		}
		else if (RootMotionOverride.GetValue().ChildBoneIndex != INDEX_NONE)
		{
			FCompactPoseBoneIndex PoseIndex = Output.Pose.GetBoneContainer().GetCompactPoseIndexFromSkeletonIndex(RootMotionOverride.GetValue().ChildBoneIndex);
			if (PoseIndex.IsValid())
			{
				Output.Pose[PoseIndex] = RootMotionOverride.GetValue().RootMotion;
			}
		}
	}
	
	RootBoneTransform.Reset();

	if (SwapRootBone != ESwapRootBone::SwapRootBone_None)
	{
		for (const FCompactPoseBoneIndex BoneIndex : Output.Pose.ForEachBoneIndex())
		{
			if (BoneIndex.IsRootBone())
			{
				RootBoneTransform = Output.Pose[BoneIndex];
				Output.Pose[BoneIndex] = FTransform::Identity;
				break;
			}
		}
	}

	return true;
}

void FAnimSequencerInstanceProxy::PostEvaluate(UAnimInstance* InAnimInstance)
{
	if (GetSkelMeshComponent() && SwapRootBone != ESwapRootBone::SwapRootBone_None)
	{
		if (RootBoneTransform.IsSet())
		{
			FTransform RelativeTransform = RootBoneTransform.GetValue();

			if (InitialTransform.IsSet())
			{
				RelativeTransform = RootBoneTransform.GetValue() * InitialTransform.GetValue();
			}

			if (SwapRootBone == ESwapRootBone::SwapRootBone_Component)
			{
				GetSkelMeshComponent()->SetRelativeLocationAndRotation(RelativeTransform.GetLocation(), RelativeTransform.GetRotation().Rotator());
			}
			else if (SwapRootBone == ESwapRootBone::SwapRootBone_Actor)
			{
				AActor* Actor = GetSkelMeshComponent()->GetOwner();
				if (Actor && Actor->GetRootComponent())
				{
					Actor->GetRootComponent()->SetRelativeLocationAndRotation(RelativeTransform.GetLocation(), RelativeTransform.GetRotation().Rotator());
				}
			}
		}
	}
}

void FAnimSequencerInstanceProxy::UpdateAnimationNode(const FAnimationUpdateContext& InContext)
{
	UpdateCounter.Increment();

	SequencerRootNode.Update_AnyThread(InContext);
}

void FAnimSequencerInstanceProxy::ConstructNodes()
{
	// construct node link node for full body and additive to apply additive node
	SequencerRootNode.Base.SetLinkNode(&FullBodyBlendNode);
	SequencerRootNode.Additive.SetLinkNode(&AdditiveBlendNode);

}

void FAnimSequencerInstanceProxy::AddReferencedObjects(UAnimInstance* InAnimInstance, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InAnimInstance, Collector);

	for (const TPair<uint32, FSequencerPlayerBase*>& IndexPlayerPair : SequencerToPlayerMap)
	{
		if(IndexPlayerPair.Value->IsOfType<FSequencerPlayerAnimSequence>())
		{
			FSequencerPlayerAnimSequence* SequencerPlayerAnimSequence = static_cast<FSequencerPlayerAnimSequence*>(IndexPlayerPair.Value);
			Collector.AddPropertyReferencesWithStructARO(FAnimNode_SequenceEvaluator_Standalone::StaticStruct(), &SequencerPlayerAnimSequence->PlayerNode);
		}
	}

	for (const TPair<uint32, FAnimNode_Mirror_Standalone*>& IndexMirrorPair : SequencerToMirrorMap)
	{
		Collector.AddPropertyReferencesWithStructARO(FAnimNode_Mirror_Standalone::StaticStruct(), IndexMirrorPair.Value);
	}
}

void FAnimSequencerInstanceProxy::ClearSequencePlayerAndMirrorMaps()
{
	for (TPair<uint32, FSequencerPlayerBase*>& Iter : SequencerToPlayerMap)
	{
		delete Iter.Value;
	}

	SequencerToPlayerMap.Empty();

	for (TPair<uint32, FAnimNode_Mirror_Standalone*>& Iter : SequencerToMirrorMap)
	{
		delete Iter.Value;
	}

	SequencerToMirrorMap.Empty();
}

void FAnimSequencerInstanceProxy::ResetPose()
{
	SequencerRootNode.Base.SetLinkNode(&SnapshotNode);
	//force evaluation?
}	
void FAnimSequencerInstanceProxy::ResetNodes()
{
	FMemory::Memzero(FullBodyBlendNode.DesiredAlphas.GetData(), FullBodyBlendNode.DesiredAlphas.GetAllocatedSize());
	FMemory::Memzero(AdditiveBlendNode.DesiredAlphas.GetData(), AdditiveBlendNode.DesiredAlphas.GetAllocatedSize());
}

FAnimSequencerInstanceProxy::~FAnimSequencerInstanceProxy()
{
	ClearSequencePlayerAndMirrorMaps();
}

void FAnimSequencerInstanceProxy::InitAnimTrack(UAnimSequenceBase* InAnimSequence, uint32 SequenceId)
{
	if (InAnimSequence != nullptr)
	{
		FSequencerPlayerAnimSequence* PlayerState = FindPlayer<FSequencerPlayerAnimSequence>(SequenceId);
		if (PlayerState == nullptr)
		{
			const bool bIsAdditive = InAnimSequence->IsValidAdditive();
			FAnimNode_MultiWayBlend& BlendNode = (bIsAdditive) ? AdditiveBlendNode : FullBodyBlendNode;
			
			// you shouldn't allow additive animation to be added here, but if it changes type after
			// you'll see this warning coming up
			if (bIsAdditive && InAnimSequence->GetAdditiveAnimType() == AAT_RotationOffsetMeshSpace)
			{
				// this doesn't work
				UE_LOGF(LogAnimation, Warning, "ERROR: Animation [%ls] in Sequencer has Mesh Space additive animation.  No support on mesh space additive animation. ", *GetNameSafe(InAnimSequence));
			}

			const int32 PoseIndex = BlendNode.AddPose() - 1;

			// add the new entry to map
			FSequencerPlayerAnimSequence* NewPlayerState = new FSequencerPlayerAnimSequence();
			NewPlayerState->PoseIndex = PoseIndex;
			NewPlayerState->bAdditive = bIsAdditive;
			
			SequencerToPlayerMap.Add(SequenceId, NewPlayerState);

			// link player to mirror node,
			FAnimNode_Mirror_Standalone* NewMirrorNode = new FAnimNode_Mirror_Standalone();
			NewMirrorNode->SetMirror(false);
			NewMirrorNode->SetSourceLinkNode(&NewPlayerState->PlayerNode);
			SequencerToMirrorMap.Add(SequenceId, NewMirrorNode); 

			// link mirror to blendnode, this will let you trigger notifies and so on
			NewPlayerState->PlayerNode.SetTeleportToExplicitTime(false);
			BlendNode.Poses[PoseIndex].SetLinkNode(NewMirrorNode);

			// set player state
			PlayerState = NewPlayerState;
		}

		// now set animation data to player
		PlayerState->PlayerNode.SetSequence(InAnimSequence);
		PlayerState->PlayerNode.SetExplicitTime(0.f);

		// initialize player
		PlayerState->PlayerNode.Initialize_AnyThread(FAnimationInitializeContext(this));

		FAnimNode_Mirror_Standalone* Mirror = SequencerToMirrorMap.FindRef(SequenceId);
		if (Mirror)
		{
			Mirror->Initialize_AnyThread(FAnimationInitializeContext(this));
			Mirror->CacheBones_AnyThread(FAnimationCacheBonesContext(this));
		}
	}
}

/*
// this isn't used yet. If we want to optimize it, we could do this way, but right now the way sequencer updates, we don't have a good point 
// where we could just clear one sequence id. We just clear all the weights before update. 
// once they go out of range, they don't get called anymore, so there is no good point of tearing down
// there is multiple tear down point but we couldn't find where only happens once activated and once getting out
// because sequencer finds the nearest point, not exact point, it doesn't have good point of tearing down
void FAnimSequencerInstanceProxy::TermAnimTrack(int32 SequenceId)
{
	FSequencerPlayerState* PlayerState = FindPlayer(SequenceId);

	if (PlayerState)
	{
		FAnimNode_MultiWayBlend& BlendNode = (PlayerState->bAdditive) ? AdditiveBlendNode : FullBodyBlendNode;

		// remove the pose from blend node
		BlendNode.Poses.RemoveAt(PlayerState->PoseIndex);
		BlendNode.DesiredAlphas.RemoveAt(PlayerState->PoseIndex);

		// remove from Sequence Map
		SequencerToPlayerMap.Remove(SequenceId);
	}
}*/

void FAnimSequencerInstanceProxy::UpdateAnimTrack(UAnimSequenceBase* InAnimSequence, uint32 SequenceId, float InPosition, float Weight, bool bFireNotifies)
{
	UpdateAnimTrack(InAnimSequence, SequenceId, TOptional<FRootMotionOverride>(), TOptional<float>(), InPosition, Weight, bFireNotifies, nullptr);
}

void FAnimSequencerInstanceProxy::UpdateAnimTrack(UAnimSequenceBase* InAnimSequence, uint32 SequenceId, TOptional<float> InFromPosition, float InToPosition, float Weight, bool bFireNotifies)
{
	UpdateAnimTrack(InAnimSequence, SequenceId, TOptional<FRootMotionOverride>(), InFromPosition, InToPosition, Weight, bFireNotifies, nullptr);
}

void FAnimSequencerInstanceProxy::UpdateAnimTrackWithRootMotion(const FAnimSequencerData& InAnimSequencerData)
{
	SwapRootBone = InAnimSequencerData.SwapRootBone;
	InitialTransform = InAnimSequencerData.InitialTransform;
	UpdateAnimTrack(InAnimSequencerData.AnimSequence, InAnimSequencerData.SequenceId, InAnimSequencerData.RootMotion, InAnimSequencerData.FromPosition, InAnimSequencerData.ToPosition, InAnimSequencerData.Weight, InAnimSequencerData.bFireNotifies, InAnimSequencerData.MirrorDataTable);
}

void FAnimSequencerInstanceProxy::UpdateAnimTrack(UAnimSequenceBase* InAnimSequence, uint32 SequenceId, const TOptional<FRootMotionOverride>& InRootMotionOverride, TOptional<float> InFromPosition, float InToPosition, float Weight, bool bFireNotifies, UMirrorDataTable* InMirrorDataTable)
{
	EnsureAnimTrack(InAnimSequence, SequenceId);

	FSequencerPlayerAnimSequence* PlayerState = FindPlayer<FSequencerPlayerAnimSequence>(SequenceId);

	PlayerState->PlayerNode.SetExplicitTime(InToPosition);
	if (InFromPosition.IsSet())
	{
		// Set the internal time accumulator at the "from" time so that the player node will correctly evaluate the
		// desired "from/to" range. We also disable the reinitialization code so it doesn't mess up that time we
		// just set.
		PlayerState->PlayerNode.SetExplicitPreviousTime(InFromPosition.GetValue());
		PlayerState->PlayerNode.SetReinitializationBehavior(ESequenceEvalReinit::NoReset);
	}

	FAnimNode_Mirror_Standalone* MirrorNode = SequencerToMirrorMap.FindRef(SequenceId);
	if (MirrorNode)
	{
		MirrorNode->SetMirror(InMirrorDataTable != nullptr);
		UMirrorDataTable* OldMirrorDataTable = MirrorNode->GetMirrorDataTable();
		MirrorNode->SetMirrorDataTable(InMirrorDataTable);

		if (InMirrorDataTable && OldMirrorDataTable != InMirrorDataTable)
		{
			MirrorNode->CacheBones_AnyThread(FAnimationCacheBonesContext(this));
		}
	}

	// if no fire notifies, we can teleport to explicit time
	PlayerState->PlayerNode.SetTeleportToExplicitTime(!bFireNotifies);
	// if moving to 0.f, we mark this to teleport. Otherwise, do not use explicit time
	FAnimNode_MultiWayBlend& BlendNode = (PlayerState->bAdditive) ? AdditiveBlendNode : FullBodyBlendNode;
	BlendNode.DesiredAlphas[PlayerState->PoseIndex] = Weight;

	// if additive, apply alpha value correctlyeTick
	// this will be used when apply additive is blending correct total alpha to additive
	if (PlayerState->bAdditive)
	{
		SequencerRootNode.Alpha = BlendNode.GetTotalAlpha();
	}
	RootMotionOverride = InRootMotionOverride;
}
void FAnimSequencerInstanceProxy::EnsureAnimTrack(UAnimSequenceBase* InAnimSequence, uint32 SequenceId)
{
	FSequencerPlayerAnimSequence* PlayerState = FindPlayer<FSequencerPlayerAnimSequence>(SequenceId);
	if (!PlayerState)
	{
		InitAnimTrack(InAnimSequence, SequenceId);
	}
	else if (PlayerState->PlayerNode.GetSequence() != InAnimSequence)
	{
		PlayerState->PlayerNode.SetSequence(InAnimSequence);
	}
}


