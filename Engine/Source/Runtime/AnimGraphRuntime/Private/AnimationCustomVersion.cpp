// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FAnimationCustomVersion::GUID(0x2EB5FDBD, 0x01AC4D10, 0x8136F38F, 0x3393A5DA);

// Register the custom version with core
FCustomVersionRegistration GRegisterAnimationCustomVersion(FAnimationCustomVersion::GUID, FAnimationCustomVersion::LatestVersion, TEXT("AnimGraphVer"));
