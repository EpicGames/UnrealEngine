// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationStateMachineLibrary.h"

#include "Animation/AnimNode_StateMachine.h"
#include "Animation/AnimNode_StateResult.h"
#include "Animation/AnimInstanceProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationStateMachineLibrary)

DEFINE_LOG_CATEGORY_STATIC(LogAnimationStateMachineLibrary, Verbose, All);

void UAnimationStateMachineLibrary::ConvertToAnimationStateResult(const FAnimNodeReference& Node, FAnimationStateResultReference& AnimationState, EAnimNodeReferenceConversionResult& Result)
{
	AnimationState = FAnimNodeReference::ConvertToType<FAnimationStateResultReference>(Node, Result);
}

void UAnimationStateMachineLibrary::ConvertToAnimationStateMachine(const FAnimNodeReference& Node, FAnimationStateMachineReference& AnimationStateMachine, EAnimNodeReferenceConversionResult& Result)
{
	AnimationStateMachine = FAnimNodeReference::ConvertToType<FAnimationStateMachineReference>(Node, Result);
}

bool UAnimationStateMachineLibrary::IsStateBlendingIn(const FAnimUpdateContext& UpdateContext, const FAnimationStateResultReference& Node)
{
	bool bResult = false;

	Node.CallAnimNodeFunction<FAnimNode_StateResult>(
		TEXT("IsStateBlendingIn"),
		[&UpdateContext, &bResult](FAnimNode_StateResult& StateResultNode)
		{
			if (const FAnimationUpdateContext* AnimationUpdateContext = UpdateContext.GetContext())
			{
				IAnimClassInterface* AnimBlueprintClass = AnimationUpdateContext->GetAnimClass();

				// Previous node to an FAnimNode_StateResult is always its owning FAnimNode_StateMachine
				const int32 MachineIndex =  AnimBlueprintClass->GetAnimNodeProperties().Num() - 1 - AnimationUpdateContext->GetPreviousNodeId();
				const int32 StateIndex = StateResultNode.GetStateIndex();

				const FAnimInstanceProxy* AnimInstanceProxy = AnimationUpdateContext->AnimInstanceProxy;
				if (const FAnimNode_StateMachine* MachineInstance = AnimInstanceProxy->GetStateMachineInstance(MachineIndex))
				{
					const int32 CurrentStateIndex = MachineInstance->GetCurrentState();

					const float StateWeight = AnimInstanceProxy->GetRecordedStateWeight(MachineInstance->StateMachineIndexInClass, StateIndex);
					if ((StateWeight < 1.0f) && (CurrentStateIndex == StateIndex))
					{
						bResult = true;
					}
				}
			}
		});

	return bResult;
}

bool UAnimationStateMachineLibrary::IsStateBlendingOut(const FAnimUpdateContext& UpdateContext, const FAnimationStateResultReference& Node)
{
	bool bResult = false;

	Node.CallAnimNodeFunction<FAnimNode_StateResult>(
		TEXT("IsStateBlendingOut"),
		[&UpdateContext, &bResult](FAnimNode_StateResult& StateResultNode)
		{
			if (const FAnimationUpdateContext* AnimationUpdateContext = UpdateContext.GetContext())
			{
				IAnimClassInterface* AnimBlueprintClass = AnimationUpdateContext->GetAnimClass();

				// Previous node to an FAnimNode_StateResult is always its owning FAnimNode_StateMachine
				const int32 MachineIndex = AnimBlueprintClass->GetAnimNodeProperties().Num() - 1 - AnimationUpdateContext->GetPreviousNodeId();
				const int32 StateIndex = StateResultNode.GetStateIndex();

				const FAnimInstanceProxy* AnimInstanceProxy = AnimationUpdateContext->AnimInstanceProxy;
				if (const FAnimNode_StateMachine* MachineInstance = AnimInstanceProxy->GetStateMachineInstance(MachineIndex))
				{
					const int32 CurrentStateIndex = MachineInstance->GetCurrentState();

					const float StateWeight = AnimInstanceProxy->GetRecordedStateWeight(MachineInstance->StateMachineIndexInClass, StateIndex);
					if ((StateWeight > 0.0f) && (CurrentStateIndex != StateIndex))
					{
						bResult = true;
					}
				}
			}
		});

	return bResult;
}

void UAnimationStateMachineLibrary::SetState(const FAnimUpdateContext& UpdateContext, const FAnimationStateMachineReference& Node, FName TargetState, float Duration
		, TEnumAsByte<ETransitionLogicType::Type> BlendType, UBlendProfile* BlendProfile, EAlphaBlendOption AlphaBlendOption, UCurveFloat* CustomBlendCurve)
{
	Node.CallAnimNodeFunction<FAnimNode_StateMachine>(
		TEXT("SetState"),
		[&UpdateContext, TargetState, Duration, BlendType, BlendProfile, AlphaBlendOption, CustomBlendCurve](FAnimNode_StateMachine& StateMachineNode)
		{
			if (const FAnimationUpdateContext* AnimationUpdateContext = UpdateContext.GetContext())
			{
				const int32 CurrentStateIndex = StateMachineNode.GetCurrentState();
				const int32 TargetStateIndex = StateMachineNode.GetStateIndex(TargetState);
					
				if(CurrentStateIndex != INDEX_NONE && TargetStateIndex != INDEX_NONE)
				{
					FAnimationTransitionBetweenStates TransitionInfo;
					TransitionInfo.PreviousState = CurrentStateIndex;
					TransitionInfo.NextState = TargetStateIndex;
					TransitionInfo.CrossfadeDuration = Duration;
					TransitionInfo.BlendMode = AlphaBlendOption;
					TransitionInfo.CustomCurve = CustomBlendCurve;
					TransitionInfo.BlendProfile = BlendProfile;
					TransitionInfo.LogicType = BlendType;

					StateMachineNode.TransitionToState(*AnimationUpdateContext, TransitionInfo);
				}
			}
		});
}

FName UAnimationStateMachineLibrary::GetState(const FAnimUpdateContext& UpdateContext, const FAnimationStateMachineReference& Node)
{
	FName OutName = NAME_None;
	Node.CallAnimNodeFunction<FAnimNode_StateMachine>(
		TEXT("GetState"),
		[&UpdateContext, &OutName](FAnimNode_StateMachine& StateMachineNode)
		{
			if (const FAnimationUpdateContext* AnimationUpdateContext = UpdateContext.GetContext())
			{
				const int32 CurrentStateIndex = StateMachineNode.GetCurrentState();
				OutName = StateMachineNode.GetStateInfo(CurrentStateIndex).StateName;
			}
		});

	return OutName;
}

float UAnimationStateMachineLibrary::GetRelevantAnimTimeRemaining(const FAnimUpdateContext& UpdateContext, const FAnimationStateResultReference& Node)
{
	float Result = MAX_flt;

	Node.CallAnimNodeFunction<FAnimNode_StateResult>(
		TEXT("IsStateBlendingOut"),
		[&UpdateContext, &Result](FAnimNode_StateResult& StateResultNode)
	{
		if (const FAnimationUpdateContext* AnimationUpdateContext = UpdateContext.GetContext())
		{
			IAnimClassInterface* AnimBlueprintClass = AnimationUpdateContext->GetAnimClass();

			// Previous node to an FAnimNode_StateResult is always its owning FAnimNode_StateMachine
			const int32 MachineIndex = AnimBlueprintClass->GetAnimNodeProperties().Num() - 1 - AnimationUpdateContext->GetPreviousNodeId();
			const int32 StateIndex = StateResultNode.GetStateIndex();

			const FAnimInstanceProxy* AnimInstanceProxy = AnimationUpdateContext->AnimInstanceProxy;
			if (const FAnimNode_StateMachine* MachineInstance = AnimInstanceProxy->GetStateMachineInstance(MachineIndex))
			{
				Result = MachineInstance->GetRelevantAnimTimeRemaining(AnimInstanceProxy, StateIndex);
			}
		}
	});

	return Result;
}

float UAnimationStateMachineLibrary::GetRelevantAnimTimeRemainingFraction(const FAnimUpdateContext& UpdateContext, const FAnimationStateResultReference& Node)
{
	float Result = 1.f;

	Node.CallAnimNodeFunction<FAnimNode_StateResult>(
		TEXT("IsStateBlendingOut"),
		[&UpdateContext, &Result](FAnimNode_StateResult& StateResultNode)
	{
		if (const FAnimationUpdateContext* AnimationUpdateContext = UpdateContext.GetContext())
		{
			IAnimClassInterface* AnimBlueprintClass = AnimationUpdateContext->GetAnimClass();

			// Previous node to an FAnimNode_StateResult is always its owning FAnimNode_StateMachine
			const int32 MachineIndex = AnimBlueprintClass->GetAnimNodeProperties().Num() - 1 - AnimationUpdateContext->GetPreviousNodeId();
			const int32 StateIndex = StateResultNode.GetStateIndex();

			const FAnimInstanceProxy* AnimInstanceProxy = AnimationUpdateContext->AnimInstanceProxy;
			if (const FAnimNode_StateMachine* MachineInstance = AnimInstanceProxy->GetStateMachineInstance(MachineIndex))
			{
				Result = MachineInstance->GetRelevantAnimTimeRemainingFraction(AnimInstanceProxy, StateIndex);
			}
		}
	});

	return Result;
}
