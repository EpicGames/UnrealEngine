// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendListBaseLibrary.h"
#include "AnimNodes/AnimNode_BlendListBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendListBaseLibrary)


FBlendListBaseReference UBlendListBaseLibrary::ConvertToBlendListBase(const FAnimNodeReference& Node,
	EAnimNodeReferenceConversionResult& Result)
{
	return FAnimNodeReference::ConvertToType<FBlendListBaseReference>(Node, Result);
}

void UBlendListBaseLibrary::ResetNode(const FBlendListBaseReference& BlendListBase)
{
	bool bResult = false;

	BlendListBase.CallAnimNodeFunction<FAnimNode_BlendListBase>(
		TEXT("ResetNode"),
		[](FAnimNode_BlendListBase& InBlendListBase)
	{
		InBlendListBase.Initialize();
	});
}
