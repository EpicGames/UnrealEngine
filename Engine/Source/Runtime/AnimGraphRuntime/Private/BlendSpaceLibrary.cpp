// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendSpaceLibrary.h"
#include "Animation/AnimNode_Inertialization.h"
#include "AnimNodes/AnimNode_BlendSpaceGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendSpaceLibrary)

DEFINE_LOG_CATEGORY_STATIC(LogBlendSpaceLibrary, Verbose, All);

FBlendSpaceReference UBlendSpaceLibrary::ConvertToBlendSpace(const FAnimNodeReference& Node,
	EAnimNodeReferenceConversionResult& Result)
{
	return FAnimNodeReference::ConvertToType<FBlendSpaceReference>(Node, Result);
}

FVector UBlendSpaceLibrary::GetPosition(const FBlendSpaceReference& BlendSpace)
{
	FVector Position = FVector::ZeroVector;

	BlendSpace.CallAnimNodeFunction<FAnimNode_BlendSpaceGraph>(
		TEXT("GetPosition"),
		[&Position](FAnimNode_BlendSpaceGraph& InBlendSpace)
		{
			Position = InBlendSpace.GetPosition();
		});

	return Position;
}

FVector UBlendSpaceLibrary::GetFilteredPosition(const FBlendSpaceReference& BlendSpace)
{
	FVector FilteredPosition = FVector::ZeroVector;

	BlendSpace.CallAnimNodeFunction<FAnimNode_BlendSpaceGraph>(
		TEXT("GetFilteredPosition"),
		[&FilteredPosition](FAnimNode_BlendSpaceGraph& InBlendSpace)
		{
			FilteredPosition = InBlendSpace.GetFilteredPosition();
		});

	return FilteredPosition;
}

void UBlendSpaceLibrary::SnapToPosition(
	const FBlendSpaceReference& BlendSpace,
	FVector NewPosition)
{
	BlendSpace.CallAnimNodeFunction<FAnimNode_BlendSpaceGraph>(
		TEXT("SnapToPosition"),
		[&NewPosition](FAnimNode_BlendSpaceGraph& InBlendSpace)
		{
			InBlendSpace.SnapToPosition(NewPosition);
		});
}
