// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendSpacePlayerLibrary.h"
#include "Animation/AnimNode_Inertialization.h"
#include "AnimNodes/AnimNode_BlendSpacePlayer.h"
#include "Animation/AnimTrace.h"
#include "Animation/AnimInstanceProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendSpacePlayerLibrary)

DEFINE_LOG_CATEGORY_STATIC(LogBlendSpacePlayerLibrary, Verbose, All);

FBlendSpacePlayerReference UBlendSpacePlayerLibrary::ConvertToBlendSpacePlayer(const FAnimNodeReference& Node,
	EAnimNodeReferenceConversionResult& Result)
{
	return FAnimNodeReference::ConvertToType<FBlendSpacePlayerReference>(Node, Result);
}

FBlendSpacePlayerReference UBlendSpacePlayerLibrary::SetBlendSpace(
	const FBlendSpacePlayerReference& BlendSpacePlayer, UBlendSpace* BlendSpace)
{
	BlendSpacePlayer.CallAnimNodeFunction<FAnimNode_BlendSpacePlayer>(
	TEXT("SetBlendSpace"),
	[BlendSpace](FAnimNode_BlendSpacePlayer& InBlendSpacePlayer)
	{
		if (!InBlendSpacePlayer.SetBlendSpace(BlendSpace))
		{
			UE_LOGF(LogBlendSpacePlayerLibrary, Warning, "Could not set blendspace on blendspace player, value is not dynamic. Set it as Always Dynamic.");
		}
	});

	return BlendSpacePlayer;
}

FBlendSpacePlayerReference UBlendSpacePlayerLibrary::SetBlendSpaceWithInertialBlending(
	const FAnimUpdateContext& UpdateContext, const FBlendSpacePlayerReference& BlendSpacePlayer,
	UBlendSpace* BlendSpace, float BlendTime)
{
	BlendSpacePlayer.CallAnimNodeFunction<FAnimNode_BlendSpacePlayer>(
		TEXT("SetBlendSpaceWithInertialBlending"),
		[BlendSpace, &UpdateContext, BlendTime](FAnimNode_BlendSpacePlayer& InBlendSpacePlayer)
		{
			const UBlendSpace* CurrentBlendSpace = InBlendSpacePlayer.GetBlendSpace();
			const bool bBlendSpaceChanged = (CurrentBlendSpace != BlendSpace);

			if (!InBlendSpacePlayer.SetBlendSpace(BlendSpace))
			{
				UE_LOGF(LogBlendSpacePlayerLibrary, Warning, "Could not set blendspace on blendspace player, value is not dynamic. Set it as Always Dynamic.");
			}

			if (bBlendSpaceChanged && BlendTime > 0.0f)
			{
				if (const FAnimationUpdateContext* AnimationUpdateContext = UpdateContext.GetContext())
				{
					if (UE::Anim::IInertializationRequester* InertializationRequester = AnimationUpdateContext->GetMessage<UE::Anim::IInertializationRequester>())
					{
						FInertializationRequest Request;
						Request.Duration = BlendTime;
#if ANIM_TRACE_ENABLED
						Request.NodeId = AnimationUpdateContext->GetCurrentNodeId();
						Request.AnimInstance = AnimationUpdateContext->AnimInstanceProxy->GetAnimInstanceObject();
#endif

						InertializationRequester->RequestInertialization(Request);
					}
				}
				else
				{
					UE_LOGF(LogBlendSpacePlayerLibrary, Warning, "SetBlendSpaceWithIntertialBlending called with invalid context.");
				}
			}
		});
	
	return BlendSpacePlayer;
}

FBlendSpacePlayerReference UBlendSpacePlayerLibrary::SetResetPlayTimeWhenBlendSpaceChanges(
	const FBlendSpacePlayerReference& BlendSpacePlayer, bool bReset)
{
	BlendSpacePlayer.CallAnimNodeFunction<FAnimNode_BlendSpacePlayer>(
		TEXT("SetResetPlayTimeWhenBlendSpaceChanges"),
		[bReset](FAnimNode_BlendSpacePlayer& InBlendSpacePlayer)
		{
			if (!InBlendSpacePlayer.SetResetPlayTimeWhenBlendSpaceChanges(bReset))
			{
				UE_LOGF(LogBlendSpacePlayerLibrary, Warning, "Could not set reset play time when blend space changes on blendspace player, value is not dynamic. Set it as Always Dynamic.");
			}
		});

	return BlendSpacePlayer;
}


FBlendSpacePlayerReference UBlendSpacePlayerLibrary::SetPlayRate(const FBlendSpacePlayerReference& BlendSpacePlayer,
	float PlayRate)
{
	BlendSpacePlayer.CallAnimNodeFunction<FAnimNode_BlendSpacePlayer>(
		TEXT("SetPlayRate"),
		[PlayRate](FAnimNode_BlendSpacePlayer& InBlendSpacePlayer)
		{
			if (!InBlendSpacePlayer.SetPlayRate(PlayRate))
			{
				UE_LOGF(LogBlendSpacePlayerLibrary, Warning, "Could not set play rate on blendspace player, value is not dynamic. Set it as Always Dynamic.");
			}
		});

	return BlendSpacePlayer;
}

FBlendSpacePlayerReference UBlendSpacePlayerLibrary::SetLoop(const FBlendSpacePlayerReference& BlendSpacePlayer,
	bool bLoop)
{
	BlendSpacePlayer.CallAnimNodeFunction<FAnimNode_BlendSpacePlayer>(
		TEXT("SetLoop"),
		[bLoop](FAnimNode_BlendSpacePlayer& InBlendSpacePlayer)
		{
			if (!InBlendSpacePlayer.SetLoop(bLoop))
			{
				UE_LOGF(LogBlendSpacePlayerLibrary, Warning, "Could not set loop on blendspace player, value is not dynamic. Set it as Always Dynamic.");
			}
		});

	return BlendSpacePlayer;
}

UBlendSpace* UBlendSpacePlayerLibrary::GetBlendSpace(const FBlendSpacePlayerReference& BlendSpacePlayer)
{
	UBlendSpace* BlendSpace = nullptr;
	BlendSpacePlayer.CallAnimNodeFunction<FAnimNode_BlendSpacePlayer>(
		TEXT("GetBlendSpace"),
		[&BlendSpace](FAnimNode_BlendSpacePlayer& InBlendSpacePlayer)
		{
			BlendSpace = InBlendSpacePlayer.GetBlendSpace();
		});
	return BlendSpace;
}

FVector UBlendSpacePlayerLibrary::GetPosition(const FBlendSpacePlayerReference& BlendSpacePlayer)
{
	FVector Position = FVector::ZeroVector;
	BlendSpacePlayer.CallAnimNodeFunction<FAnimNode_BlendSpacePlayer>(
		TEXT("GetPosition"),
		[&Position](FAnimNode_BlendSpacePlayer& InBlendSpacePlayer)
		{
			Position = InBlendSpacePlayer.GetPosition();
		});
	return Position;
}

float UBlendSpacePlayerLibrary::GetStartPosition(const FBlendSpacePlayerReference& BlendSpacePlayer)
{
	float StartPosition = 0.f;
	BlendSpacePlayer.CallAnimNodeFunction<FAnimNode_BlendSpacePlayer>(
		TEXT("GetStartPosition"),
		[&StartPosition](FAnimNode_BlendSpacePlayer& InBlendSpacePlayer)
		{
			StartPosition = InBlendSpacePlayer.GetStartPosition();
		});
	return StartPosition;
}

float UBlendSpacePlayerLibrary::GetPlayRate(const FBlendSpacePlayerReference& BlendSpacePlayer)
{
	float PlayRate = 0.f;
	BlendSpacePlayer.CallAnimNodeFunction<FAnimNode_BlendSpacePlayer>(
		TEXT("GetPlayRate"),
		[&PlayRate](FAnimNode_BlendSpacePlayer& InBlendSpacePlayer)
		{
			PlayRate = InBlendSpacePlayer.GetPlayRate();
		});
	return PlayRate;
}

bool UBlendSpacePlayerLibrary::GetLoop(const FBlendSpacePlayerReference& BlendSpacePlayer)
{
	bool bLoop = false;
	BlendSpacePlayer.CallAnimNodeFunction<FAnimNode_BlendSpacePlayer>(
		TEXT("GetLoop"),
		[&bLoop](FAnimNode_BlendSpacePlayer& InBlendSpacePlayer)
		{
			bLoop = InBlendSpacePlayer.IsLooping();
		});
	return bLoop;
}

bool UBlendSpacePlayerLibrary::ShouldResetPlayTimeWhenBlendSpaceChanges(
	const FBlendSpacePlayerReference& BlendSpacePlayer)
{
	bool bResetPlayTimeWhenBlendSpaceChanges = false;
	BlendSpacePlayer.CallAnimNodeFunction<FAnimNode_BlendSpacePlayer>(
		TEXT("ShouldResetPlayTimeWhenBlendSpaceChanges"),
		[&bResetPlayTimeWhenBlendSpaceChanges](FAnimNode_BlendSpacePlayer& InBlendSpacePlayer)
		{
			bResetPlayTimeWhenBlendSpaceChanges = InBlendSpacePlayer.ShouldResetPlayTimeWhenBlendSpaceChanges();
		});
	return bResetPlayTimeWhenBlendSpaceChanges;
}

void UBlendSpacePlayerLibrary::SnapToPosition(
	const FBlendSpacePlayerReference& BlendSpacePlayer,
	FVector NewPosition)
{
	BlendSpacePlayer.CallAnimNodeFunction<FAnimNode_BlendSpacePlayer>(
		TEXT("SnapToPosition"),
		[&NewPosition](FAnimNode_BlendSpacePlayer& InBlendSpacePlayer)
		{
			InBlendSpacePlayer.SnapToPosition(NewPosition);
		});
}

FBlendSpacePlayerReference UBlendSpacePlayerLibrary::SetAccumulatedTime(const FBlendSpacePlayerReference& BlendSpacePlayer, float Time)
{
	BlendSpacePlayer.CallAnimNodeFunction<FAnimNode_BlendSpacePlayer>(
		TEXT("SetAccumulatedTime"),
		[Time](FAnimNode_BlendSpacePlayer& InBlendSpacePlayer)
		{
			InBlendSpacePlayer.SetAccumulatedTime(Time);
		});

	return BlendSpacePlayer;
}

float UBlendSpacePlayerLibrary::GetAccumulatedTime(const FBlendSpacePlayerReference& BlendSpacePlayer)
{
	float AccumulatedTime = 0.f;
	BlendSpacePlayer.CallAnimNodeFunction<FAnimNode_BlendSpacePlayer>(
		TEXT("GetAccumulatedTime"),
		[&AccumulatedTime](FAnimNode_BlendSpacePlayer& InBlendSpacePlayer)
		{
			AccumulatedTime = InBlendSpacePlayer.GetAccumulatedTime();
		});

	return AccumulatedTime;
}
