// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "BoneIndices.h"
#include "BonePose.h"
#include "BoneControllers/AnimNode_RotationMultiplier.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimTrace.h"

/////////////////////////////////////////////////////
// FAnimNode_RotationMultiplier

FAnimNode_RotationMultiplier::FAnimNode_RotationMultiplier()
	: Multiplier(0.0f)
	, RotationAxisToRefer(BA_X)
	, bIsAdditive(false)
{
}

void FAnimNode_RotationMultiplier::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	DebugLine += "(";
	AddDebugNodeData(DebugLine);
	DebugLine += FString::Printf(TEXT(" Src: %s Dst: %s Multiplier: %.2f)"), *SourceBone.BoneName.ToString(), *TargetBone.BoneName.ToString(), Multiplier);
	DebugData.AddDebugItem(DebugLine);

	ComponentPose.GatherDebugData(DebugData);
}

FVector GetAxisVector(const EBoneAxis Axis)
{
	switch (Axis)
	{
	case BA_X:
	default:
		return FVector(1.f,0.f,0.f);
	case BA_Y:
		return FVector(0.f,1.f,0.f);
	case BA_Z:
		return FVector(0.f,0.f,1.f);
	}
}

FQuat FAnimNode_RotationMultiplier::ExtractAngle(const FTransform& RefPoseTransform, const FTransform& LocalBoneTransform, const EBoneAxis Axis)
{
	// local bone transform with reference rotation
	FTransform ReferenceBoneTransform = RefPoseTransform;
	ReferenceBoneTransform.SetTranslation(LocalBoneTransform.GetTranslation());

	// find delta angle between the two quaternions X Axis.
	const FVector RotationAxis = GetAxisVector(Axis);
	const FVector LocalRotationVector = LocalBoneTransform.GetRotation().RotateVector(RotationAxis);
	const FVector ReferenceRotationVector = ReferenceBoneTransform.GetRotation().RotateVector(RotationAxis);

	const FQuat LocalToRefQuat = FQuat::FindBetweenNormals(LocalRotationVector, ReferenceRotationVector);
	checkSlow( LocalToRefQuat.IsNormalized() );

	// Rotate parent bone atom from position in local space to reference skeleton
	// Since our rotation rotates both vectors with shortest arc
	// we're essentially left with a quaternion that has angle difference with reference skeleton version
	const FQuat BoneQuatAligned = LocalToRefQuat* LocalBoneTransform.GetRotation();
	checkSlow( BoneQuatAligned.IsNormalized() );

	// Find that delta angle
	const FQuat DeltaQuat = (ReferenceBoneTransform.GetRotation().Inverse()) * BoneQuatAligned;
	checkSlow( DeltaQuat.IsNormalized() );

#if 0 //DEBUG_TWISTBONECONTROLLER
	UE_LOGF(LogSkeletalControl, Log, "\t ExtractAngle, Bone: %ls", 
		*SourceBone.BoneName.ToString());
	UE_LOGF(LogSkeletalControl, Log, "\t\t Bone Quat: %ls, Rot: %ls, AxisX: %ls", *LocalBoneTransform.GetRotation().ToString(), *LocalBoneTransform.GetRotation().Rotator().ToString(), *LocalRotationVector.ToString() );
	UE_LOGF(LogSkeletalControl, Log, "\t\t BoneRef Quat: %ls, Rot: %ls, AxisX: %ls", *ReferenceBoneTransform.GetRotation().ToString(), *ReferenceBoneTransform.GetRotation().Rotator().ToString(), *ReferenceRotationVector.ToString() );
	UE_LOGF(LogSkeletalControl, Log, "\t\t LocalToRefQuat Quat: %ls, Rot: %ls", *LocalToRefQuat.ToString(), *LocalToRefQuat.Rotator().ToString() );

	const FVector BoneQuatAlignedX = LocalBoneTransform.GetRotation().RotateVector(RotationAxis);
	UE_LOGF(LogSkeletalControl, Log, "\t\t BoneQuatAligned Quat: %ls, Rot: %ls, AxisX: %ls", *BoneQuatAligned.ToString(), *BoneQuatAligned.Rotator().ToString(), *BoneQuatAlignedX.ToString() );
	UE_LOGF(LogSkeletalControl, Log, "\t\t DeltaQuat Quat: %ls, Rot: %ls", *DeltaQuat.ToString(), *DeltaQuat.Rotator().ToString() );

	FTransform BoneAtomAligned(BoneQuatAligned, ReferenceBoneTransform.GetTranslation());
	const FQuat DeltaQuatAligned = FQuat::FindBetween(BoneAtomAligned.GetScaledAxis( EAxis::X ), ReferenceBoneTransform.GetScaledAxis( EAxis::X ));
	UE_LOGF(LogSkeletalControl, Log, "\t\t DeltaQuatAligned Quat: %ls, Rot: %ls", *DeltaQuatAligned.ToString(), *DeltaQuatAligned.Rotator().ToString() );
	FVector DeltaAxis;
	float	DeltaAngle;
	DeltaQuat.ToAxisAndAngle(DeltaAxis, DeltaAngle);
	UE_LOGF(LogSkeletalControl, Log, "\t\t DeltaAxis: %ls, DeltaAngle: %f", *DeltaAxis.ToString(), DeltaAngle );
#endif

	return DeltaQuat;
}

FQuat FAnimNode_RotationMultiplier::MultiplyQuatBasedOnSourceIndex(const FTransform& RefPoseTransform, const FTransform& LocalBoneTransform, const EBoneAxis Axis, float InMultiplier, const FQuat& ReferenceQuat)
{
	// Find delta angle for source bone.
	FQuat DeltaQuat = ExtractAngle(RefPoseTransform, LocalBoneTransform, Axis);

	// Turn to Axis and Angle
	FVector RotationAxis;
	float	RotationAngle;
	DeltaQuat.ToAxisAndAngle(RotationAxis, RotationAngle);

	const FVector DefaultAxis = GetAxisVector(Axis);

	// See if we need to invert angle - shortest path
	if( (RotationAxis | DefaultAxis) < 0.f )
	{
		RotationAxis = -RotationAxis;
		RotationAngle = -RotationAngle;
	}

	// Make sure it is the shortest angle.
	RotationAngle = FMath::UnwindRadians(RotationAngle);

	// New bone rotation
	FQuat OutQuat = ReferenceQuat * FQuat(RotationAxis, RotationAngle* InMultiplier);
	// Normalize resulting quaternion.
	OutQuat.Normalize();

#if 0 //DEBUG_TWISTBONECONTROLLER
	UE_LOGF(LogSkeletalControl, Log, "\t RefQuat: %ls, Rot: %ls", *ReferenceQuat.ToString(), *ReferenceQuat.Rotator().ToString() );
	UE_LOGF(LogSkeletalControl, Log, "\t NewQuat: %ls, Rot: %ls", *OutQuat.ToString(), *OutQuat.Rotator().ToString() );
	UE_LOGF(LogSkeletalControl, Log, "\t RollAxis: %ls, RollAngle: %f", *RotationAxis.ToString(), RotationAngle );
#endif

	return OutQuat;
}

void FAnimNode_RotationMultiplier::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(RotationMultiplier, !IsInGameThread());

	check(OutBoneTransforms.Num() == 0);

	if ( Multiplier != 0.f )
	{
		// Reference bone
		const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();
		const FCompactPoseBoneIndex TargetBoneIndex = TargetBone.GetCompactPoseIndex(BoneContainer);
		const FCompactPoseBoneIndex SourceBoneIndex = SourceBone.GetCompactPoseIndex(BoneContainer);

		const FQuat RefQuat = Output.Pose.GetPose().GetRefPose(TargetBoneIndex).GetRotation();
		const FTransform& SourceRefPose = Output.Pose.GetPose().GetRefPose(SourceBoneIndex);
		FQuat NewQuat = MultiplyQuatBasedOnSourceIndex(SourceRefPose, Output.Pose.GetLocalSpaceTransform(SourceBoneIndex), RotationAxisToRefer, Multiplier, RefQuat);

		FTransform NewLocalTransform = Output.Pose.GetLocalSpaceTransform(TargetBoneIndex);
		
		if (bIsAdditive)
		{
			NewQuat = NewLocalTransform.GetRotation() * NewQuat;
		}
		
		NewLocalTransform.SetRotation(NewQuat);

		const FCompactPoseBoneIndex ParentIndex = Output.Pose.GetPose().GetParentBoneIndex(TargetBoneIndex);
		if( ParentIndex != INDEX_NONE )
		{
			const FTransform& ParentTM = Output.Pose.GetComponentSpaceTransform(ParentIndex);
			FTransform NewTransform = NewLocalTransform * ParentTM;
			OutBoneTransforms.Add( FBoneTransform(TargetBoneIndex, NewTransform) );
		}
		else
		{
			OutBoneTransforms.Add( FBoneTransform(TargetBoneIndex, NewLocalTransform) );
		}
	}

	TRACE_ANIM_NODE_VALUE(Output, TEXT("Source Bone"), SourceBone.BoneName);
	TRACE_ANIM_NODE_VALUE(Output, TEXT("Target Bone"), TargetBone.BoneName);
	TRACE_ANIM_NODE_VALUE(Output, TEXT("Multiplier"), Multiplier);
}

bool FAnimNode_RotationMultiplier::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) 
{
	// if both bones are valid
	return (TargetBone.IsValidToEvaluate(RequiredBones) && (TargetBone==SourceBone || SourceBone.IsValidToEvaluate(RequiredBones)));
}

void FAnimNode_RotationMultiplier::InitializeBoneReferences(const FBoneContainer& RequiredBones) 
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	SourceBone.Initialize(RequiredBones);
	TargetBone.Initialize(RequiredBones);
}
