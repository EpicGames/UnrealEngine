// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_AnimDynamics.h"
#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/WorldSettings.h"
#include "Animation/AnimInstanceProxy.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "SceneInterface.h"
#include "CommonAnimationLibrary.h"
#include "ProfilingDebugging/CsvProfiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_AnimDynamics)

DEFINE_STAT(STAT_AnimDynamicsOverall);
DEFINE_STAT(STAT_AnimDynamicsWindData);
DEFINE_STAT(STAT_AnimDynamicsBoneEval);
DEFINE_STAT(STAT_AnimDynamicsSubSteps);

CSV_DECLARE_CATEGORY_MODULE_EXTERN(ENGINE_API, Animation);

TAutoConsoleVariable<int32> CVarRestrictLod(TEXT("p.AnimDynamicsRestrictLOD"), -1, TEXT("Forces anim dynamics to be enabled for only a specified LOD, -1 to enable on all LODs."));
TAutoConsoleVariable<int32> CVarLODThreshold(TEXT("p.AnimDynamicsLODThreshold"), -1, TEXT("Max LOD that anim dynamics is allowed to run on. Provides a global threshold that overrides per-node the LODThreshold property. -1 means no override."), ECVF_Scalability);
TAutoConsoleVariable<int32> CVarEnableDynamics(TEXT("p.AnimDynamics"), 1, TEXT("Enables/Disables anim dynamics node updates."), ECVF_Scalability);
TAutoConsoleVariable<int32> CVarEnableScaling(TEXT("p.AnimDynamics.ScaleSupport"), 1, TEXT("Enables/Disables scaling support for anim dynamics nodes."));
TAutoConsoleVariable<int32> CVarEnableAdaptiveSubstep(TEXT("p.AnimDynamicsAdaptiveSubstep"), 0, TEXT("Enables/disables adaptive substepping. Adaptive substepping will substep the simulation when it is necessary and maintain a debt buffer for time, always trying to utilise as much time as possible."));
TAutoConsoleVariable<int32> CVarAdaptiveSubstepNumDebtFrames(TEXT("p.AnimDynamicsNumDebtFrames"), 5, TEXT("Number of frames to maintain as time debt when using adaptive substepping, this should be at least 1 or the time debt will never be cleared."));
TAutoConsoleVariable<int32> CVarEnableWind(TEXT("p.AnimDynamicsWind"), 1, TEXT("Enables/Disables anim dynamics wind forces globally."), ECVF_Scalability);
TAutoConsoleVariable<float> CVarComponentAppliedLinearAccClampOverride(TEXT("p.AnimDynamics.ComponentAppliedLinearAccClampOverride"), -1.0f, TEXT("Override the per asset setting for all axis (X,Y & Z) of ComponentAppliedLinearAccClamp for all Anim Dynamics Nodes. Negative values are ignored."));
TAutoConsoleVariable<float> CVarGravityScale(TEXT("p.AnimDynamics.GravityScale"), 1.0f, TEXT("Multiplies the default gravity and the gravity override on all Anim Dynamics Nodes."));
TAutoConsoleVariable<bool>  CVarDisableWorldSpace(TEXT("p.AnimDynamics.DisableWorldSpace"), false, TEXT("Disable World Sapce transform and override it to component space on all Anim Dynamics Nodes."));
TAutoConsoleVariable<bool>  CVarScaleInitialBodyMassByComponentSpace(TEXT("p.AnimDynamics.ScaleInitialBodyMassByComponentSpace"), true, TEXT("Scale Initial Body Mass By Component Space Scale when using World Space."));
TAutoConsoleVariable<bool>  CVarTeleportBodiesOnScaleChange(TEXT("p.AnimDynamics.TeleportBodiesOnScaleChange"), true, TEXT("Teleport Bodies On Scale Change when using World Space."));
TAutoConsoleVariable<bool>  CVarScaleWorldSpaceSprings(TEXT("p.AnimDynamics.ScaleWorldSpaceSprings"), true, TEXT("Scale spring constants to match body scale changes when in world space."));
TAutoConsoleVariable<bool>  CVarPropagateSimulationScaleToOutputPose(TEXT("p.AnimDynamics.EnableUniformPoseScalePropagation"), true, TEXT("Copies a the scale applied to the simulated bodies into the output bone transforms. This scale is the minimum value of the x,y and z scales that are set in the input bone transforms."));
TAutoConsoleVariable<bool>  CVarUpdateBodiesOnScaleChange(TEXT("p.AnimDynamics.EnableUpdateBodiesOnScaleChange"), true, TEXT("When true, input bone transform scale is compared to the scale applied to simulated bodies in the last physics evaluation. Physics body scales will be updated in the next physics evaluation if they are different."));

// FSimSpaceSettingsWithControl forced overrides for testing
bool bAnimDynamics_SimSpace_EnableOverride = false;
FAnimPhysSimSpaceSettings AnimDynamics_SimSpaceOverride;
FAutoConsoleVariableRef CVarAnimDynamicsSpaceOverride(TEXT("p.AnimDynamics.Space.Override"), bAnimDynamics_SimSpace_EnableOverride, TEXT("Force-enable the advanced simulation space movement forces"), ECVF_Default);
FAutoConsoleVariableRef CVarAnimDynamicsSpaceAngularAlpha(TEXT("p.AnimDynamics.Space.SimSpaceAngularAlpha"), AnimDynamics_SimSpaceOverride.SimSpaceAngularAlpha, TEXT("AnimDynamics SimSpaceSettings overrides"), ECVF_Default);
FAutoConsoleVariableRef CVarAnimDynamicsSpaceMaxCompAngVel(TEXT("p.AnimDynamics.Space.MaxAngularVelocity"), AnimDynamics_SimSpaceOverride.MaxAngularVelocity, TEXT("AnimDynamics SimSpaceSettings overrides"), ECVF_Default);
FAutoConsoleVariableRef CVarAnimDynamicsSpaceMaxCompAngAcc(TEXT("p.AnimDynamics.Space.MaxAngularAcceleration"), AnimDynamics_SimSpaceOverride.MaxAngularAcceleration, TEXT("AnimDynamics SimSpaceSettings overrides"), ECVF_Default);

// FindChainBones
// 
// Returns an ordered list of the names of all the bones in a chain between BeginBoneName and EndBoneName.
//
void FindChainBones(const FName BeginBoneName, const FName EndBoneName, TFunctionRef< FName (const FName) > GetParentBoneName, TArray<FName>& OutChainBoneNames)
{
	OutChainBoneNames.Add(BeginBoneName);

	if (!EndBoneName.IsNone())
	{
		const uint32 OutputStartIndex = OutChainBoneNames.Num();

		// Add the end of the chain. We have to walk from the bottom upwards to find a chain
		// as walking downwards doesn't guarantee a single end point.

		// Walk up the chain until we either find the top or hit the root bone
		FName ParentBoneName = EndBoneName;
		for (; !ParentBoneName.IsNone() && (ParentBoneName != BeginBoneName); ParentBoneName = GetParentBoneName(ParentBoneName))
		{
			OutChainBoneNames.Insert(ParentBoneName, OutputStartIndex);
		}

		if (ParentBoneName != BeginBoneName)
		{
			UE_LOGF(LogAnimation, Error, "AnimDynamics: Attempted to find bone chain starting at %ls and ending at %ls but failed.", *BeginBoneName.ToString(), *EndBoneName.ToString());

			// Remove any accumulated chain bone names beyond the start bone from output.
			OutChainBoneNames.RemoveAt(OutputStartIndex, OutChainBoneNames.Num() - OutputStartIndex);
		}
	}
}

const float FAnimNode_AnimDynamics::MaxTimeDebt = (1.0f / 60.0f) * 5.0f; // 5 frames max debt

#if ENABLE_ANIM_DRAW_DEBUG

TAutoConsoleVariable<int32> CVarShowDebug(TEXT("p.animdynamics.showdebug"), 0, TEXT("Enable/disable the drawing of animdynamics data."));
TAutoConsoleVariable<FString> CVarDebugBone(TEXT("p.animdynamics.debugbone"), FString(), TEXT("Filters p.animdynamics.showdebug to a specific bone by name."));

void FAnimNode_AnimDynamics::DrawBodies(FComponentSpacePoseContext& InContext, const TArray<FAnimPhysRigidBody*>& InBodies)
{
	if(CVarShowDebug.GetValueOnAnyThread() == 0)
	{
		return;
	}

	auto ToWorldV = [this](FComponentSpacePoseContext& InPoseContext, const FVector& SimLocation)
	{
		FVector OutLoc = GetComponentSpaceTransformFromSimSpace(SimulationSpace, InPoseContext, FTransform(SimLocation)).GetTranslation();
		OutLoc = InPoseContext.AnimInstanceProxy->GetComponentTransform().TransformPosition(SimLocation);
		return OutLoc;
	};

	FAnimInstanceProxy* Proxy = InContext.AnimInstanceProxy;

	check(Proxy);

	const FString FilteredBoneName = CVarDebugBone.GetValueOnAnyThread();
	const bool bFilterBone = FilteredBoneName.Len() > 0;

	const int32 NumBodies = Bodies.Num();

	check(PhysicsBodyDefinitions.Num() >= NumBodies);

	for(int32 BodyIndex = 0 ; BodyIndex < NumBodies ; ++BodyIndex)
	{
		const FAnimPhysRigidBody& Body = Bodies[BodyIndex].RigidBody.PhysBody;
		const FAnimPhysBodyDefinition& PhysicsBodyDef = PhysicsBodyDefinitions[BodyIndex];

		if(bFilterBone && PhysicsBodyDef.BoundBone.BoneName != FName(*FilteredBoneName))
		{
			continue;
		}

		FTransform Transform(Body.Pose.Orientation, Body.Pose.Position, SimulationSpace != AnimPhysSimSpaceType::World || CVarDisableWorldSpace.GetValueOnAnyThread() ? FVector(Body.Pose.Scale) : FVector(1));
		Transform = GetComponentSpaceTransformFromSimSpace(SimulationSpace, InContext, Transform);

		Proxy->AnimDrawDebugCoordinateSystem(ToWorldV(InContext, Transform.GetTranslation()), Transform.Rotator(), 2.0f * Body.Pose.Scale, false, -1.0f, 0.15f);

		for(const FAnimPhysShape& Shape : Body.Shapes)
		{
			const int32 NumTris = Shape.Triangles.Num();
			for(int32 TriIndex = 0; TriIndex < NumTris; ++TriIndex)
			{
				FIntVector Tri = Shape.Triangles[TriIndex];

				Proxy->AnimDrawDebugLine(ToWorldV(InContext, Transform.TransformPosition(Shape.Vertices[Tri.X])), ToWorldV(InContext, Transform.TransformPosition(Shape.Vertices[Tri.Y])), FColor::Yellow, false, -1.0f, 0.15f);
				Proxy->AnimDrawDebugLine(ToWorldV(InContext, Transform.TransformPosition(Shape.Vertices[Tri.Y])), ToWorldV(InContext, Transform.TransformPosition(Shape.Vertices[Tri.Z])), FColor::Yellow, false, -1.0f, 0.15f);
				Proxy->AnimDrawDebugLine(ToWorldV(InContext, Transform.TransformPosition(Shape.Vertices[Tri.Z])), ToWorldV(InContext, Transform.TransformPosition(Shape.Vertices[Tri.X])), FColor::Yellow, false, -1.0f, 0.15f);
			}
		}
	}

	if(SimulationSpace != AnimPhysSimSpaceType::World || CVarDisableWorldSpace.GetValueOnAnyThread())
	{
		FTransform Origin;

		switch(SimulationSpace)
		{
			case AnimPhysSimSpaceType::Actor:
			{
				Origin = Proxy->GetActorTransform();
			}
			break;
			case AnimPhysSimSpaceType::BoneRelative:
			{
				Origin = Proxy->GetComponentTransform();

				const FCompactPoseBoneIndex CompactPoseBoneIndex(RelativeSpaceBone.BoneIndex);

				if (InContext.Pose.GetPose().IsValidIndex(CompactPoseBoneIndex)) // Check bone index validity here to avoid a fatal assert in the call to GetComponentSpaceTransform.
				{
					Origin *= InContext.Pose.GetComponentSpaceTransform(CompactPoseBoneIndex);
				}
			}
			break;
			case AnimPhysSimSpaceType::Component:
			{
				Origin = Proxy->GetComponentTransform();
			}
			break;
			case AnimPhysSimSpaceType::RootRelative:
			{
				Origin = Proxy->GetComponentTransform() * InContext.Pose.GetComponentSpaceTransform(FCompactPoseBoneIndex(0));
			}
			break;

			default: break;
		}

		Proxy->AnimDrawDebugSphere(Origin.GetTranslation(), 25.0f, 16, FColor::Green, false, -1.0f, 0.15f);
		Proxy->AnimDrawDebugCoordinateSystem(Origin.GetTranslation(), Origin.Rotator(), 3.0f, false, -1.0f, 0.15f);
	}
}
#endif

FAnimPhysSimSpaceSettings::FAnimPhysSimSpaceSettings()
	: SimSpaceAngularAlpha(0)
	, MaxAngularVelocity(10000)
	, MaxAngularAcceleration(10000)
	, ExternalAngularVelocity(FVector::ZeroVector)
{
}

FAnimNode_AnimDynamics::FAnimNode_AnimDynamics()
: LinearDampingOverride(0.0f)
, AngularDampingOverride(0.0f)
, PreviousCompWorldSpaceTM(FTransform::Identity)
, PreviousActorWorldSpaceTM(FTransform::Identity)
, GravityScale(1.0f)
, GravityOverride(ForceInitToZero)
, LinearSpringConstant(0.0f)
, AngularSpringConstant(0.0f)
, WorldSpaceMinimumScale(0.01f)
, WindScale(1.0f)
, AngularBiasOverride(0.0f)
, NumSolverIterationsPreUpdate(4)
, NumSolverIterationsPostUpdate(1)
, ExternalForce(ForceInitToZero)
, SimulationSpace(AnimPhysSimSpaceType::Component)
, LastSimSpace(AnimPhysSimSpaceType::Component)
, InitTeleportType(ETeleportType::None)
, bUseSphericalLimits(false)
, bUsePlanarLimit(true)
, bDoUpdate(true)
, bDoEval(true)
, bOverrideLinearDamping(false)
, bOverrideAngularBias(false)
, bOverrideAngularDamping(false)
, bEnableWind(false)
, bWindWasEnabled(false)
, bUseGravityOverride(false)
, bGravityOverrideInSimSpace(false)
, bLinearSpring(false)
, bAngularSpring(false)
, bChain(false)
, RetargetingSettings(FRotationRetargetingInfo(false /* enabled */))
#if WITH_EDITORONLY_DATA
, BoxExtents_DEPRECATED(FVector::ZeroVector)
, LocalJointOffset_DEPRECATED(FVector::ZeroVector)
, CollisionType_DEPRECATED(AnimPhysCollisionType::CoM)
, SphereCollisionRadius_DEPRECATED(0.0f)
#endif
#if WITH_EDITOR
, bDoPhysicsUpdateInEditor(true)
#endif
, bEnabled(true)
, bShouldUpdateBodiesOnScaleChange(false)
#if ENABLE_ANIM_DRAW_DEBUG
, FilteredBoneIndex(INDEX_NONE)
#endif
{
	ComponentLinearAccScale = FVector::ZeroVector;
	ComponentLinearVelScale = FVector::ZeroVector;
	ComponentAppliedLinearAccClamp = FVector(100000, 100000, 100000);

	PreviousComponentLinearVelocity = FVector::ZeroVector;
	PreviousSimSpaceAngularVelocity = FVector::ZeroVector;

	PhysicsBodyDefinitions.Add(FAnimPhysBodyDefinition());
}

void FAnimNode_AnimDynamics::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_SkeletalControlBase::Initialize_AnyThread(Context);

	FBoneContainer& RequiredBones = Context.AnimInstanceProxy->GetRequiredBones();

	InitializeBoneReferences(RequiredBones);

	if(BoundBone.IsValidToEvaluate(RequiredBones))
	{
		RequestInitialise(ETeleportType::ResetPhysics);
	}

	PreviousCompWorldSpaceTM = Context.AnimInstanceProxy->GetComponentTransform();
	PreviousActorWorldSpaceTM = Context.AnimInstanceProxy->GetActorTransform();
	PreviousSimSpaceTM = PreviousCompWorldSpaceTM;

	NextTimeStep = 0.0f;
	TimeDebt = 0.0f;
}

void FAnimNode_AnimDynamics::UpdateInternal(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(UpdateInternal)
	FAnimNode_SkeletalControlBase::UpdateInternal(Context);

	NextTimeStep = Context.GetDeltaTime();
}

struct FSimBodiesScratch : public TThreadSingleton<FSimBodiesScratch>
{
	TArray<FAnimPhysRigidBody*> SimBodies;
};

bool FAnimNode_AnimDynamics::IsAnimDynamicsSystemEnabledFor(int32 InLOD)
{
	int32 RestrictToLOD = CVarRestrictLod.GetValueOnAnyThread();
	bool bEnabledForLod = RestrictToLOD >= 0 ? InLOD == RestrictToLOD : true;

	// note this doesn't check LODThreshold of global value here. That's checked in
	// GetLODThreshold per node
	return (CVarEnableDynamics.GetValueOnAnyThread() == 1 && bEnabledForLod);
}

FTransform FAnimNode_AnimDynamics::GetSimToWorldSpaceTransform(AnimPhysSimSpaceType SimSpace, FComponentSpacePoseContext& Output) const
{
	FTransform OutTransform;

	if (CVarDisableWorldSpace.GetValueOnAnyThread() && SimSpace == AnimPhysSimSpaceType::World)
	{
		SimSpace = AnimPhysSimSpaceType::Component;
	}

	switch (SimSpace)
	{
	case AnimPhysSimSpaceType::Component:
	{
		OutTransform = Output.AnimInstanceProxy->GetComponentTransform();
		
		break;
	}

	case AnimPhysSimSpaceType::Actor:
	{
		OutTransform = Output.AnimInstanceProxy->GetActorTransform();

		break;
	}

	case AnimPhysSimSpaceType::RootRelative:
	{
		const FBoneContainer& RequiredBones = Output.Pose.GetPose().GetBoneContainer();

		FCompactPoseBoneIndex RootBoneCompactIndex(0);

		FTransform RelativeBoneTransform = Output.Pose.GetComponentSpaceTransform(RootBoneCompactIndex);
		RelativeBoneTransform = Output.AnimInstanceProxy->GetComponentTransform() * RelativeBoneTransform;
		OutTransform = RelativeBoneTransform;

		break;
	}

	case AnimPhysSimSpaceType::BoneRelative:
	{
		const FBoneContainer& RequiredBones = Output.Pose.GetPose().GetBoneContainer();
		if (RelativeSpaceBone.IsValidToEvaluate(RequiredBones))
		{
			FTransform RelativeBoneTransform = Output.Pose.GetComponentSpaceTransform(RelativeSpaceBone.GetCompactPoseIndex(RequiredBones));
			RelativeBoneTransform = Output.AnimInstanceProxy->GetComponentTransform() * RelativeBoneTransform;
			OutTransform = RelativeBoneTransform;
		}

		break;
	}
	case AnimPhysSimSpaceType::World:
	{
		break;
	}

	default:
		break;
	}

	return OutTransform;
}

void FAnimNode_AnimDynamics::CalculateSimulationSpaceMotion(
	AnimPhysSimSpaceType        Space,
	FAnimPhysSimSpaceSettings	Settings,
	FComponentSpacePoseContext& Output,
	const float              Dt,
	FVector& SpaceAngularVel,
	FVector& SpaceAngularAcc)
{
	SpaceAngularVel = FVector::ZeroVector;
	SpaceAngularAcc = FVector::ZeroVector;

	// If the system is disabled, nothing else to do
	if ((Settings.SimSpaceAngularAlpha == 0.0f) || (Dt < UE_SMALL_NUMBER))
	{
		return;
	}

	// If the simulation is in world space, the simulation space is stationary
	if (Space == AnimPhysSimSpaceType::World && !CVarDisableWorldSpace.GetValueOnAnyThread())
	{
		SpaceAngularVel = Settings.ExternalAngularVelocity;
		return;
	}

	const FTransform CurrentSimSpaceTM = GetSimToWorldSpaceTransform(SimulationSpace, Output);

	// World-space component angular velocity and acceleration
	SpaceAngularVel = Chaos::FRotation3::CalculateAngularVelocity(PreviousSimSpaceTM.GetRotation(), CurrentSimSpaceTM.GetRotation(), Dt);
	SpaceAngularAcc = (SpaceAngularVel - PreviousSimSpaceAngularVelocity) / Dt;

	PreviousSimSpaceAngularVelocity = SpaceAngularVel;
	PreviousSimSpaceTM = CurrentSimSpaceTM;

	// Clamped world-space motion of the simulation space
	SpaceAngularVel = SpaceAngularVel.GetClampedToMaxSize(Settings.MaxAngularVelocity) + Settings.ExternalAngularVelocity;
	SpaceAngularAcc = SpaceAngularAcc.GetClampedToMaxSize(Settings.MaxAngularAcceleration);

	// Transform world-space motion into simulation space
	SpaceAngularVel = CurrentSimSpaceTM.InverseTransformVectorNoScale(SpaceAngularVel);
	SpaceAngularAcc = CurrentSimSpaceTM.InverseTransformVectorNoScale(SpaceAngularAcc);

	// Apply WorldAlpha to simulation space rotation (usually to reduce it)
	SpaceAngularVel *= Settings.SimSpaceAngularAlpha;
	SpaceAngularAcc *= Settings.SimSpaceAngularAlpha;

}

void FAnimNode_AnimDynamics::UpdateBodyScaling(const int32 BodyIndex, float Scale) const
{
	if (!PhysicsBodyDefinitions.IsValidIndex(BodyIndex))
	{
		return;
	}

	const FAnimPhysBodyDefinition& PhysicsBodyDef = PhysicsBodyDefinitions[BodyIndex];
	FAnimPhysRigidBody* RigidBody = BaseBodyPtrs[BodyIndex];

	if (RigidBody->Pose.Scale != Scale)
	{
		FAnimPhys::ScaleRigidBodyMass(RigidBody, Scale / RigidBody->Pose.Scale);

		TArray<FAnimPhysShape> BodyShapes;

		BodyShapes.Add(FAnimPhysShape::MakeBox(PhysicsBodyDef.BoxExtents * Scale));

		switch (RigidBody->CollisionType)
		{
		case AnimPhysCollisionType::CustomSphere:
			RigidBody->SphereCollisionRadius = static_cast<float>(PhysicsBodyDef.SphereCollisionRadius * Scale);
			break;
		case AnimPhysCollisionType::InnerSphere:
			RigidBody->SphereCollisionRadius = static_cast<float>(PhysicsBodyDef.BoxExtents.GetAbsMin() * Scale / 2.0);
			break;
		case AnimPhysCollisionType::OuterSphere:
			RigidBody->SphereCollisionRadius = static_cast<float>(PhysicsBodyDef.BoxExtents.GetAbsMax() * Scale / 2.0);
			break;
		default:
			break;
		}
		RigidBody->Shapes = BodyShapes;

		FAnimPhys::RigidBodyShapeScaleUpdate(RigidBody, Scale / RigidBody->Pose.Scale);
	}
}

void FAnimNode_AnimDynamics::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	SCOPE_CYCLE_COUNTER(STAT_AnimDynamicsOverall);
	CSV_SCOPED_TIMING_STAT(Animation, AnimDynamicsEval);

	if (IsAnimDynamicsSystemEnabledFor(Output.AnimInstanceProxy->GetLODLevel()) && bEnabled)
	{
		const int bScalingEnabled = CVarEnableScaling.GetValueOnAnyThread();

		if(LastSimSpace != SimulationSpace)
		{
			// Our sim space has been changed since our last update, we need to convert all of our
			// body transforms into the new space.
			ConvertSimulationSpace(Output, LastSimSpace, SimulationSpace);
		}

		// Pretty nasty - but there isn't really a good way to get clean bone transforms (without the modification from
		// previous runs) so we have to initialize here, checking often so we can restart a simulation in the editor.
		if (InitTeleportType != ETeleportType::None)
		{
			InitPhysics(Output);
			InitTeleportType = ETeleportType::None;
		}

		const FTransform SimWorldTransform = GetSimToWorldSpaceTransform(SimulationSpace != AnimPhysSimSpaceType::World ? SimulationSpace : AnimPhysSimSpaceType::Component, Output);

		if (bScalingEnabled)
		{
			if (bShouldUpdateBodiesOnScaleChange || !SimWorldTransform.GetScale3D().Equals(PreviousSimSpaceTM.GetScale3D(), UE_KINDA_SMALL_NUMBER))
			{
				const bool bWorldSpace = SimulationSpace == AnimPhysSimSpaceType::World && !CVarDisableWorldSpace.GetValueOnAnyThread();
				const bool bTeleportEnabled = bWorldSpace && CVarTeleportBodiesOnScaleChange.GetValueOnAnyThread();

				// If the scaling of the sim space has changed, update the bodies to the proper scale
				const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();
				for (int32 ActiveIndex : ActiveBoneIndices)
				{
					const FAnimPhysBodyDefinition& PhysicsBodyDef = PhysicsBodyDefinitions[ActiveIndex];
					const FBoneReference& CurrentBoneRef = PhysicsBodyDef.BoundBone;

					// If our bone isn't valid, move on
					if (!CurrentBoneRef.IsValidToEvaluate(BoneContainer))
					{
						continue;
					}

					FAnimPhysLinkedBody& ChainBody = Bodies[ActiveIndex];
					FAnimPhysRigidBody& RigidBody = Bodies[ActiveIndex].RigidBody.PhysBody;

					FCompactPoseBoneIndex BoneIndex = CurrentBoneRef.GetCompactPoseIndex(BoneContainer);
					FTransform BoundBoneTransform = GetBoneTransformInSimSpace(Output, BoneIndex);

					float Scale = BoundBoneTransform.GetScale3D().GetAbsMin();
					Scale = Scale < UE_KINDA_SMALL_NUMBER ? UE_KINDA_SMALL_NUMBER : Scale;

					if (RigidBody.Pose.Scale != Scale)
					{
						if (bWorldSpace)
						{
							UpdateBodyScaling(ActiveIndex, Scale);
						}
						RigidBody.Pose.Scale = Scale;

						if (bTeleportEnabled)
						{
							FTransform BodyTransform(RigidBody.Pose.Orientation, RigidBody.Pose.Position - RigidBody.Pose.Orientation.RotateVector(PhysicsBodyDefinitions[ActiveIndex].LocalJointOffset));
							BodyTransform = GetComponentSpaceTransformFromSimSpace(SimulationSpace, Output, BodyTransform, PreviousCompWorldSpaceTM, PreviousActorWorldSpaceTM);

							// move to new space
							BodyTransform = GetSimSpaceTransformFromComponentSpace(SimulationSpace, Output, BodyTransform);

							RigidBody.Pose.Orientation = BodyTransform.GetRotation();
							RigidBody.PreviousOrientation = RigidBody.Pose.Orientation;
							RigidBody.NextOrientation = RigidBody.Pose.Orientation;

							RigidBody.Pose.Position = BodyTransform.GetTranslation() + RigidBody.Pose.Orientation.RotateVector(PhysicsBodyDefinitions[ActiveIndex].LocalJointOffset);
						}
					}
				}

				bShouldUpdateBodiesOnScaleChange = false;
			}
		}

		const FBoneContainer& RequiredBones = Output.Pose.GetPose().GetBoneContainer();
		while(BodiesToReset.Num() > 0)
		{
			FAnimPhysLinkedBody* BodyToReset = BodiesToReset.Pop(EAllowShrinking::No);

			if(BodyToReset && BodyToReset->RigidBody.BoundBone.IsValidToEvaluate(RequiredBones))
			{
				FTransform BoneTransform = GetBoneTransformInSimSpace(Output, BodyToReset->RigidBody.BoundBone.GetCompactPoseIndex(RequiredBones));
				FAnimPhysRigidBody& PhysBody = BodyToReset->RigidBody.PhysBody;

				PhysBody.Pose.Position = BoneTransform.GetTranslation();
				PhysBody.Pose.Orientation = BoneTransform.GetRotation();

				float Scale = BoneTransform.GetScale3D().GetAbsMin();
				Scale = Scale < UE_KINDA_SMALL_NUMBER ? UE_KINDA_SMALL_NUMBER : Scale;
				PhysBody.Pose.Scale = Scale;

				PhysBody.LinearMomentum = FVector::ZeroVector;
				PhysBody.AngularMomentum = FVector::ZeroVector;
			}
		}

		// Disable simulation below minimum scale in world space mode.
		if (bScalingEnabled && (SimulationSpace == AnimPhysSimSpaceType::World) && !CVarDisableWorldSpace.GetValueOnAnyThread() && (SimWorldTransform.GetScale3D().SizeSquared() < WorldSpaceMinimumScale * WorldSpaceMinimumScale))
		{
			return;
		}

		if (ShouldDoPhysicsUpdate() && NextTimeStep > AnimPhysicsMinDeltaTime)
		{
			// Calculate gravity direction
			SimSpaceGravityDirection = TransformWorldVectorToSimSpace(Output, FVector(0.0f, 0.0f, -1.0f));

			FVector OrientedExternalForce = ExternalForce;
			if(!OrientedExternalForce.IsNearlyZero())
			{
				OrientedExternalForce = TransformWorldVectorToSimSpace(Output, OrientedExternalForce);
			}

			// We don't send any bodies that don't have valid bones to the simulation
			TArray<FAnimPhysRigidBody*>& SimBodies = FSimBodiesScratch::Get().SimBodies;
			SimBodies.Empty(SimBodies.Num());
			for(int32& ActiveIndex : ActiveBoneIndices)
			{
				if(BaseBodyPtrs.IsValidIndex(ActiveIndex))
				{
					SimBodies.Add(BaseBodyPtrs[ActiveIndex]);
				}
			}

			FVector ComponentLinearAcc(0.0f);
			FVector SimSpaceGravityOverride = GravityOverride;

			FVector SimFrameAngularVelocity;
			FVector SimFrameAngularAcc;

			FAnimPhysSimSpaceSettings UseSimSpaceSettings = SimSpaceSettings;
			if (bAnimDynamics_SimSpace_EnableOverride)
			{
				if (AnimDynamics_SimSpaceOverride.SimSpaceAngularAlpha >= 0.0f)
				{
					UseSimSpaceSettings.SimSpaceAngularAlpha = AnimDynamics_SimSpaceOverride.SimSpaceAngularAlpha;
				}
				if (AnimDynamics_SimSpaceOverride.MaxAngularVelocity >= 0.0f)
				{
					UseSimSpaceSettings.MaxAngularVelocity = AnimDynamics_SimSpaceOverride.MaxAngularVelocity;
				}
				if (AnimDynamics_SimSpaceOverride.MaxAngularAcceleration >= 0.0f)
				{ 
					UseSimSpaceSettings.MaxAngularAcceleration = AnimDynamics_SimSpaceOverride.MaxAngularAcceleration;
				}
			}

			CalculateSimulationSpaceMotion(SimulationSpace, UseSimSpaceSettings, Output, NextTimeStep, SimFrameAngularVelocity, SimFrameAngularAcc);

			if (SimulationSpace != AnimPhysSimSpaceType::World || CVarDisableWorldSpace.GetValueOnAnyThread())
			{
				// Transform Gravity Override into simulation space
				if (bUseGravityOverride && !bGravityOverrideInSimSpace)
				{
					SimSpaceGravityOverride = TransformWorldVectorToSimSpace(Output, SimSpaceGravityOverride);
				}

				FTransform CurrentTransform = Output.AnimInstanceProxy->GetComponentTransform();

				// Calc linear velocity
				const FVector ComponentDeltaLocation =  CurrentTransform.GetTranslation() - PreviousCompWorldSpaceTM.GetTranslation();
				const FVector ComponentLinearVelocity = (ComponentDeltaLocation / NextTimeStep);

				// Apply acceleration that opposed velocity (basically 'drag')
				if(bScalingEnabled)
				{
					ComponentLinearAcc += TransformWorldVectorToSimSpaceScaled(Output, -ComponentLinearVelocity) * ComponentLinearVelScale;
				}
				else
				{ 
					ComponentLinearAcc += TransformWorldVectorToSimSpace(Output, -ComponentLinearVelocity) * ComponentLinearVelScale;
				}

				// Calc linear acceleration
				const FVector ComponentLinearAcceleration = (ComponentLinearVelocity - PreviousComponentLinearVelocity) / NextTimeStep;
				PreviousComponentLinearVelocity = ComponentLinearVelocity;
				// Apply opposite acceleration to bodies
				if (bScalingEnabled)
				{
					ComponentLinearAcc += TransformWorldVectorToSimSpaceScaled(Output, -ComponentLinearAcceleration) * ComponentLinearAccScale;
				}
				else
				{
					ComponentLinearAcc += TransformWorldVectorToSimSpace(Output, -ComponentLinearAcceleration) * ComponentLinearAccScale;
				}

				// Clamp ComponentLinearAcc to desired strength.	
				FVector LinearAccClamp = ComponentAppliedLinearAccClamp;

				const float LinearAccClampOverride = CVarComponentAppliedLinearAccClampOverride.GetValueOnAnyThread();
				if (LinearAccClampOverride >= 0.0f) // Ignore values < 0
				{
					LinearAccClamp.Set(LinearAccClampOverride, LinearAccClampOverride, LinearAccClampOverride);
				}

				ComponentLinearAcc = ComponentLinearAcc.BoundToBox(-LinearAccClamp, LinearAccClamp);
			}

			// Update gravity.
			{
				const float ExternalGravityScale = CVarGravityScale.GetValueOnAnyThread();

				const FVector AppliedGravityOverride = SimSpaceGravityOverride * ExternalGravityScale;
				const float AppliedGravityScale = GravityScale * ExternalGravityScale;

				for (FAnimPhysRigidBody* ChainBody : SimBodies)
				{
					ChainBody->GravityOverride = AppliedGravityOverride;
					ChainBody->GravityScale = AppliedGravityScale;
				}
			}

			if (CVarEnableAdaptiveSubstep.GetValueOnAnyThread() == 1)
			{
				float CurrentTimeDilation = Output.AnimInstanceProxy->GetTimeDilation();
				float FixedTimeStep = MaxSubstepDeltaTime * CurrentTimeDilation;

				// Clamp the fixed timestep down to max physics tick time.
				// at high speeds the simulation will not converge as the delta time is too high, this will
				// help to keep constraints together at a cost of physical accuracy
				FixedTimeStep = FMath::Clamp(FixedTimeStep, 0.0f, MaxPhysicsDeltaTime);

				// Calculate number of substeps we should do.
				int32 NumIters = FMath::TruncToInt((NextTimeStep + (TimeDebt * CurrentTimeDilation)) / FixedTimeStep);
				NumIters = FMath::Clamp(NumIters, 0, MaxSubsteps);

				SET_DWORD_STAT(STAT_AnimDynamicsSubSteps, NumIters);

				// Store the remaining time as debt for later frames
				TimeDebt = (NextTimeStep + TimeDebt) - (NumIters * FixedTimeStep);
				TimeDebt = FMath::Clamp(TimeDebt, 0.0f, MaxTimeDebt);

				NextTimeStep = FixedTimeStep;

				for (int32 Iter = 0; Iter < NumIters; ++Iter)
				{
					UpdateLimits(Output);
					FAnimPhys::PhysicsUpdate(FixedTimeStep, SimBodies, LinearLimits, AngularLimits, Springs, SimSpaceGravityDirection, OrientedExternalForce, ComponentLinearAcc, SimFrameAngularAcc, SimFrameAngularVelocity, NumSolverIterationsPreUpdate, NumSolverIterationsPostUpdate);
				}
			}
			else
			{
				// Do variable frame-time update
				const float MaxDeltaTime = MaxPhysicsDeltaTime;

				NextTimeStep = FMath::Min(NextTimeStep, MaxDeltaTime);

				UpdateLimits(Output);
				FAnimPhys::PhysicsUpdate(NextTimeStep, SimBodies, LinearLimits, AngularLimits, Springs, SimSpaceGravityDirection, OrientedExternalForce, ComponentLinearAcc, SimFrameAngularAcc, SimFrameAngularVelocity, NumSolverIterationsPreUpdate, NumSolverIterationsPostUpdate);
			}
			
#if ENABLE_ANIM_DRAW_DEBUG
			DrawBodies(Output, SimBodies);
#endif
		}

		if (bDoEval)
		{
			CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_AnimDynamicsBoneEval, FAnimPhys::bEnableDetailedStats);

			const bool bPropagateSimulationScaleToOutputPose = CVarPropagateSimulationScaleToOutputPose.GetValueOnAnyThread();

			const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();

			check(Bodies.Num() <= PhysicsBodyDefinitions.Num());
			check(Bodies.Num() <= PhysicsBodyJointOffsets.Num());

			for (int32 Idx = 0; Idx < Bodies.Num(); ++Idx)
			{
				FBoneReference& CurrentChainBone = PhysicsBodyDefinitions[Idx].BoundBone;
				FAnimPhysRigidBody& CurrentBody = Bodies[Idx].RigidBody.PhysBody;

				// Skip invalid bones
				if(!CurrentChainBone.IsValidToEvaluate(BoneContainer))
				{
					continue;
				}

				FCompactPoseBoneIndex BoneIndex = CurrentChainBone.GetCompactPoseIndex(BoneContainer);

				// Calculate target bone transform from physics body.

				FTransform NewBoneTransform;
				
				if (bScalingEnabled && SimulationSpace == AnimPhysSimSpaceType::World && !CVarDisableWorldSpace.GetValueOnAnyThread())
				{
					NewBoneTransform = FTransform(CurrentBody.Pose.Orientation, CurrentBody.Pose.Position - CurrentBody.Pose.Orientation.RotateVector(PhysicsBodyJointOffsets[Idx]), FVector(CurrentBody.Pose.Scale));
				}
				else
				{
					NewBoneTransform = FTransform(CurrentBody.Pose.Orientation, CurrentBody.Pose.Position - CurrentBody.Pose.Orientation.RotateVector(PhysicsBodyJointOffsets[Idx]));
				}

				if (RetargetingSettings.bEnabled)
				{
					FTransform ParentTransform = FTransform::Identity;
					FCompactPoseBoneIndex ParentBoneIndex = BoneContainer.GetParentBoneIndex(BoneIndex);
					if (ParentBoneIndex != INDEX_NONE)
					{
						ParentTransform = GetBoneTransformInSimSpace(Output, ParentBoneIndex);
					}

					FQuat RetargetedRotation = CommonAnimationLibrary::RetargetSingleRotation(
						NewBoneTransform.GetRotation(),
						RetargetingSettings.Source * ParentTransform,
						RetargetingSettings.Target * ParentTransform,
						RetargetingSettings.CustomCurve,
						RetargetingSettings.EasingType,
						RetargetingSettings.bFlipEasing,
						RetargetingSettings.EasingWeight,
						RetargetingSettings.RotationComponent,
						RetargetingSettings.TwistAxis,
						RetargetingSettings.bUseAbsoluteAngle,
						RetargetingSettings.SourceMinimum,
						RetargetingSettings.SourceMaximum,
						RetargetingSettings.TargetMinimum,
						RetargetingSettings.TargetMaximum);

					NewBoneTransform.SetRotation(RetargetedRotation);
				}

				if (bPropagateSimulationScaleToOutputPose)
				{
					// Override output bone transform scale with a uniform scale vector with the same value on each axis as the scale value applied in the simulation.
					NewBoneTransform.SetScale3D(FVector(CurrentBody.Pose.Scale));
				}

				NewBoneTransform = GetComponentSpaceTransformFromSimSpace(SimulationSpace, Output, NewBoneTransform);

				OutBoneTransforms.Add(FBoneTransform(BoneIndex, NewBoneTransform));
			}
		}

		PreviousSimSpaceTM = SimWorldTransform;

		// Store our sim space in case it changes
		LastSimSpace = SimulationSpace;

		// Store previous component and actor space transform
		PreviousCompWorldSpaceTM = Output.AnimInstanceProxy->GetComponentTransform();
		PreviousActorWorldSpaceTM = Output.AnimInstanceProxy->GetActorTransform();
	}
}

void FAnimNode_AnimDynamics::FindChainBoneNames(const FReferenceSkeleton& ReferenceSkeleton, TArray<FName>& ChainBoneNames)
{
	auto GetParentBoneNameFn = [&ReferenceSkeleton](const FName BoneName)
	{
		int32 BoneIndex = ReferenceSkeleton.FindBoneIndex(BoneName);

		if (ReferenceSkeleton.IsValidIndex(BoneIndex))
		{
			BoneIndex = ReferenceSkeleton.GetParentIndex(BoneIndex);
		}

		if (ReferenceSkeleton.IsValidIndex(BoneIndex))
		{
			return ReferenceSkeleton.GetBoneName(BoneIndex);
		}

		return FName(NAME_None);
	};

	FindChainBones(BoundBone.BoneName, ChainEnd.BoneName, GetParentBoneNameFn, ChainBoneNames);
}

void FAnimNode_AnimDynamics::ValidateChainPhysicsBodyDefinitions(const FReferenceSkeleton& ReferenceSkeleton)
{
	TArray<FName> ChainBoneNames;
	FindChainBoneNames(ReferenceSkeleton, ChainBoneNames);

	// If another array of physics bodies has been pasted over this one we may have too many entries (in which case SetNum will truncate the array) or we may have too few (in which case SetNum will padd with default constucted body defs).
	PhysicsBodyDefinitions.SetNum(ChainBoneNames.Num());

	for (uint32 BodyIndex = 0, BodyIndexMax = FMath::Min(ChainBoneNames.Num(), PhysicsBodyDefinitions.Num()); BodyIndex < BodyIndexMax; ++BodyIndex)
	{
		PhysicsBodyDefinitions[BodyIndex].BoundBone.BoneName = ChainBoneNames[BodyIndex];
	}
}

void FAnimNode_AnimDynamics::UpdateChainPhysicsBodyDefinitions(const FReferenceSkeleton& ReferenceSkeleton)
{
	TArray<FName> ChainBoneNames;
	FindChainBoneNames(ReferenceSkeleton, ChainBoneNames);
	check(ChainBoneNames.Num() > 0);

	// If there was only one physics body then copy its values to all new chain bodies (emulating legacy behaviour), otherwise use values from default construction.
	FAnimPhysBodyDefinition PrototypePhysBodyDef;

	if (PhysicsBodyDefinitions.Num() == 1)
	{
		PrototypePhysBodyDef = PhysicsBodyDefinitions[0];
	}


	// Remove any bodies for bones that are not in the chain.
	PhysicsBodyDefinitions.RemoveAll([&ChainBoneNames](const FAnimPhysBodyDefinition& Value) { return ChainBoneNames.Find(Value.BoundBone.BoneName) == INDEX_NONE;});
	PhysicsBodyDefinitions.Reserve(ChainBoneNames.Num());

	// Create a new Physics Body Def for any new bones in the chain and add them before or after the existing bodies as appropriate to maintain the order of the chain bones.
	{
		uint32 PhysicsBodyDefIndex = 0;

		for (FName BoneName : ChainBoneNames)
		{
			if (!PhysicsBodyDefinitions.FindByPredicate([BoneName](const FAnimPhysBodyDefinition& Value) { return Value.BoundBone.BoneName == BoneName; }))
			{
				// Add the new bone to the chain.
				PhysicsBodyDefinitions.Insert(PrototypePhysBodyDef, PhysicsBodyDefIndex);
				PhysicsBodyDefinitions[PhysicsBodyDefIndex].BoundBone.BoneName = BoneName;
			}

			++PhysicsBodyDefIndex;
		}
	}
}

void FAnimNode_AnimDynamics::InitializeBoneReferences(const FBoneContainer& RequiredBones)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	BoundBone.Initialize(RequiredBones);

	if (bChain)
	{
		ChainEnd.Initialize(RequiredBones);
	}

	for (FAnimPhysPlanarLimit& PlanarLimit : PlanarLimits)
	{
		PlanarLimit.DrivingBone.Initialize(RequiredBones);
	}

	for (FAnimPhysSphericalLimit& SphericalLimit : SphericalLimits)
	{
		SphericalLimit.DrivingBone.Initialize(RequiredBones);
	}

	if (SimulationSpace == AnimPhysSimSpaceType::BoneRelative)
	{
		RelativeSpaceBone.Initialize(RequiredBones);
	}
	
	// If we're currently simulating (LOD change etc.)
	bool bSimulating = ActiveBoneIndices.Num() > 0;

	const int32 NumRefs = PhysicsBodyDefinitions.Num();
	for(int32 BoneRefIdx = 0; BoneRefIdx < NumRefs; ++BoneRefIdx)
	{
		FBoneReference& BoneRef = PhysicsBodyDefinitions[BoneRefIdx].BoundBone;
		BoneRef.Initialize(RequiredBones);

		if(bSimulating)
		{
			if(BoneRef.IsValidToEvaluate(RequiredBones) && !ActiveBoneIndices.Contains(BoneRefIdx))
			{
				// This body is inactive and needs to be reset to bone position
				// as it is now required for the current LOD
				BodiesToReset.Add(&Bodies[BoneRefIdx]);
			}
		}
	}

	ActiveBoneIndices.Empty(ActiveBoneIndices.Num());
	const int32 NumBodies = Bodies.Num();
	for(int32 BodyIdx = 0; BodyIdx < NumBodies; ++BodyIdx)
	{
		FAnimPhysLinkedBody& LinkedBody = Bodies[BodyIdx];
		LinkedBody.RigidBody.BoundBone.Initialize(RequiredBones);

		// If this bone is active in this LOD, add to the active list.
		if(LinkedBody.RigidBody.BoundBone.IsValidToEvaluate(RequiredBones))
		{
			ActiveBoneIndices.Add(BodyIdx);
		}
	}
}

void FAnimNode_AnimDynamics::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	const float ActualBiasedAlpha = AlphaScaleBias.ApplyTo(Alpha);

	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("(Alpha: %.1f%%)"), ActualBiasedAlpha*100.f);

	DebugData.AddDebugItem(DebugLine);
	ComponentPose.GatherDebugData(DebugData);
}

bool FAnimNode_AnimDynamics::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones)
{
	bool bValid = BoundBone.IsValidToEvaluate(RequiredBones);

	if (bChain)
	{
		bool bChainEndValid = ChainEnd.IsValidToEvaluate(RequiredBones);
		bool bSubChainValid = false;

		if(!bChainEndValid)
		{
			// Check for LOD subchain
			int32 NumValidBonesFromRoot = 0;
			for(const FAnimPhysBodyDefinition& PhysicsBodyDef : PhysicsBodyDefinitions)
			{
				if(PhysicsBodyDef.BoundBone.IsValidToEvaluate(RequiredBones))
				{
					bSubChainValid = true;
					break;
				}
			}
		}

		bValid = bValid && (bChainEndValid || bSubChainValid);
	}

	return bValid;
}

int32 FAnimNode_AnimDynamics::GetNumBodies() const
{
	return Bodies.Num();
}

const FAnimPhysRigidBody& FAnimNode_AnimDynamics::GetPhysBody(int32 BodyIndex) const
{
	return Bodies[BodyIndex].RigidBody.PhysBody;
}

FTransform FAnimNode_AnimDynamics::GetBodyComponentSpaceTransform(const FAnimPhysRigidBody& Body, const USkeletalMeshComponent* const SkelComp) const
{
	return GetComponentSpaceTransformFromSimSpace(SimulationSpace, SkelComp, FTransform(Body.Pose.Orientation, Body.Pose.Position));
}

#if WITH_EDITOR
FVector FAnimNode_AnimDynamics::GetBodyLocalJointOffset(const int32 BodyIndex) const
{
	if (PhysicsBodyDefinitions.IsValidIndex(BodyIndex))
	{
		return PhysicsBodyDefinitions[BodyIndex].LocalJointOffset;
	}
	return FVector::ZeroVector;
}

#endif

void FAnimNode_AnimDynamics::RequestInitialise(ETeleportType InTeleportType)
{ 
	// Request an initialization. Teleport type can only go higher - i.e. if we have requested a reset, then a teleport will still reset fully
	InitTeleportType = ((InTeleportType > InitTeleportType) ? InTeleportType : InitTeleportType); 
}

void FAnimNode_AnimDynamics::InitPhysics(FComponentSpacePoseContext& Output)
{
	const bool bScalingSupportEnabled = CVarEnableScaling.GetValueOnAnyThread() && (SimulationSpace == AnimPhysSimSpaceType::World) && !CVarDisableWorldSpace.GetValueOnAnyThread();

	switch (InitTeleportType)
	{
	case ETeleportType::ResetPhysics:
	{
		// Clear up any existing physics data
		TermPhysics();

		const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();

		if (PhysicsBodyDefinitions.Num() && ((PhysicsBodyDefinitions[0].BoundBone.BoneName != BoundBone.BoneName) || (PhysicsBodyDefinitions.Last().BoundBone.BoneName != ChainEnd.BoneName)))
		{
			UpdateChainPhysicsBodyDefinitions(BoneContainer.GetReferenceSkeleton());
		}

		// Transform GravityOverride to simulation space if necessary.
		const FVector GravityOverrideSimSpace = (bUseGravityOverride && !bGravityOverrideInSimSpace) ? TransformWorldVectorToSimSpace(Output, GravityOverride) : GravityOverride;
		const float ExternalGravityScale = CVarGravityScale.GetValueOnAnyThread();

		check(PhysicsBodyDefinitions.Num() > 0);
		if (PhysicsBodyDefinitions.Num() > 0)
		{
			Bodies.Reserve(PhysicsBodyDefinitions.Num());
			PhysicsBodyDefinitions[0].BoundBone = BoundBone;
		}

		ConstraintOffsets.Reset(PhysicsBodyDefinitions.Num());
		PhysicsBodyJointOffsets.Reset(PhysicsBodyDefinitions.Num());

		FTransform PreviousBodyTransformSimSpace;
		PreviousBodyTransformSimSpace.SetIdentity();

		const FTransform InitialComponentTransform = GetSimToWorldSpaceTransform(AnimPhysSimSpaceType::Component, Output);
		float InitialComponentScale = InitialComponentTransform.GetScale3D().GetAbsMin();
		InitialComponentScale = InitialComponentScale < UE_KINDA_SMALL_NUMBER ? UE_KINDA_SMALL_NUMBER : InitialComponentScale;

		for (FAnimPhysBodyDefinition& PhysicsBodyDef : PhysicsBodyDefinitions)
		{
			TArray<FAnimPhysShape> BodyShapes;

			PhysicsBodyDef.BoundBone.Initialize(BoneContainer);

			FTransform BodyTransform = GetBoneTransformInSimSpace(Output, PhysicsBodyDef.BoundBone.GetCompactPoseIndex(BoneContainer));

			float BodyScale = BodyTransform.GetScale3D().GetAbsMin();
			BodyScale = BodyScale < UE_KINDA_SMALL_NUMBER ? UE_KINDA_SMALL_NUMBER : BodyScale;
			
			if (bScalingSupportEnabled)
			{
				BodyShapes.Add(FAnimPhysShape::MakeBox(PhysicsBodyDef.BoxExtents * BodyScale));
				BodyTransform.SetTranslation(BodyTransform.TransformPosition(PhysicsBodyDef.LocalJointOffset));
			}
			else
			{
				BodyShapes.Add(FAnimPhysShape::MakeBox(PhysicsBodyDef.BoxExtents));
				BodyTransform.SetTranslation(BodyTransform.GetTranslation() + BodyTransform.GetRotation().RotateVector(PhysicsBodyDef.LocalJointOffset)); // Transform for physics body in Sim Space.
			}

			FAnimPhysLinkedBody NewChainBody(BodyShapes, BodyTransform.GetTranslation(), PhysicsBodyDef.BoundBone);
			FAnimPhysRigidBody& PhysicsBody = NewChainBody.RigidBody.PhysBody;
			PhysicsBody.Pose.Orientation = BodyTransform.GetRotation();
			PhysicsBody.Pose.Scale = BodyScale;
			PhysicsBody.PreviousOrientation = PhysicsBody.Pose.Orientation;
			PhysicsBody.NextOrientation = PhysicsBody.Pose.Orientation;
			PhysicsBody.CollisionType = PhysicsBodyDef.CollisionType;

			//Masses originally were always 1 regardless of Scale or BodyShape Bounds. Allow the user to enable mass scaling so masses are adjusted based on initial scale.
			if (bScalingSupportEnabled && !CVarDisableWorldSpace.GetValueOnAnyThread()  && CVarScaleInitialBodyMassByComponentSpace.GetValueOnAnyThread())
			{
				FAnimPhys::ScaleRigidBodyMass(&PhysicsBody, InitialComponentScale);
			}

			switch (PhysicsBody.CollisionType)
			{
			case AnimPhysCollisionType::CustomSphere:
				PhysicsBody.SphereCollisionRadius = static_cast<float>(bScalingSupportEnabled ? PhysicsBodyDef.SphereCollisionRadius * BodyScale : PhysicsBodyDef.SphereCollisionRadius);
				break;
			case AnimPhysCollisionType::InnerSphere:
				PhysicsBody.SphereCollisionRadius = static_cast<float>(bScalingSupportEnabled ? (PhysicsBodyDef.BoxExtents.GetAbsMin() * BodyScale / 2.0) : PhysicsBodyDef.BoxExtents.GetAbsMin() / 2.0);
				break;
			case AnimPhysCollisionType::OuterSphere:
				PhysicsBody.SphereCollisionRadius = static_cast<float>(bScalingSupportEnabled ? (PhysicsBodyDef.BoxExtents.GetAbsMax() * BodyScale / 2.0) : PhysicsBodyDef.BoxExtents.GetAbsMax() / 2.0);
				break;
			default:
				break;
			}

			if (bOverrideLinearDamping)
			{
				PhysicsBody.bLinearDampingOverriden = true;
				PhysicsBody.LinearDamping = LinearDampingOverride;
			}

			if (bOverrideAngularDamping)
			{
				PhysicsBody.bAngularDampingOverriden = true;
				PhysicsBody.AngularDamping = AngularDampingOverride;
			}

			PhysicsBody.GravityScale = GravityScale * ExternalGravityScale;
			PhysicsBody.bUseGravityOverride = bUseGravityOverride;
			PhysicsBody.GravityOverride = GravityOverrideSimSpace * ExternalGravityScale;

			PhysicsBody.bWindEnabled = bWindWasEnabled;

			if (Bodies.Num() > 0)
			{
				// Link to parent
				NewChainBody.ParentBody = &Bodies.Last().RigidBody;

				// Calculate constraint offset positions in the space of each body. For world space we scale the limit distances in UpdateLimits to account for dynamic scale. Store the unscaled value in ConstraintOffsets.
				const FVector ConstaintLocationSimSpace = (BodyTransform.GetTranslation() - PreviousBodyTransformSimSpace.GetTranslation()) * 0.5f / (bScalingSupportEnabled ? PhysicsBody.Pose.Scale : 1);
				ConstraintOffsets.Add(FAnimConstraintOffsetPair(PreviousBodyTransformSimSpace.GetRotation().UnrotateVector(ConstaintLocationSimSpace), BodyTransform.GetRotation().UnrotateVector(-ConstaintLocationSimSpace)));
			}
			else
			{
				// The first physics body is constrained to the location of it's bound bone.
				ConstraintOffsets.Add(FAnimConstraintOffsetPair(FVector::ZeroVector, -PhysicsBodyDef.LocalJointOffset)); // Offset is the vector from the physics body to its associated bone in the bodies local space.
			}

			// Set up transient constraint data
			const bool bXAxisLocked = PhysicsBodyDef.ConstraintSetup.LinearXLimitType != AnimPhysLinearConstraintType::Free && PhysicsBodyDef.ConstraintSetup.LinearAxesMin.X - PhysicsBodyDef.ConstraintSetup.LinearAxesMax.X == 0.0f;
			const bool bYAxisLocked = PhysicsBodyDef.ConstraintSetup.LinearYLimitType != AnimPhysLinearConstraintType::Free && PhysicsBodyDef.ConstraintSetup.LinearAxesMin.Y - PhysicsBodyDef.ConstraintSetup.LinearAxesMax.Y == 0.0f;
			const bool bZAxisLocked = PhysicsBodyDef.ConstraintSetup.LinearZLimitType != AnimPhysLinearConstraintType::Free && PhysicsBodyDef.ConstraintSetup.LinearAxesMin.Z - PhysicsBodyDef.ConstraintSetup.LinearAxesMax.Z == 0.0f;
			PhysicsBodyDef.ConstraintSetup.bLinearFullyLocked = bXAxisLocked && bYAxisLocked && bZAxisLocked;

			Bodies.Add(NewChainBody);
			ActiveBoneIndices.Add(Bodies.Num() - 1);
			PhysicsBodyJointOffsets.Add(PhysicsBodyDef.LocalJointOffset);

			PreviousBodyTransformSimSpace = BodyTransform;
		}

		BaseBodyPtrs.Reset();
		for (FAnimPhysLinkedBody& Body : Bodies)
		{
			BaseBodyPtrs.Add(&Body.RigidBody.PhysBody);
		}

		// Cache physics settings to avoid accessing UPhysicsSettings continuously
		if (UPhysicsSettings* Settings = UPhysicsSettings::Get())
		{
			AnimPhysicsMinDeltaTime = Settings->AnimPhysicsMinDeltaTime;
			MaxPhysicsDeltaTime = Settings->MaxPhysicsDeltaTime;
			MaxSubstepDeltaTime = Settings->MaxSubstepDeltaTime;
			MaxSubsteps = Settings->MaxSubsteps;
		}
		else
		{
			AnimPhysicsMinDeltaTime = 0.f;
			MaxPhysicsDeltaTime = (1.0f / 30.0f);
			MaxSubstepDeltaTime = (1.0f / 60.0f);
			MaxSubsteps = 4;
		}

		SimSpaceGravityDirection = TransformWorldVectorToSimSpace(Output, FVector(0.0f, 0.0f, -1.0f));
	}
	break;

	case ETeleportType::TeleportPhysics:
	{
		// Clear any external forces
		ExternalForce = FVector::ZeroVector;

		// Move any active bones
		for (const int32& BodyIndex : ActiveBoneIndices)
		{
			FAnimPhysRigidBody& Body = Bodies[BodyIndex].RigidBody.PhysBody;

			// Get old comp space transform
			FTransform BodyTransform(Body.Pose.Orientation, Body.Pose.Position - Body.Pose.Orientation.RotateVector(PhysicsBodyDefinitions[BodyIndex].LocalJointOffset));
			BodyTransform = GetComponentSpaceTransformFromSimSpace(SimulationSpace, Output, BodyTransform, PreviousCompWorldSpaceTM, PreviousActorWorldSpaceTM);

			// move to new space
			BodyTransform = GetSimSpaceTransformFromComponentSpace(SimulationSpace, Output, BodyTransform);

			Body.Pose.Orientation = BodyTransform.GetRotation();
			Body.PreviousOrientation = Body.Pose.Orientation;
			Body.NextOrientation = Body.Pose.Orientation;

			Body.Pose.Position = BodyTransform.GetTranslation() + Body.Pose.Orientation.RotateVector(PhysicsBodyDefinitions[BodyIndex].LocalJointOffset);
		}
	}
	break;
	}

	InitTeleportType = ETeleportType::None;
	PreviousCompWorldSpaceTM = Output.AnimInstanceProxy->GetComponentTransform();
	PreviousActorWorldSpaceTM = Output.AnimInstanceProxy->GetActorTransform();

	PreviousSimSpaceTM = GetSimToWorldSpaceTransform(SimulationSpace != AnimPhysSimSpaceType::World ? SimulationSpace : AnimPhysSimSpaceType::Component, Output);
}

void FAnimNode_AnimDynamics::TermPhysics()
{
	Bodies.Reset();
	LinearLimits.Reset();
	AngularLimits.Reset();
	Springs.Reset();
	ActiveBoneIndices.Reset();
	PhysicsBodyJointOffsets.Reset();
	ConstraintOffsets.Reset();
	BodiesToReset.Reset();

	for (FAnimPhysBodyDefinition& PhysicsBodyDef : PhysicsBodyDefinitions)
	{
		PhysicsBodyDef.BoundBone.BoneIndex = INDEX_NONE;
	}
}

void FAnimNode_AnimDynamics::UpdateLimits(FComponentSpacePoseContext& Output)
{
	CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_AnimDynamicsLimitUpdate, FAnimPhys::bEnableDetailedStats);

	const bool bShouldTestForBoneTransformScaleChange = CVarUpdateBodiesOnScaleChange.GetValueOnAnyThread();

	// We're always going to use the same number so don't realloc
	LinearLimits.Empty(LinearLimits.Num());
	AngularLimits.Empty(AngularLimits.Num());
	Springs.Empty(Springs.Num());

	const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();

	const bool bShouldScale = SimulationSpace == AnimPhysSimSpaceType::World && !CVarDisableWorldSpace.GetValueOnAnyThread() && CVarEnableScaling.GetValueOnAnyThread();
	const bool bScaleSprings = bShouldScale && CVarScaleWorldSpaceSprings.GetValueOnAnyThread();

	for(int32 ActiveIndex : ActiveBoneIndices)
	{
		const FAnimPhysBodyDefinition& PhysicsBodyDef = PhysicsBodyDefinitions[ActiveIndex];
		const FBoneReference& CurrentBoneRef = PhysicsBodyDef.BoundBone;

		// If our bone isn't valid, move on
		if(!CurrentBoneRef.IsValidToEvaluate(BoneContainer))
		{
			continue;
		}

		FAnimPhysLinkedBody& ChainBody = Bodies[ActiveIndex];
		FAnimPhysRigidBody& RigidBody = Bodies[ActiveIndex].RigidBody.PhysBody;
		
		FAnimPhysRigidBody* PrevBody = nullptr;
		if (ChainBody.ParentBody)
		{
			PrevBody = &ChainBody.ParentBody->PhysBody;
		}

		// Get joint transform
		FCompactPoseBoneIndex BoneIndex = CurrentBoneRef.GetCompactPoseIndex(BoneContainer);

		FTransform BoundBoneComponentSpaceTransform = Output.Pose.GetComponentSpaceTransform(BoneIndex);
		FTransform BoundBoneTransform = GetSimSpaceTransformFromComponentSpace(SimulationSpace, Output, BoundBoneComponentSpaceTransform);

		if(bShouldTestForBoneTransformScaleChange && !bShouldUpdateBodiesOnScaleChange)
		{
			// Check if input bone transform scale has changed since the last physics init.
			bShouldUpdateBodiesOnScaleChange = RigidBody.Pose.Scale != BoundBoneComponentSpaceTransform.GetScale3D().GetAbsMin();
		}

		FTransform ShapeTransform = BoundBoneTransform;
		
		if (PrevBody)
		{
			ShapeTransform = FTransform(FQuat::Identity, ConstraintOffsets[ActiveIndex].Body0Offset * (bShouldScale ? PrevBody->Pose.Scale : 1));
		}

		// Local offset to joint for Body1
		const FVector Body1JointOffset = ConstraintOffsets[ActiveIndex].Body1Offset * (bShouldScale ? RigidBody.Pose.Scale : 1);

		if (PhysicsBodyDef.ConstraintSetup.bLinearFullyLocked)
		{
			// Rather than calculate prismatic limits, just lock the transform (1 limit instead of 6)
			FAnimPhys::ConstrainPositionNailed(NextTimeStep, LinearLimits, PrevBody, ShapeTransform.GetTranslation(), &RigidBody, Body1JointOffset);
		}
		else
		{
			if (PhysicsBodyDef.ConstraintSetup.LinearXLimitType != AnimPhysLinearConstraintType::Free)
			{
				FAnimPhys::ConstrainAlongDirection(NextTimeStep, LinearLimits, PrevBody, ShapeTransform.GetTranslation(), &RigidBody, Body1JointOffset, ShapeTransform.GetRotation().GetAxisX(), FVector2D(PhysicsBodyDef.ConstraintSetup.LinearAxesMin.X, PhysicsBodyDef.ConstraintSetup.LinearAxesMax.X));
			}

			if (PhysicsBodyDef.ConstraintSetup.LinearYLimitType != AnimPhysLinearConstraintType::Free)
			{
				FAnimPhys::ConstrainAlongDirection(NextTimeStep, LinearLimits, PrevBody, ShapeTransform.GetTranslation(), &RigidBody, Body1JointOffset, ShapeTransform.GetRotation().GetAxisY(), FVector2D(PhysicsBodyDef.ConstraintSetup.LinearAxesMin.Y, PhysicsBodyDef.ConstraintSetup.LinearAxesMax.Y));
			}

			if (PhysicsBodyDef.ConstraintSetup.LinearZLimitType != AnimPhysLinearConstraintType::Free)
			{
				FAnimPhys::ConstrainAlongDirection(NextTimeStep, LinearLimits, PrevBody, ShapeTransform.GetTranslation(), &RigidBody, Body1JointOffset, ShapeTransform.GetRotation().GetAxisZ(), FVector2D(PhysicsBodyDef.ConstraintSetup.LinearAxesMin.Z, PhysicsBodyDef.ConstraintSetup.LinearAxesMax.Z));
			}
		}

		if (PhysicsBodyDef.ConstraintSetup.AngularConstraintType == AnimPhysAngularConstraintType::Angular)
		{
#if WITH_EDITOR
			// Check the ranges are valid when running in the editor, log if something is wrong
			if(PhysicsBodyDef.ConstraintSetup.AngularLimitsMin.X > PhysicsBodyDef.ConstraintSetup.AngularLimitsMax.X ||
			   PhysicsBodyDef.ConstraintSetup.AngularLimitsMin.Y > PhysicsBodyDef.ConstraintSetup.AngularLimitsMax.Y ||
			   PhysicsBodyDef.ConstraintSetup.AngularLimitsMin.Z > PhysicsBodyDef.ConstraintSetup.AngularLimitsMax.Z)
			{
				UE_LOGF(LogAnimation, Warning, "AnimDynamics: Min/Max angular limits for bone %ls incorrect, at least one min axis value is greater than the corresponding max.", *BoundBone.BoneName.ToString());
			}
#endif

			// Add angular limits. any limit with 360+ degree range is ignored and left free.
			FAnimPhys::ConstrainAngularRange(NextTimeStep, AngularLimits, PrevBody, &RigidBody, ShapeTransform.GetRotation(), PhysicsBodyDef.ConstraintSetup.TwistAxis, PhysicsBodyDef.ConstraintSetup.AngularLimitsMin, PhysicsBodyDef.ConstraintSetup.AngularLimitsMax, bOverrideAngularBias ? AngularBiasOverride : AnimPhysicsConstants::JointBiasFactor);
		}
		else
		{
			FAnimPhys::ConstrainConeAngle(NextTimeStep, AngularLimits, PrevBody, BoundBoneTransform.GetRotation().GetAxisX(), &RigidBody, FVector(1.0f, 0.0f, 0.0f), PhysicsBodyDef.ConstraintSetup.ConeAngle, bOverrideAngularBias ? AngularBiasOverride : AnimPhysicsConstants::JointBiasFactor);
		}

		if(PlanarLimits.Num() > 0 && bUsePlanarLimit)
		{
			for(FAnimPhysPlanarLimit& PlanarLimit : PlanarLimits)
			{
				FTransform LimitPlaneTransform = PlanarLimit.PlaneTransform;
				if(PlanarLimit.DrivingBone.IsValidToEvaluate(BoneContainer))
				{
					FCompactPoseBoneIndex DrivingBoneIndex = PlanarLimit.DrivingBone.GetCompactPoseIndex(BoneContainer);

					FTransform DrivingBoneTransform = GetBoneTransformInSimSpace(Output, DrivingBoneIndex);

					LimitPlaneTransform *= DrivingBoneTransform;
				}
				
				FAnimPhys::ConstrainPlanar(NextTimeStep, LinearLimits, &RigidBody, LimitPlaneTransform);
			}
		}

		if(SphericalLimits.Num() > 0 && bUseSphericalLimits)
		{
			for(FAnimPhysSphericalLimit& SphericalLimit : SphericalLimits)
			{
				FTransform SphereTransform = FTransform::Identity;
				SphereTransform.SetTranslation(SphericalLimit.SphereLocalOffset);

				if(SphericalLimit.DrivingBone.IsValidToEvaluate(BoneContainer))
				{
					FCompactPoseBoneIndex DrivingBoneIndex = SphericalLimit.DrivingBone.GetCompactPoseIndex(BoneContainer);

					FTransform DrivingBoneTransform = GetBoneTransformInSimSpace(Output, DrivingBoneIndex);

					SphereTransform *= DrivingBoneTransform;
				}

				switch(SphericalLimit.LimitType)
				{
				case ESphericalLimitType::Inner:
					FAnimPhys::ConstrainSphericalInner(NextTimeStep, LinearLimits, &RigidBody, SphereTransform, SphericalLimit.LimitRadius * (bShouldScale ? RigidBody.Pose.Scale : 1));
					break;
				case ESphericalLimitType::Outer:
					FAnimPhys::ConstrainSphericalOuter(NextTimeStep, LinearLimits, &RigidBody, SphereTransform, SphericalLimit.LimitRadius * (bShouldScale ? RigidBody.Pose.Scale : 1));
					break;
				default:
					break;
				}
			}
		}

		// Add spring if we need spring forces
		if (bAngularSpring || bLinearSpring)
		{
			FAnimPhys::CreateSpring(Springs, PrevBody, ShapeTransform.GetTranslation(), &RigidBody, FVector::ZeroVector);
			FAnimPhysSpring& NewSpring = Springs.Last();
			NewSpring.SpringConstantLinear = LinearSpringConstant * (bScaleSprings ? RigidBody.Pose.Scale : 1);
			NewSpring.SpringConstantAngular = AngularSpringConstant * (bScaleSprings ? RigidBody.Pose.Scale * RigidBody.Pose.Scale * RigidBody.Pose.Scale : 1);
			NewSpring.AngularTarget = PhysicsBodyDef.ConstraintSetup.AngularTarget.GetSafeNormal();
			NewSpring.AngularTargetAxis = PhysicsBodyDef.ConstraintSetup.AngularTargetAxis;
			NewSpring.TargetOrientationOffset = ShapeTransform.GetRotation();
			NewSpring.bApplyAngular = bAngularSpring;
			NewSpring.bApplyLinear = bLinearSpring;
		}
	}
}

void FAnimNode_AnimDynamics::PreUpdate(const UAnimInstance* InAnimInstance)
{
	// If dynamics are disabled, skip all this work as it'll never get used
	if(CVarEnableDynamics.GetValueOnAnyThread() == 0)
	{
		return;
	}

	if(!InAnimInstance)
	{
		// No anim instance, won't be able to find our world.
		return;
	}
	
	const USkeletalMeshComponent* SkelComp = InAnimInstance->GetSkelMeshComponent();
	
	if(!SkelComp || !SkelComp->GetWorld())
	{
		// Can't find our world.
		return;
	}

	bEnabled = SkelComp->GetAllowAnimDynamicsAnimNode();

	const UWorld* World = SkelComp->GetWorld();

	if(CVarEnableWind.GetValueOnAnyThread() == 1 && bEnableWind)
	{
		CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_AnimDynamicsWindData, FAnimPhys::bEnableDetailedStats);

		for(FAnimPhysRigidBody* Body : BaseBodyPtrs)
		{
			Body->bWindEnabled = bEnableWind;

			if(Body->bWindEnabled && World->Scene)
			{
				FSceneInterface* Scene = World->Scene;

				// Unused by our simulation but needed for the call to GetWindParameters below
				float WindMinGust;
				float WindMaxGust;

				// Setup wind data
				Body->bWindEnabled = true;
				Scene->GetWindParameters_GameThread(SkelComp->GetComponentTransform().TransformPosition(Body->Pose.Position), Body->WindData.WindDirection, Body->WindData.WindSpeed, WindMinGust, WindMaxGust);

				Body->WindData.WindDirection = SkelComp->GetComponentTransform().Inverse().TransformVector(Body->WindData.WindDirection);
				Body->WindData.WindAdaption = FMath::FRandRange(0.0f, 2.0f);
				Body->WindData.BodyWindScale = WindScale;
			}
		}
	}
	else if (bWindWasEnabled)
	{
		CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_AnimDynamicsWindData, FAnimPhys::bEnableDetailedStats);
	
		bWindWasEnabled = false;
		for(FAnimPhysRigidBody* Body : BaseBodyPtrs)
		{
			Body->bWindEnabled = false;
		}
	}

#if ENABLE_ANIM_DRAW_DEBUG
	FilteredBoneIndex = INDEX_NONE;
	if(SkelComp)
	{
		FString FilteredBoneName = CVarDebugBone.GetValueOnGameThread();
		if(FilteredBoneName.Len() > 0)
		{
			FilteredBoneIndex = SkelComp->GetBoneIndex(FName(*FilteredBoneName));
		}
	}
#endif
}

int32 FAnimNode_AnimDynamics::GetLODThreshold() const
{
	if(CVarLODThreshold.GetValueOnAnyThread() != -1)
	{
		if(LODThreshold != -1)
		{
			return FMath::Min(LODThreshold, CVarLODThreshold.GetValueOnAnyThread());
		}
		else
		{
			return CVarLODThreshold.GetValueOnAnyThread();
		}
	}
	else
	{
		return LODThreshold;
	}
}

FTransform FAnimNode_AnimDynamics::GetBoneTransformInSimSpace(FComponentSpacePoseContext& Output, const FCompactPoseBoneIndex& BoneIndex) const
{
	FTransform Transform = Output.Pose.GetComponentSpaceTransform(BoneIndex);

	return GetSimSpaceTransformFromComponentSpace(SimulationSpace, Output, Transform);
}

FTransform FAnimNode_AnimDynamics::GetComponentSpaceTransformFromSimSpace(AnimPhysSimSpaceType SimSpace, FComponentSpacePoseContext& Output, const FTransform& InSimTransform) const
{
	return GetComponentSpaceTransformFromSimSpace(SimSpace, Output, InSimTransform, Output.AnimInstanceProxy->GetComponentTransform(), Output.AnimInstanceProxy->GetActorTransform());
}

FTransform FAnimNode_AnimDynamics::GetComponentSpaceTransformFromSimSpace(AnimPhysSimSpaceType SimSpace, FComponentSpacePoseContext& Output, const FTransform& InSimTransform, const FTransform& InCompWorldSpaceTM, const FTransform& InActorWorldSpaceTM) const
{
	FTransform OutTransform = InSimTransform;

	if (SimSpace == AnimPhysSimSpaceType::World && CVarDisableWorldSpace.GetValueOnAnyThread())
	{
		SimSpace = AnimPhysSimSpaceType::Component;
	}

	switch(SimSpace)
	{
		// Change nothing, already in component space
	case AnimPhysSimSpaceType::Component:
	{
		break;
	}

	case AnimPhysSimSpaceType::Actor:
	{
		FTransform WorldTransform(OutTransform * InActorWorldSpaceTM);
		WorldTransform.SetToRelativeTransform(InCompWorldSpaceTM);
		OutTransform = WorldTransform;

		break;
	}

	case AnimPhysSimSpaceType::RootRelative:
	{
		const FBoneContainer& RequiredBones = Output.Pose.GetPose().GetBoneContainer();

		FCompactPoseBoneIndex RootBoneCompactIndex(0);

		FTransform RelativeBoneTransform = Output.Pose.GetComponentSpaceTransform(RootBoneCompactIndex);
		OutTransform = OutTransform * RelativeBoneTransform;

		break;
	}

	case AnimPhysSimSpaceType::BoneRelative:
	{
		const FBoneContainer& RequiredBones = Output.Pose.GetPose().GetBoneContainer();
		if(RelativeSpaceBone.IsValidToEvaluate(RequiredBones))
		{
			FTransform RelativeBoneTransform = Output.Pose.GetComponentSpaceTransform(RelativeSpaceBone.GetCompactPoseIndex(RequiredBones));
			OutTransform = OutTransform * RelativeBoneTransform;
		}

		break;
	}
	case AnimPhysSimSpaceType::World:
	{
		OutTransform *= InCompWorldSpaceTM.Inverse();
	}

	default:
		break;
	}

	return OutTransform;
}

FTransform FAnimNode_AnimDynamics::GetComponentSpaceTransformFromSimSpace(AnimPhysSimSpaceType SimSpace, const USkeletalMeshComponent* const SkelComp, const FTransform& InSimTransform) const
{
	FTransform OutTransformCS = InSimTransform;

	check(SkelComp);
	if (SkelComp)
	{
		if (SimSpace == AnimPhysSimSpaceType::RootRelative)
		{
			const FTransform RelativeBoneTransform = SkelComp->GetBoneTransform(0);
			OutTransformCS = InSimTransform * RelativeBoneTransform;
		}
		else if (SimSpace == AnimPhysSimSpaceType::BoneRelative)
		{
			const FTransform RelativeBoneTransform = SkelComp->GetBoneTransform(SkelComp->GetBoneIndex(RelativeSpaceBone.BoneName));
			OutTransformCS = InSimTransform * RelativeBoneTransform;
		}
	}

	return OutTransformCS;
}

FTransform FAnimNode_AnimDynamics::GetSimSpaceTransformFromComponentSpace(AnimPhysSimSpaceType SimSpace, FComponentSpacePoseContext& Output, const FTransform& InComponentTransform) const
{
	FTransform ResultTransform = InComponentTransform;

	if (SimSpace == AnimPhysSimSpaceType::World && CVarDisableWorldSpace.GetValueOnAnyThread())
	{
		SimSpace = AnimPhysSimSpaceType::Component;
	}

	switch(SimSpace)
	{
		// Change nothing, already in component space
	case AnimPhysSimSpaceType::Component:
	{
		break;
	}

	case AnimPhysSimSpaceType::Actor:
	{
		FTransform WorldTransform = ResultTransform * Output.AnimInstanceProxy->GetComponentTransform();
		WorldTransform.SetToRelativeTransform(Output.AnimInstanceProxy->GetActorTransform());
		ResultTransform = WorldTransform;

		break;
	}

	case AnimPhysSimSpaceType::RootRelative:
	{
		const FBoneContainer& RequiredBones = Output.Pose.GetPose().GetBoneContainer();

		FCompactPoseBoneIndex RootBoneCompactIndex(0);

		FTransform RelativeBoneTransform = Output.Pose.GetComponentSpaceTransform(RootBoneCompactIndex);
		ResultTransform = ResultTransform.GetRelativeTransform(RelativeBoneTransform);

		break;
	}

	case AnimPhysSimSpaceType::BoneRelative:
	{
		const FBoneContainer& RequiredBones = Output.Pose.GetPose().GetBoneContainer();
		if(RelativeSpaceBone.IsValidToEvaluate(RequiredBones))
		{
			FTransform RelativeBoneTransform = Output.Pose.GetComponentSpaceTransform(RelativeSpaceBone.GetCompactPoseIndex(RequiredBones));
			ResultTransform = ResultTransform.GetRelativeTransform(RelativeBoneTransform);
		}

		break;
	}

	case AnimPhysSimSpaceType::World:
	{
		// Out to world space
		ResultTransform *= Output.AnimInstanceProxy->GetComponentTransform();
	}

	default:
		break;
	}

	return ResultTransform;
}

FVector FAnimNode_AnimDynamics::TransformWorldVectorToSimSpace(FComponentSpacePoseContext& Output, const FVector& InVec) const
{
	FVector OutVec = InVec;

	// If Disable World Space cVar is enabled act like it is in component space
	if (SimulationSpace == AnimPhysSimSpaceType::World && CVarDisableWorldSpace.GetValueOnAnyThread())
	{
		OutVec = Output.AnimInstanceProxy->GetComponentTransform().InverseTransformVectorNoScale(OutVec);
	}

	switch(SimulationSpace)
	{
	case AnimPhysSimSpaceType::Component:
	{
		OutVec = Output.AnimInstanceProxy->GetComponentTransform().InverseTransformVectorNoScale(OutVec);

		break;
	}

	case AnimPhysSimSpaceType::Actor:
	{
		OutVec = Output.AnimInstanceProxy->GetActorTransform().InverseTransformVectorNoScale(OutVec);

		break;
	}

	case AnimPhysSimSpaceType::RootRelative:
	{
		const FBoneContainer& RequiredBones = Output.Pose.GetPose().GetBoneContainer();

		FCompactPoseBoneIndex RootBoneCompactIndex(0);

		FTransform RelativeBoneTransform = Output.Pose.GetComponentSpaceTransform(RootBoneCompactIndex);
		RelativeBoneTransform = Output.AnimInstanceProxy->GetComponentTransform() * RelativeBoneTransform;
		OutVec = RelativeBoneTransform.InverseTransformVectorNoScale(OutVec);

		break;
	}

	case AnimPhysSimSpaceType::BoneRelative:
	{
		const FBoneContainer& RequiredBones = Output.Pose.GetPose().GetBoneContainer();
		if(RelativeSpaceBone.IsValidToEvaluate(RequiredBones))
		{
			FTransform RelativeBoneTransform = Output.Pose.GetComponentSpaceTransform(RelativeSpaceBone.GetCompactPoseIndex(RequiredBones));
			RelativeBoneTransform = Output.AnimInstanceProxy->GetComponentTransform() * RelativeBoneTransform;
			OutVec = RelativeBoneTransform.InverseTransformVectorNoScale(OutVec);
		}

		break;
	}
	case AnimPhysSimSpaceType::World:
	{
		break;
	}

	default:
		break;
	}

	return OutVec;
}

FVector FAnimNode_AnimDynamics::TransformWorldVectorToSimSpaceScaled(FComponentSpacePoseContext& Output, const FVector& InVec) const
{
	FVector OutVec = InVec;

	// If Disable World Space cVar is enabled act like it is in component space
	if (SimulationSpace == AnimPhysSimSpaceType::World && CVarDisableWorldSpace.GetValueOnAnyThread())
	{
		OutVec = Output.AnimInstanceProxy->GetComponentTransform().InverseTransformVector(OutVec);
	}

	switch (SimulationSpace)
	{
	case AnimPhysSimSpaceType::Component:
	{
		OutVec = Output.AnimInstanceProxy->GetComponentTransform().InverseTransformVector(OutVec);

		break;
	}

	case AnimPhysSimSpaceType::Actor:
	{
		OutVec = Output.AnimInstanceProxy->GetActorTransform().InverseTransformVector(OutVec);

		break;
	}

	case AnimPhysSimSpaceType::RootRelative:
	{
		const FBoneContainer& RequiredBones = Output.Pose.GetPose().GetBoneContainer();

		FCompactPoseBoneIndex RootBoneCompactIndex(0);

		FTransform RelativeBoneTransform = Output.Pose.GetComponentSpaceTransform(RootBoneCompactIndex);
		RelativeBoneTransform = Output.AnimInstanceProxy->GetComponentTransform() * RelativeBoneTransform;
		OutVec = RelativeBoneTransform.InverseTransformVector(OutVec);

		break;
	}

	case AnimPhysSimSpaceType::BoneRelative:
	{
		const FBoneContainer& RequiredBones = Output.Pose.GetPose().GetBoneContainer();
		if (RelativeSpaceBone.IsValidToEvaluate(RequiredBones))
		{
			FTransform RelativeBoneTransform = Output.Pose.GetComponentSpaceTransform(RelativeSpaceBone.GetCompactPoseIndex(RequiredBones));
			RelativeBoneTransform = Output.AnimInstanceProxy->GetComponentTransform() * RelativeBoneTransform;
			OutVec = RelativeBoneTransform.InverseTransformVector(OutVec);
		}

		break;
	}
	case AnimPhysSimSpaceType::World:
	{
		break;
	}

	default:
		break;
	}

	return OutVec;
}

void FAnimNode_AnimDynamics::ConvertSimulationSpace(FComponentSpacePoseContext& Output, AnimPhysSimSpaceType From, AnimPhysSimSpaceType To) const
{
	const int bScalingSupportEnabled = CVarEnableScaling.GetValueOnAnyThread();
	const bool bWorldDisabled = CVarDisableWorldSpace.GetValueOnAnyThread();

	for(int Idx = 0; Idx < BaseBodyPtrs.Num(); Idx++)
	{
		FAnimPhysRigidBody* Body = BaseBodyPtrs[Idx];
		if(!Body)
		{
			continue;
		}

		// Get transform
		FTransform BodyTransform(Body->Pose.Orientation, Body->Pose.Position, FVector(Body->Pose.Scale));
		// Out to component space
		BodyTransform = GetComponentSpaceTransformFromSimSpace(LastSimSpace, Output, BodyTransform);
		// In to new space
		BodyTransform = GetSimSpaceTransformFromComponentSpace(SimulationSpace, Output, BodyTransform);

		// Push back to body
		Body->Pose.Orientation = BodyTransform.GetRotation();
		Body->Pose.Position = BodyTransform.GetTranslation();

		float NewScale = BodyTransform.GetScale3D().GetAbsMin();
		NewScale = NewScale < UE_KINDA_SMALL_NUMBER ? UE_KINDA_SMALL_NUMBER : NewScale;

		if (bScalingSupportEnabled)
		{
			if (From == AnimPhysSimSpaceType::World && !bWorldDisabled && To != AnimPhysSimSpaceType::World)
			{
				UpdateBodyScaling(Idx, 1);
			}
			else if (From != AnimPhysSimSpaceType::World && To == AnimPhysSimSpaceType::World && !bWorldDisabled)
			{
				UpdateBodyScaling(Idx, NewScale * Body->Pose.Scale);
			}
		}

		Body->Pose.Scale = NewScale;
	}
}

bool FAnimNode_AnimDynamics::ShouldDoPhysicsUpdate() const
{
	#if WITH_EDITOR
	if (!bDoPhysicsUpdateInEditor)
	{
		return false;
	}
	#endif

	return bDoUpdate;
}

