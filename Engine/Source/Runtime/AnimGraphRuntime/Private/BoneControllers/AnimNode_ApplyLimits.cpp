// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_ApplyLimits.h"
#include "AnimationCoreLibrary.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimStats.h"
#include "AnimationRuntime.h"
#include "AngularLimit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_ApplyLimits)

/////////////////////////////////////////////////////
// FAnimNode_ApplyLimits

FAnimNode_ApplyLimits::FAnimNode_ApplyLimits()
{
}

void FAnimNode_ApplyLimits::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	DebugLine += "(";
	AddDebugNodeData(DebugLine);
	DebugLine += FString::Printf(TEXT(")"));
	DebugData.AddDebugItem(DebugLine);

	ComponentPose.GatherDebugData(DebugData);
}


void FAnimNode_ApplyLimits::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(ApplyLimits, !IsInGameThread());

	checkSlow(OutBoneTransforms.Num() == 0);

	FPoseContext LocalPose0(Output.AnimInstanceProxy);
	FPoseContext LocalPose1(Output.AnimInstanceProxy);
	FCSPose<FCompactPose>::ConvertComponentPosesToLocalPoses(Output.Pose, LocalPose0.Pose);
	FCSPose<FCompactPose>::ConvertComponentPosesToLocalPoses(Output.Pose, LocalPose1.Pose);
	LocalPose0.Curve = Output.Curve;
	LocalPose1.Curve = Output.Curve;
	LocalPose0.CustomAttributes = Output.CustomAttributes;
	LocalPose1.CustomAttributes = Output.CustomAttributes;
	
	const FTransform ComponentTransform = Output.AnimInstanceProxy->GetComponentTransform();
	const FBoneContainer& BoneContainer = LocalPose0.Pose.GetBoneContainer();

	bool bAppliedLimit = false;
	const int32 AngularLimitCount = AngularRangeLimits.Num();
	for (int32 AngularLimitIndex = 0; AngularLimitIndex < AngularLimitCount; ++AngularLimitIndex)
	{
		const FAngularRangeLimit& AngularLimit = AngularRangeLimits[AngularLimitIndex];
		const FCompactPoseBoneIndex BoneIndex = AngularLimit.Bone.GetCompactPoseIndex(BoneContainer);

		FTransform& BoneTransform = LocalPose0.Pose[BoneIndex];

		const FTransform& RefBoneTransform = BoneContainer.GetRefPoseTransform(BoneIndex);

		FQuat BoneRotation = BoneTransform.GetRotation();
		if (AnimationCore::ConstrainAngularRangeUsingEuler(BoneRotation, RefBoneTransform.GetRotation(), AngularLimit.LimitMin + AngularOffsets[AngularLimitIndex], AngularLimit.LimitMax + AngularOffsets[AngularLimitIndex]))
		{
			BoneTransform.SetRotation(BoneRotation);
			bAppliedLimit = true;
		}
	}

	if(bAppliedLimit)
	{
		const float BlendWeight = FMath::Clamp<float>(ActualAlpha, 0.f, 1.f);

		FPoseContext BlendedPose(Output.AnimInstanceProxy);

		const FAnimationPoseData AnimationPoseData0(LocalPose0);
		const FAnimationPoseData AnimationPoseData1(LocalPose1);
		FAnimationPoseData BlendedAnimationPoseData(BlendedPose);
		FAnimationRuntime::BlendTwoPosesTogether(AnimationPoseData0, AnimationPoseData1, BlendWeight, BlendedAnimationPoseData);

 		Output.Pose.InitPose(BlendedPose.Pose);
	 	Output.Curve = BlendedPose.Curve;
	}
}

bool FAnimNode_ApplyLimits::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones)
{
	for (FAngularRangeLimit& AngularLimit : AngularRangeLimits)
	{
		if (AngularLimit.Bone.IsValidToEvaluate(RequiredBones))
		{
			return true;
		}
	}

	return false;
}

void FAnimNode_ApplyLimits::RecalcLimits()
{
	AngularOffsets.SetNumZeroed(AngularRangeLimits.Num());
}

void FAnimNode_ApplyLimits::OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance)
{
	RecalcLimits();
}

void FAnimNode_ApplyLimits::InitializeBoneReferences(const FBoneContainer& RequiredBones)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	for (FAngularRangeLimit& AngularLimit : AngularRangeLimits)
	{
		AngularLimit.Bone.Initialize(RequiredBones);
	}

	RecalcLimits();
}


