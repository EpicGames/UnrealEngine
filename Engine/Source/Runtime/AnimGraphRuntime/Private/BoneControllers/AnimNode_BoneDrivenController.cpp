// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_BoneDrivenController.h"

#include "Components/SkeletalMeshComponent.h"
#include "Curves/CurveFloat.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_BoneDrivenController)

/////////////////////////////////////////////////////
// FAnimNode_BoneDrivenController

FAnimNode_BoneDrivenController::FAnimNode_BoneDrivenController()
	: DrivingCurve(nullptr)
	, Multiplier(1.0f)
	, RangeMin(-1.0f)
	, RangeMax(1.0f)
	, RemappedMin(0.0f)
	, RemappedMax(1.0f)
#if WITH_EDITORONLY_DATA
	, TargetComponent_DEPRECATED(EComponentType::None)
#endif
	, DestinationMode(EDrivenDestinationMode::Bone)
	, ModificationMode(EDrivenBoneModificationMode::AddToInput)
	, SourceComponent(EComponentType::None)
	, bUseRange(false)
	, bAffectTargetTranslationX(false)
	, bAffectTargetTranslationY(false)
	, bAffectTargetTranslationZ(false)
	, bAffectTargetRotationX(false)
	, bAffectTargetRotationY(false)
	, bAffectTargetRotationZ(false)
	, bAffectTargetScaleX(false)
	, bAffectTargetScaleY(false)
	, bAffectTargetScaleZ(false)
{
}

void FAnimNode_BoneDrivenController::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += "(";
	AddDebugNodeData(DebugLine);
	if (DestinationMode == EDrivenDestinationMode::Bone)
	{
		DebugLine += FString::Printf(TEXT("  DrivingBone: %s\nDrivenBone: %s"), *SourceBone.BoneName.ToString(), *TargetBone.BoneName.ToString());
	}
	else
	{
		DebugLine += FString::Printf(TEXT("  DrivingBone: %s\nDrivenParameter: %s"), *SourceBone.BoneName.ToString(), *ParameterName.ToString());
	}
	
	DebugData.AddDebugItem(DebugLine);

	ComponentPose.GatherDebugData(DebugData);
}

void FAnimNode_BoneDrivenController::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(BoneDrivenController, !IsInGameThread());

	check(OutBoneTransforms.Num() == 0);

	// Early out if we're not driving from or to anything
	if (SourceComponent == EComponentType::None || DestinationMode != EDrivenDestinationMode::Bone)
	{
		return;
	}

	// Get the Local space transform and the ref pose transform to see how the transform for the source bone has changed
	const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();
	const FTransform& SourceRefPoseBoneTransform = BoneContainer.GetRefPoseArray()[SourceBone.BoneIndex];
	const FTransform SourceCurrentBoneTransform = Output.Pose.GetLocalSpaceTransform(SourceBone.GetCompactPoseIndex(BoneContainer));

	const double FinalDriverValue = ExtractSourceValue(SourceCurrentBoneTransform, SourceRefPoseBoneTransform);
	
	
	// Calculate a new local-space bone position by adding or replacing target components in the current local space position
	const FCompactPoseBoneIndex TargetBoneIndex = TargetBone.GetCompactPoseIndex(BoneContainer);

	const FTransform OriginalLocalTM = Output.Pose.GetLocalSpaceTransform(TargetBoneIndex);
	FVector NewTrans(OriginalLocalTM.GetTranslation());
	FVector NewScale(OriginalLocalTM.GetScale3D());
	FQuat NewRot(OriginalLocalTM.GetRotation());

	if (ModificationMode == EDrivenBoneModificationMode::AddToInput)
	{
		// Add the mapped value to the target components
		if (bAffectTargetTranslationX) { NewTrans.X += FinalDriverValue; }
		if (bAffectTargetTranslationY) { NewTrans.Y += FinalDriverValue; }
		if (bAffectTargetTranslationZ) { NewTrans.Z += FinalDriverValue; }

		if (bAffectTargetRotationX || bAffectTargetRotationY || bAffectTargetRotationZ)
		{
			FVector NewRotDeltaEuler(ForceInitToZero);
			if (bAffectTargetRotationX) { NewRotDeltaEuler.X = FinalDriverValue; }
			if (bAffectTargetRotationY) { NewRotDeltaEuler.Y = FinalDriverValue; }
			if (bAffectTargetRotationZ) { NewRotDeltaEuler.Z = FinalDriverValue; }
			NewRot = NewRot * FQuat::MakeFromEuler(NewRotDeltaEuler);
		}

		if (bAffectTargetScaleX) { NewScale.X += FinalDriverValue; }
		if (bAffectTargetScaleY) { NewScale.Y += FinalDriverValue; }
		if (bAffectTargetScaleZ) { NewScale.Z += FinalDriverValue; }
	}
	else if (ModificationMode == EDrivenBoneModificationMode::ReplaceComponent)
	{
		// Replace the target components with the mapped value
		if (bAffectTargetTranslationX) { NewTrans.X = FinalDriverValue; }
		if (bAffectTargetTranslationY) { NewTrans.Y = FinalDriverValue; }
		if (bAffectTargetTranslationZ) { NewTrans.Z = FinalDriverValue; }

		if (bAffectTargetRotationX || bAffectTargetRotationY || bAffectTargetRotationZ)
		{
			FVector NewRotEuler(NewRot.Euler());
			if (bAffectTargetRotationX) { NewRotEuler.X = FinalDriverValue; }
			if (bAffectTargetRotationY) { NewRotEuler.Y = FinalDriverValue; }
			if (bAffectTargetRotationZ) { NewRotEuler.Z = FinalDriverValue; }
			NewRot = FQuat::MakeFromEuler(NewRotEuler);
		}

		if (bAffectTargetScaleX) { NewScale.X = FinalDriverValue; }
		if (bAffectTargetScaleY) { NewScale.Y = FinalDriverValue; }
		if (bAffectTargetScaleZ) { NewScale.Z = FinalDriverValue; }
	}
	else if (ModificationMode == EDrivenBoneModificationMode::AddToRefPose)
	{
		const FTransform RefPoseTransform = Output.Pose.GetPose().GetRefPose(TargetBoneIndex);

		// Add the mapped value to the ref pose components
		if (bAffectTargetTranslationX) { NewTrans.X = RefPoseTransform.GetTranslation().X + FinalDriverValue; }
		if (bAffectTargetTranslationY) { NewTrans.Y = RefPoseTransform.GetTranslation().Y + FinalDriverValue; }
		if (bAffectTargetTranslationZ) { NewTrans.Z = RefPoseTransform.GetTranslation().Z + FinalDriverValue; }

		if (bAffectTargetRotationX || bAffectTargetRotationY || bAffectTargetRotationZ)
		{
			const FVector RefPoseRotEuler(RefPoseTransform.GetRotation().Euler());

			// Replace any components that are being driven with their ref pose value first and create a delta rotation as well
			FVector SourceRotationEuler(NewRot.Euler());
			FVector NewRotDeltaEuler(ForceInitToZero);
			if (bAffectTargetRotationX) { SourceRotationEuler.X = RefPoseRotEuler.X; NewRotDeltaEuler.X = FinalDriverValue; }
			if (bAffectTargetRotationY) { SourceRotationEuler.Y = RefPoseRotEuler.Y; NewRotDeltaEuler.Y = FinalDriverValue; }
			if (bAffectTargetRotationZ) { SourceRotationEuler.Z = RefPoseRotEuler.Z; NewRotDeltaEuler.Z = FinalDriverValue; }

			// Combine the (modified) source and the delta rotation
			NewRot = FQuat::MakeFromEuler(SourceRotationEuler) * FQuat::MakeFromEuler(NewRotDeltaEuler);
		}

		if (bAffectTargetScaleX) { NewScale.X = RefPoseTransform.GetScale3D().X + FinalDriverValue; }
		if (bAffectTargetScaleY) { NewScale.Y = RefPoseTransform.GetScale3D().Y + FinalDriverValue; }
		if (bAffectTargetScaleZ) { NewScale.Z = RefPoseTransform.GetScale3D().Z + FinalDriverValue; }
	}
	else
	{
		ensureMsgf(false, TEXT("Unknown entry in EDrivenBoneModificationMode"));
	}

	const FTransform ModifiedLocalTM(NewRot, NewTrans, NewScale);

	// If we have a parent, concatenate the transform, otherwise just take the new transform
	const FCompactPoseBoneIndex ParentIndex = Output.Pose.GetPose().GetParentBoneIndex(TargetBoneIndex);

	if (ParentIndex != INDEX_NONE)
	{
		const FTransform& ParentTM = Output.Pose.GetComponentSpaceTransform(ParentIndex);

		OutBoneTransforms.Add(FBoneTransform(TargetBoneIndex, ModifiedLocalTM * ParentTM));
	}
	else
	{
		OutBoneTransforms.Add(FBoneTransform(TargetBoneIndex, ModifiedLocalTM));
	}

#if ANIM_TRACE_ENABLED
	TRACE_ANIM_NODE_VALUE(Output, TEXT("Driving Bone"), SourceBone.BoneName);
	if (DestinationMode == EDrivenDestinationMode::Bone)
	{
		TRACE_ANIM_NODE_VALUE(Output, TEXT("Driven Bone"), TargetBone.BoneName);
	}
	else
	{
		TRACE_ANIM_NODE_VALUE(Output, TEXT("Driven Parameter"), ParameterName);
	}
#endif
}

void FAnimNode_BoneDrivenController::EvaluateComponentSpaceInternal(FComponentSpacePoseContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateComponentSpaceInternal)
	// Early out if we're not driving from or to anything
	if (SourceComponent == EComponentType::None || DestinationMode == EDrivenDestinationMode::Bone)
	{
		return;
	}

	// Get the Local space transform and the ref pose transform to see how the transform for the source bone has changed
	const FBoneContainer& BoneContainer = Context.Pose.GetPose().GetBoneContainer();
	const FTransform& SourceRefPoseBoneTransform = BoneContainer.GetRefPoseArray()[SourceBone.BoneIndex];
	const FTransform SourceCurrentBoneTransform = Context.Pose.GetLocalSpaceTransform(SourceBone.GetCompactPoseIndex(BoneContainer));

	if (DestinationMode == EDrivenDestinationMode::MorphTarget || DestinationMode == EDrivenDestinationMode::MaterialParameter)
	{
		const double FinalDriverValue = ExtractSourceValue(SourceCurrentBoneTransform, SourceRefPoseBoneTransform);
		
		//	Morph target and Material parameter curves
		Context.Curve.Set(ParameterName, static_cast<float>(FinalDriverValue));
	}
}

const double FAnimNode_BoneDrivenController::ExtractSourceValue(const FTransform &InCurrentBoneTransform, const FTransform &InRefPoseBoneTransform)
{
	// Resolve source value
	double SourceValue = 0.0;
	if (SourceComponent < EComponentType::RotationX)
	{
		const FVector TranslationDiff = InCurrentBoneTransform.GetLocation() - InRefPoseBoneTransform.GetLocation();
		SourceValue = TranslationDiff[(int32)(SourceComponent - EComponentType::TranslationX)];
	}
	else if (SourceComponent < EComponentType::Scale)
	{
		const FVector RotationDiff = (InCurrentBoneTransform.GetRotation() * InRefPoseBoneTransform.GetRotation().Inverse()).Euler();
		SourceValue = RotationDiff[(int32)(SourceComponent - EComponentType::RotationX)];
	}
	else if (SourceComponent == EComponentType::Scale)
	{
		const FVector CurrentScale = InCurrentBoneTransform.GetScale3D();
		const FVector RefScale = InRefPoseBoneTransform.GetScale3D();
		const double ScaleDiff = FMath::Max3(CurrentScale[0], CurrentScale[1], CurrentScale[2]) - FMath::Max3(RefScale[0], RefScale[1], RefScale[2]);
		SourceValue = ScaleDiff;
	}
	else
	{
		const FVector ScaleDiff = InCurrentBoneTransform.GetScale3D() - InRefPoseBoneTransform.GetScale3D();
		SourceValue = ScaleDiff[(int32)(SourceComponent - EComponentType::ScaleX)];
	}

	// Determine the resulting value
	double FinalDriverValue = SourceValue;
	if (DrivingCurve != nullptr)
	{
		// Remap thru the curve if set
		FinalDriverValue = DrivingCurve->GetFloatValue(static_cast<float>(FinalDriverValue));
	}
	else
	{
		// Apply the fixed function remapping/clamping
		if (bUseRange)
		{
			const double ClampedAlpha = FMath::Clamp(FMath::GetRangePct(RangeMin, RangeMax, FinalDriverValue), 0.0, 1.0);
			FinalDriverValue = FMath::Lerp(RemappedMin, RemappedMax, ClampedAlpha);
		}

		FinalDriverValue *= Multiplier;
	}

	return FinalDriverValue;
}

bool FAnimNode_BoneDrivenController::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones)
{
	return SourceBone.IsValidToEvaluate(RequiredBones) && ( TargetBone.IsValidToEvaluate(RequiredBones) || DestinationMode != EDrivenDestinationMode::Bone );
}

#if WITH_EDITORONLY_DATA

void FAnimNode_BoneDrivenController::ConvertTargetComponentToBits()
{
	switch (TargetComponent_DEPRECATED)
	{
	case EComponentType::TranslationX:
		bAffectTargetTranslationX = true;
		break;
	case EComponentType::TranslationY:
		bAffectTargetTranslationY = true;
		break;
	case EComponentType::TranslationZ:
		bAffectTargetTranslationZ = true;
		break;
	case EComponentType::RotationX:
		bAffectTargetRotationX = true;
		break;
	case EComponentType::RotationY:
		bAffectTargetRotationY = true;
		break;
	case EComponentType::RotationZ:
		bAffectTargetRotationZ = true;
		break;
	case EComponentType::Scale:
		bAffectTargetScaleX = true;
		bAffectTargetScaleY = true;
		bAffectTargetScaleZ = true;
		break;
	}
}

#endif

void FAnimNode_BoneDrivenController::InitializeBoneReferences(const FBoneContainer& RequiredBones)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	SourceBone.Initialize(RequiredBones);
	TargetBone.Initialize(RequiredBones);
}

