// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_CCDIK.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimStats.h"
#include "AnimationRuntime.h"
#include "DrawDebugHelpers.h"
#include "EngineDefines.h"
#include "Animation/AnimInstanceProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_CCDIK)

/////////////////////////////////////////////////////
// AnimNode_CCDIK
// Implementation of the CCDIK IK Algorithm

FAnimNode_CCDIK::FAnimNode_CCDIK()
	: EffectorLocation(FVector::ZeroVector)
	, EffectorLocationSpace(BCS_ComponentSpace)
	, Precision(1.f)
	, MaxIterations(10)
	, bStartFromTail(true)
	, bEnableRotationLimit(false)
{
}

FVector FAnimNode_CCDIK::GetCurrentLocation(FCSPose<FCompactPose>& MeshBases, const FCompactPoseBoneIndex& BoneIndex)
{
	return MeshBases.GetComponentSpaceTransform(BoneIndex).GetLocation();
}

FTransform FAnimNode_CCDIK::GetTargetTransform(const FTransform& InComponentTransform, FCSPose<FCompactPose>& MeshBases, FBoneSocketTarget& InTarget, EBoneControlSpace Space, const FVector& InOffset)
{
	FTransform OutTransform;
	if (Space == BCS_BoneSpace)
	{
		OutTransform = InTarget.GetTargetTransform(InOffset, MeshBases, InComponentTransform);
	}
	else
	{
		// parent bone space still goes through this way
		// if your target is socket, it will try find parents of joint that socket belongs to
		OutTransform.SetLocation(InOffset);
		FAnimationRuntime::ConvertBoneSpaceTransformToCS(InComponentTransform, MeshBases, OutTransform, InTarget.GetCompactPoseBoneIndex(), Space);
	}

	return OutTransform;
}

void FAnimNode_CCDIK::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(CCDIK, !IsInGameThread());

	const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();

	// Update EffectorLocation if it is based off a bone position
	FTransform CSEffectorTransform = GetTargetTransform(Output.AnimInstanceProxy->GetComponentTransform(), Output.Pose, EffectorTarget, EffectorLocationSpace, EffectorLocation);
	FVector const CSEffectorLocation = CSEffectorTransform.GetLocation();

	// Gather all bone indices between root and tip.
	TArray<FCompactPoseBoneIndex> BoneIndices;
	
	{
		const FCompactPoseBoneIndex RootIndex = RootBone.GetCompactPoseIndex(BoneContainer);
		FCompactPoseBoneIndex BoneIndex = TipBone.GetCompactPoseIndex(BoneContainer);
		do
		{
			BoneIndices.Insert(BoneIndex, 0);
			BoneIndex = Output.Pose.GetPose().GetParentBoneIndex(BoneIndex);
		} while (BoneIndex != RootIndex);
		BoneIndices.Insert(BoneIndex, 0);
	}

	// Gather transforms
	int32 const NumTransforms = BoneIndices.Num();
	OutBoneTransforms.AddUninitialized(NumTransforms);

	// Gather chain links. These are non zero length bones.
	TArray<FCCDIKChainLink> Chain;
	Chain.Reserve(NumTransforms);
	// Start with Root Bone
	{
		const FCompactPoseBoneIndex& RootBoneIndex = BoneIndices[0];
		const FTransform& LocalTransform = Output.Pose.GetLocalSpaceTransform(RootBoneIndex);
		const FTransform& BoneCSTransform = Output.Pose.GetComponentSpaceTransform(RootBoneIndex);

		OutBoneTransforms[0] = FBoneTransform(RootBoneIndex, BoneCSTransform);
		Chain.Add(FCCDIKChainLink(BoneCSTransform, LocalTransform, 0));
	}

	// Go through remaining transforms
	for (int32 TransformIndex = 1; TransformIndex < NumTransforms; TransformIndex++)
	{
		const FCompactPoseBoneIndex& BoneIndex = BoneIndices[TransformIndex];

		const FTransform& LocalTransform = Output.Pose.GetLocalSpaceTransform(BoneIndex);
		const FTransform& BoneCSTransform = Output.Pose.GetComponentSpaceTransform(BoneIndex);
		FVector const BoneCSPosition = BoneCSTransform.GetLocation();

		OutBoneTransforms[TransformIndex] = FBoneTransform(BoneIndex, BoneCSTransform);

		// Calculate the combined length of this segment of skeleton
		double const BoneLength = FVector::Dist(BoneCSPosition, OutBoneTransforms[TransformIndex - 1].Transform.GetLocation());

		if (!FMath::IsNearlyZero(BoneLength))
		{
			Chain.Add(FCCDIKChainLink(BoneCSTransform, LocalTransform, TransformIndex));
		}
		else
		{
			// Mark this transform as a zero length child of the last link.
			// It will inherit position and delta rotation from parent link.
			FCCDIKChainLink & ParentLink = Chain[Chain.Num() - 1];
			ParentLink.ChildZeroLengthTransformIndices.Add(TransformIndex);
		}
	}

	// solve
	bool bBoneLocationUpdated = AnimationCore::SolveCCDIK(Chain, CSEffectorLocation, Precision, MaxIterations, bStartFromTail, bEnableRotationLimit, RotationLimitPerJoints);

	// If we moved some bones, update bone transforms.
	if (bBoneLocationUpdated)
	{
		int32 NumChainLinks = Chain.Num();

		// First step: update bone transform positions from chain links.
		for (int32 LinkIndex = 0; LinkIndex < NumChainLinks; LinkIndex++)
		{
			FCCDIKChainLink const & ChainLink = Chain[LinkIndex];
			OutBoneTransforms[ChainLink.TransformIndex].Transform = ChainLink.Transform;

			// If there are any zero length children, update position of those
			int32 const NumChildren = ChainLink.ChildZeroLengthTransformIndices.Num();
			for (int32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
			{
				OutBoneTransforms[ChainLink.ChildZeroLengthTransformIndices[ChildIndex]].Transform = ChainLink.Transform;
			}
		}

#if WITH_EDITOR && UE_ENABLE_DEBUG_DRAWING
		DebugLines.Reset(OutBoneTransforms.Num());
		DebugLines.AddUninitialized(OutBoneTransforms.Num());
		for (int32 Index = 0; Index < OutBoneTransforms.Num(); ++Index)
		{
			DebugLines[Index] = OutBoneTransforms[Index].Transform.GetLocation();
		}
#endif // WITH_EDITOR

	}
}

bool FAnimNode_CCDIK::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones)
{
	if (EffectorLocationSpace == BCS_ParentBoneSpace || EffectorLocationSpace == BCS_BoneSpace)
	{
		if (!EffectorTarget.IsValidToEvaluate(RequiredBones))
		{
			return false;
		}
	}

	// Allow evaluation if all parameters are initialized and TipBone is child of RootBone
	return
		(
		TipBone.IsValidToEvaluate(RequiredBones)
		&& RootBone.IsValidToEvaluate(RequiredBones)
		&& Precision > 0
		&& RequiredBones.BoneIsChildOf(TipBone.BoneIndex, RootBone.BoneIndex)
		);
}

#if WITH_EDITOR
void FAnimNode_CCDIK::ResizeRotationLimitPerJoints(int32 NewSize)
{
	if (NewSize == 0)
	{
		RotationLimitPerJoints.Reset();
	}
	else if (RotationLimitPerJoints.Num() != NewSize)
	{
		int32 StartIndex = RotationLimitPerJoints.Num();
		RotationLimitPerJoints.SetNum(NewSize);
		for (int32 Index = StartIndex; Index < RotationLimitPerJoints.Num(); ++Index)
		{
			RotationLimitPerJoints[Index] = 30.f;
		}
	}
}
#endif 

void FAnimNode_CCDIK::InitializeBoneReferences(const FBoneContainer& RequiredBones)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	TipBone.Initialize(RequiredBones);
	RootBone.Initialize(RequiredBones);
	EffectorTarget.InitializeBoneReferences(RequiredBones);
}

void FAnimNode_CCDIK::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	DebugData.AddDebugItem(DebugLine);
	ComponentPose.GatherDebugData(DebugData);
}


