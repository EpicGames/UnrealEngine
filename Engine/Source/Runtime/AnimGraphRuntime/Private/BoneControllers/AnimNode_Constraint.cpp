// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_Constraint.h"
#include "AnimationCoreLibrary.h"
#include "AnimationRuntime.h"
#include "Animation/AnimStats.h"
#include "PrimitiveDrawingUtils.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_Constraint)

/////////////////////////////////////////////////////
// FAnimNode_Constraint

FAnimNode_Constraint::FAnimNode_Constraint()
{
}

void FAnimNode_Constraint::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	const float ActualBiasedAlpha = AlphaScaleBias.ApplyTo(Alpha);

	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("(Alpha: %.1f%%)"), ActualBiasedAlpha*100.f);
	DebugData.AddDebugItem(DebugLine);

	const int32 ConstraintNum = ConstraintSetup.Num();
	for (int32 ConstraintIndex = 0; ConstraintIndex < ConstraintNum; ++ConstraintIndex)
	{
		const FConstraint& Constraint = ConstraintSetup[ConstraintIndex];
		const float Weight = ConstraintWeights.IsValidIndex(ConstraintIndex) ? ConstraintWeights[ConstraintIndex] : 0.0f;
		DebugLine = FString::Printf(TEXT("  Target : %s (%0.2f) "), *Constraint.TargetBone.BoneName.ToString(), Weight);
		DebugData.AddDebugItem(DebugLine);
	}

	ComponentPose.GatherDebugData(DebugData);
}

void FAnimNode_Constraint::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(Constraint, !IsInGameThread());

 	const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();
	const int32 ConstraintNum = ConstraintSetup.Num();
#if WITH_EDITOR
	CachedTargetTransforms.Empty();
#endif // WITH_EDITOR
	for (int32 ConstraintIndex = 0; ConstraintIndex < ConstraintNum; ++ConstraintIndex)
	{
		const FConstraint& Constraint = ConstraintSetup[ConstraintIndex];
		const float Weight = ConstraintWeights.IsValidIndex(ConstraintIndex) ? ConstraintWeights[ConstraintIndex] : 0.0f;

		if (Constraint.IsValidToEvaluate(BoneContainer) && Constraint.ConstraintDataIndex != INDEX_NONE)
		{
			ConstraintData[Constraint.ConstraintDataIndex].Weight = Weight;
			ConstraintData[Constraint.ConstraintDataIndex].CurrentTransform = Output.Pose.GetComponentSpaceTransform(Constraint.TargetBone.CachedCompactPoseIndex);

#if WITH_EDITOR
			CachedTargetTransforms.Add(ConstraintData[Constraint.ConstraintDataIndex].CurrentTransform);
#endif // WITH_EDITOR
		}
	}

	if (ConstraintData.Num() > 0)
	{
		FTransform SourceTransform = Output.Pose.GetComponentSpaceTransform(BoneToModify.CachedCompactPoseIndex);
		// for constraint anim node, we always use identity as base		
		FTransform ConstrainedTransform = AnimationCore::SolveConstraints(SourceTransform, FTransform::Identity, ConstraintData);
		OutBoneTransforms.Add(FBoneTransform(BoneToModify.CachedCompactPoseIndex, ConstrainedTransform));

#if WITH_EDITOR
		CachedOriginalTransform = SourceTransform;
		CachedConstrainedTransform = ConstrainedTransform;
#endif // WITH_EDITOR
	}

#if ANIM_TRACE_ENABLED
	for (int32 ConstraintIndex = 0; ConstraintIndex < ConstraintNum; ++ConstraintIndex)
	{
		TRACE_ANIM_NODE_VALUE(Output, *ConstraintSetup[ConstraintIndex].TargetBone.BoneName.ToString(), ConstraintWeights.IsValidIndex(ConstraintIndex) ? ConstraintWeights[ConstraintIndex] : 0.0f);
	}
#endif
}

bool FAnimNode_Constraint::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) 
{
	// if any of them are valid
	bool bHaveValidConstraint = false;

	for (int32 ConstraintIndex = 0; ConstraintIndex<ConstraintSetup.Num() ; ++ConstraintIndex)
	{
		FConstraint& Constraint = ConstraintSetup[ConstraintIndex];
		// make sure it has weight
		if (ConstraintWeights.IsValidIndex(ConstraintIndex) && ConstraintWeights[ConstraintIndex] > ZERO_ANIMWEIGHT_THRESH)
		{
			bHaveValidConstraint |= Constraint.IsValidToEvaluate(RequiredBones);
		}
	}

	return (bHaveValidConstraint && BoneToModify.IsValidToEvaluate(RequiredBones));
}

void FAnimNode_Constraint::InitializeBoneReferences(const FBoneContainer& RequiredBones) 
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	BoneToModify.Initialize(RequiredBones);

	ConstraintData.Reset(ConstraintSetup.Num());

	const int32 ConstraintNum = ConstraintSetup.Num();

	if (BoneToModify.IsValidToEvaluate(RequiredBones))
	{
		FTransform SourceTransform = FAnimationRuntime::GetComponentSpaceRefPose(BoneToModify.CachedCompactPoseIndex, RequiredBones);
		
		for (int32 ConstraintIndex = 0; ConstraintIndex < ConstraintNum; ++ConstraintIndex)
		{
			FConstraint& Constraint = ConstraintSetup[ConstraintIndex];
			Constraint.Initialize(RequiredBones);

			if (Constraint.TargetBone.IsValidToEvaluate(RequiredBones))
			{
				FConstraintData NewConstraintData(FTransformConstraintDescription(Constraint.TransformType), Constraint.TargetBone.BoneName, 0.f, Constraint.OffsetOption != EConstraintOffsetOption::None);
				// copy the axes filter options, later figure out cleaner way to do this (constructor)
				NewConstraintData.Constraint.ConstraintDescription->AxesFilterOption = Constraint.PerAxis;

				FTransform TargetTransform = (NewConstraintData.bMaintainOffset) ? FAnimationRuntime::GetComponentSpaceRefPose(Constraint.TargetBone.CachedCompactPoseIndex, RequiredBones) : FTransform::Identity;
				
				// for constraint anim node, we always use identity as base	
				NewConstraintData.SaveInverseOffset(SourceTransform, TargetTransform, FTransform::Identity);

				Constraint.ConstraintDataIndex = ConstraintData.Add(NewConstraintData);
			}
			else
			{
				Constraint.ConstraintDataIndex = INDEX_NONE;
			}
		}
	}
}

#if WITH_EDITOR
// can't use World Draw functions because this is called from Render of viewport, AFTER ticking component, 
// which means LineBatcher already has ticked, so it won't render anymore
// to use World Draw functions, we have to call this from tick of actor
void FAnimNode_Constraint::ConditionalDebugDraw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* MeshComp) const
{
	if (PDI && MeshComp)
	{
		// draw my transform
		FTransform LocalToWorld = MeshComp->GetComponentTransform();
		FTransform OriginalTransform = CachedOriginalTransform * LocalToWorld;
		FTransform ConstrainedTransform = CachedConstrainedTransform * LocalToWorld;

		DrawCoordinateSystem(PDI, OriginalTransform.GetLocation(), OriginalTransform.GetRotation().Rotator(), 20.f, SDPG_Foreground);
		DrawCoordinateSystem(PDI, ConstrainedTransform.GetLocation(), ConstrainedTransform.GetRotation().Rotator(), 20.f, SDPG_Foreground);

		// draw my target's transform for all targets
		FVector SourceLocation = ConstrainedTransform.GetLocation();
		for (int32 Index = 0; Index < CachedTargetTransforms.Num(); ++Index)
		{
			FTransform TargetTrasnform = CachedTargetTransforms[Index] * LocalToWorld;
			DrawDashedLine(PDI, SourceLocation, TargetTrasnform.GetLocation(), FColor::Yellow, 5.f, SDPG_World);
			DrawCoordinateSystem(PDI, TargetTrasnform.GetLocation(), TargetTrasnform.GetRotation().Rotator(), 20.f, SDPG_Foreground);
		}
	}
}
#endif // WITH_EDITOR

