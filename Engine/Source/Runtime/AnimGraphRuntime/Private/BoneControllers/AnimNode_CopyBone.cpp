// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_CopyBone.h"
#include "AnimationRuntime.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_CopyBone)

/////////////////////////////////////////////////////
// FAnimNode_CopyBone

FAnimNode_CopyBone::FAnimNode_CopyBone()
	: bCopyTranslation(false)
	, bCopyRotation(false)
	, bCopyScale(false)
	, ControlSpace(BCS_ComponentSpace)
{
}

void FAnimNode_CopyBone::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);
	
	DebugLine += "(";
	AddDebugNodeData(DebugLine);
	DebugLine += FString::Printf(TEXT(" Src: %s Dst: %s)"), *SourceBone.BoneName.ToString(), *TargetBone.BoneName.ToString());
	DebugData.AddDebugItem(DebugLine);

	ComponentPose.GatherDebugData(DebugData);
}

void FAnimNode_CopyBone::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(CopyBone, !IsInGameThread());

	check(OutBoneTransforms.Num() == 0);

	// Pass through if we're not doing anything.
	if( !bCopyTranslation && !bCopyRotation && !bCopyScale )
	{
		return;
	}

	// Get component space transform for source and current bone.
	const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();
	FCompactPoseBoneIndex SourceBoneIndex = SourceBone.GetCompactPoseIndex(BoneContainer);
	FCompactPoseBoneIndex TargetBoneIndex = TargetBone.GetCompactPoseIndex(BoneContainer);

	FTransform SourceBoneTM = Output.Pose.GetComponentSpaceTransform(SourceBoneIndex);
	FTransform CurrentBoneTM = Output.Pose.GetComponentSpaceTransform(TargetBoneIndex);

	if(ControlSpace != BCS_ComponentSpace)
	{
		// Convert out to selected space
		FAnimationRuntime::ConvertCSTransformToBoneSpace(Output.AnimInstanceProxy->GetComponentTransform(), Output.Pose, SourceBoneTM, SourceBoneIndex, ControlSpace);
		FAnimationRuntime::ConvertCSTransformToBoneSpace(Output.AnimInstanceProxy->GetComponentTransform(), Output.Pose, CurrentBoneTM, TargetBoneIndex, ControlSpace);
	}
	
	// Copy individual components
	if (bCopyTranslation)
	{
		CurrentBoneTM.SetTranslation( SourceBoneTM.GetTranslation() );
	}

	if (bCopyRotation)
	{
		CurrentBoneTM.SetRotation( SourceBoneTM.GetRotation() );
	}

	if (bCopyScale)
	{
		CurrentBoneTM.SetScale3D( SourceBoneTM.GetScale3D() );
	}

	if(ControlSpace != BCS_ComponentSpace)
	{
		// Convert back out if we aren't operating in component space
		FAnimationRuntime::ConvertBoneSpaceTransformToCS(Output.AnimInstanceProxy->GetComponentTransform(), Output.Pose, CurrentBoneTM, TargetBoneIndex, ControlSpace);
	}

	// Output new transform for current bone.
	OutBoneTransforms.Add(FBoneTransform(TargetBoneIndex, CurrentBoneTM));

	TRACE_ANIM_NODE_VALUE(Output, TEXT("Source Bone"), SourceBone.BoneName);
	TRACE_ANIM_NODE_VALUE(Output, TEXT("Target Bone"), TargetBone.BoneName);
}

bool FAnimNode_CopyBone::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) 
{
	// if both bones are valid
	return (TargetBone.IsValidToEvaluate(RequiredBones) && (TargetBone==SourceBone || SourceBone.IsValidToEvaluate(RequiredBones)));
}

void FAnimNode_CopyBone::InitializeBoneReferences(const FBoneContainer& RequiredBones) 
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	SourceBone.Initialize(RequiredBones);
	TargetBone.Initialize(RequiredBones);
}

