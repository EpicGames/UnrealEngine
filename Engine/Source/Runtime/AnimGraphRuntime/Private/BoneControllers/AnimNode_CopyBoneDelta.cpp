// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_CopyBoneDelta.h"
#include "Components/SkeletalMeshComponent.h"
#include "AnimationRuntime.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimStats.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_CopyBoneDelta)

FAnimNode_CopyBoneDelta::FAnimNode_CopyBoneDelta()
	: bCopyTranslation(false)
	, bCopyRotation(false)
	, bCopyScale(false)
	, CopyMode(CopyBoneDeltaMode::Accumulate)
	, TranslationMultiplier(1.0f)
	, RotationMultiplier(1.0f)
	, ScaleMultiplier(1.0f)
{

}

void FAnimNode_CopyBoneDelta::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	DebugData.AddDebugItem(DebugLine);

	ComponentPose.GatherDebugData(DebugData);
}

void FAnimNode_CopyBoneDelta::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	if(!bCopyTranslation && !bCopyRotation && !bCopyScale)
	{
		return;
	}

	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(CopyBoneDelta, !IsInGameThread());

	const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();
	FCompactPoseBoneIndex SourceBoneIndex = SourceBone.GetCompactPoseIndex(BoneContainer);
	FCompactPoseBoneIndex TargetBoneIndex = TargetBone.GetCompactPoseIndex(BoneContainer);

	FTransform SourceTM = Output.Pose.GetComponentSpaceTransform(SourceBoneIndex);
	FTransform TargetTM = Output.Pose.GetComponentSpaceTransform(TargetBoneIndex);

	// Convert to parent space
	FAnimationRuntime::ConvertCSTransformToBoneSpace(Output.AnimInstanceProxy->GetComponentTransform(), Output.Pose, SourceTM, SourceBoneIndex, BCS_ParentBoneSpace);
	FAnimationRuntime::ConvertCSTransformToBoneSpace(Output.AnimInstanceProxy->GetComponentTransform(), Output.Pose, TargetTM, TargetBoneIndex, BCS_ParentBoneSpace);

	// Ref pose transform
	FTransform RefLSTransform = BoneContainer.GetReferenceSkeleton().GetRefBonePose()[SourceBone.GetMeshPoseIndex(BoneContainer).GetInt()];

	// Get transform relative to ref pose
	SourceTM.SetToRelativeTransform(RefLSTransform);

	if(CopyMode == CopyBoneDeltaMode::Accumulate)
	{
		if(bCopyTranslation)
		{
			TargetTM.AddToTranslation(SourceTM.GetTranslation() * TranslationMultiplier);
		}
		if(bCopyRotation)
		{
			FVector Axis;
			float Angle;
			SourceTM.GetRotation().ToAxisAndAngle(Axis, Angle);

			TargetTM.SetRotation(FQuat(Axis, Angle * RotationMultiplier) * TargetTM.GetRotation());
		}
		if(bCopyScale)
		{
			TargetTM.SetScale3D(TargetTM.GetScale3D() * (SourceTM.GetScale3D() * ScaleMultiplier));
		}
	}
	else //CopyMode = CopyBoneDeltaMode::Copy
	{
		if(bCopyTranslation)
		{
			TargetTM.SetTranslation(SourceTM.GetTranslation() * TranslationMultiplier);
		}

		if(bCopyRotation)
		{
			FVector Axis;
			float Angle;
			SourceTM.GetRotation().ToAxisAndAngle(Axis, Angle);

			TargetTM.SetRotation(FQuat(Axis, Angle * RotationMultiplier));
		}

		if(bCopyScale)
		{
			TargetTM.SetScale3D(SourceTM.GetScale3D() * ScaleMultiplier);
		}
	}

	// Back out to component space
	FAnimationRuntime::ConvertBoneSpaceTransformToCS(Output.AnimInstanceProxy->GetComponentTransform(), Output.Pose, TargetTM, TargetBoneIndex, BCS_ParentBoneSpace);

	OutBoneTransforms.Add(FBoneTransform(TargetBoneIndex, TargetTM));
}

bool FAnimNode_CopyBoneDelta::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones)
{
	return (TargetBone.IsValidToEvaluate(RequiredBones) && (TargetBone == SourceBone || SourceBone.IsValidToEvaluate(RequiredBones)));
}

void FAnimNode_CopyBoneDelta::InitializeBoneReferences(const FBoneContainer& RequiredBones)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	SourceBone.Initialize(RequiredBones);
	TargetBone.Initialize(RequiredBones);
}

