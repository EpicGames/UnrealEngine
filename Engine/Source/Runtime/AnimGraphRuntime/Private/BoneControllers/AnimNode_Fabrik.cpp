// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_Fabrik.h"
#include "AnimationRuntime.h"
#include "Components/SkeletalMeshComponent.h"
#include "DrawDebugHelpers.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimStats.h"
#include "FABRIK.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_Fabrik)

/////////////////////////////////////////////////////
// AnimNode_Fabrik
// Implementation of the FABRIK IK Algorithm
// Please see http://www.academia.edu/9165835/FABRIK_A_fast_iterative_solver_for_the_Inverse_Kinematics_problem for more details

FAnimNode_Fabrik::FAnimNode_Fabrik()
	: EffectorTransform(FTransform::Identity)
	, Precision(1.f)
	, MaxIterations(10)
	, EffectorTransformSpace(BCS_ComponentSpace)
	, EffectorRotationSource(BRS_KeepLocalSpaceRotation)
#if WITH_EDITORONLY_DATA
	, bEnableDebugDraw(false)
#endif
{
}

FVector FAnimNode_Fabrik::GetCurrentLocation(FCSPose<FCompactPose>& MeshBases, const FCompactPoseBoneIndex& BoneIndex)
{
	return MeshBases.GetComponentSpaceTransform(BoneIndex).GetLocation();
}

FTransform FAnimNode_Fabrik::GetTargetTransform(const FTransform& InComponentTransform, FCSPose<FCompactPose>& MeshBases, FBoneSocketTarget& InTarget, EBoneControlSpace Space, const FTransform& InOffset) 
{
	FTransform OutTransform;
	if (Space == BCS_BoneSpace)
	{
		OutTransform = InTarget.GetTargetTransform(InOffset, MeshBases, InComponentTransform);
	}
	else
	{
		// parent bone space still goes through this way
		// if your target is socket, it will try find parents of joint that socket belongs to
		OutTransform = InOffset;
		FAnimationRuntime::ConvertBoneSpaceTransformToCS(InComponentTransform, MeshBases, OutTransform, InTarget.GetCompactPoseBoneIndex(), Space);
	}

	return OutTransform;
}

void FAnimNode_Fabrik::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(Fabrik, !IsInGameThread());

	const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();

	// Update EffectorLocation if it is based off a bone position
	FTransform CSEffectorTransform = EffectorTransform;
	CSEffectorTransform = GetTargetTransform(Output.AnimInstanceProxy->GetComponentTransform(), Output.Pose, EffectorTarget, EffectorTransformSpace, EffectorTransform);

	FVector const CSEffectorLocation = CSEffectorTransform.GetLocation();

#if WITH_EDITOR
	CachedEffectorCSTransform = CSEffectorTransform;
#endif

	// Gather all bone indices between root and tip.
	TArray<FCompactPoseBoneIndex> BoneIndices;
	
	{
		const FCompactPoseBoneIndex RootIndex = RootBone.GetCompactPoseIndex(BoneContainer);
		FCompactPoseBoneIndex BoneIndex = TipBone.GetCompactPoseIndex(BoneContainer);
		do
		{
			BoneIndices.Insert(BoneIndex, 0);
			BoneIndex = Output.Pose.GetPose().GetParentBoneIndex(BoneIndex);
		} while (BoneIndex != RootIndex);
		BoneIndices.Insert(BoneIndex, 0);
	}

	// Maximum length of skeleton segment at full extension
	double MaximumReach = 0;

	// Gather transforms
	int32 const NumTransforms = BoneIndices.Num();
	OutBoneTransforms.AddUninitialized(NumTransforms);

	// Gather chain links. These are non zero length bones.
	TArray<FFABRIKChainLink> Chain;
	Chain.Reserve(NumTransforms);

	// Start with Root Bone
	{
		const FCompactPoseBoneIndex& RootBoneIndex = BoneIndices[0];
		const FTransform& BoneCSTransform = Output.Pose.GetComponentSpaceTransform(RootBoneIndex);

		OutBoneTransforms[0] = FBoneTransform(RootBoneIndex, BoneCSTransform);
		Chain.Add(FFABRIKChainLink(BoneCSTransform.GetLocation(), 0.f, RootBoneIndex, 0));
	}

	// Go through remaining transforms
	for (int32 TransformIndex = 1; TransformIndex < NumTransforms; TransformIndex++)
	{
		const FCompactPoseBoneIndex& BoneIndex = BoneIndices[TransformIndex];

		const FTransform& BoneCSTransform = Output.Pose.GetComponentSpaceTransform(BoneIndex);
		FVector const BoneCSPosition = BoneCSTransform.GetLocation();

		OutBoneTransforms[TransformIndex] = FBoneTransform(BoneIndex, BoneCSTransform);

		// Calculate the combined length of this segment of skeleton
		double const BoneLength = FVector::Dist(BoneCSPosition, OutBoneTransforms[TransformIndex-1].Transform.GetLocation());

		if (!FMath::IsNearlyZero(BoneLength))
		{
			Chain.Add(FFABRIKChainLink(BoneCSPosition, BoneLength, BoneIndex, TransformIndex));
			MaximumReach += BoneLength;
		}
		else
		{
			// Mark this transform as a zero length child of the last link.
			// It will inherit position and delta rotation from parent link.
			FFABRIKChainLink & ParentLink = Chain[Chain.Num()-1];
			ParentLink.ChildZeroLengthTransformIndices.Add(TransformIndex);
		}
	}

	int32 const NumChainLinks = Chain.Num();
	bool bBoneLocationUpdated = AnimationCore::SolveFabrik(Chain, CSEffectorLocation, MaximumReach, Precision, MaxIterations);
	// If we moved some bones, update bone transforms.
	if (bBoneLocationUpdated)
	{
		// First step: update bone transform positions from chain links.
		for (int32 LinkIndex = 0; LinkIndex < NumChainLinks; LinkIndex++)
		{
			FFABRIKChainLink const & ChainLink = Chain[LinkIndex];
			OutBoneTransforms[ChainLink.TransformIndex].Transform.SetTranslation(ChainLink.Position);

			// If there are any zero length children, update position of those
			int32 const NumChildren = ChainLink.ChildZeroLengthTransformIndices.Num();
			for (int32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
			{
				OutBoneTransforms[ChainLink.ChildZeroLengthTransformIndices[ChildIndex]].Transform.SetTranslation(ChainLink.Position);
			}
		}

		// FABRIK algorithm - re-orientation of bone local axes after translation calculation
		for (int32 LinkIndex = 0; LinkIndex < NumChainLinks - 1; LinkIndex++)
		{
			FFABRIKChainLink const & CurrentLink = Chain[LinkIndex];
			FFABRIKChainLink const & ChildLink = Chain[LinkIndex + 1];

			// Calculate pre-translation vector between this bone and child
			FVector const OldDir = (GetCurrentLocation(Output.Pose, FCompactPoseBoneIndex(ChildLink.BoneIndex)) - GetCurrentLocation(Output.Pose, FCompactPoseBoneIndex(CurrentLink.BoneIndex))).GetUnsafeNormal();

			// Get vector from the post-translation bone to it's child
			FVector const NewDir = (ChildLink.Position - CurrentLink.Position).GetUnsafeNormal();

			// Calculate axis of rotation from pre-translation vector to post-translation vector
			FVector const RotationAxis = FVector::CrossProduct(OldDir, NewDir).GetSafeNormal();
			double const RotationAngle = FMath::Acos(FVector::DotProduct(OldDir, NewDir));
			FQuat const DeltaRotation = FQuat(RotationAxis, RotationAngle);
			// We're going to multiply it, in order to not have to re-normalize the final quaternion, it has to be a unit quaternion.
			checkSlow(DeltaRotation.IsNormalized());

			// Calculate absolute rotation and set it
			FTransform& CurrentBoneTransform = OutBoneTransforms[CurrentLink.TransformIndex].Transform;
			CurrentBoneTransform.SetRotation(DeltaRotation * CurrentBoneTransform.GetRotation());
			CurrentBoneTransform.NormalizeRotation();

			// Update zero length children if any
			int32 const NumChildren = CurrentLink.ChildZeroLengthTransformIndices.Num();
			for (int32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
			{
				FTransform& ChildBoneTransform = OutBoneTransforms[CurrentLink.ChildZeroLengthTransformIndices[ChildIndex]].Transform;
				ChildBoneTransform.SetRotation(DeltaRotation * ChildBoneTransform.GetRotation());
				ChildBoneTransform.NormalizeRotation();
			}
		}
	}

	// Special handling for tip bone's rotation.
	int32 const TipBoneTransformIndex = OutBoneTransforms.Num() - 1;
	switch (EffectorRotationSource)
	{
	case BRS_KeepLocalSpaceRotation:
		OutBoneTransforms[TipBoneTransformIndex].Transform = Output.Pose.GetLocalSpaceTransform(BoneIndices[TipBoneTransformIndex]) * OutBoneTransforms[TipBoneTransformIndex - 1].Transform;
		break;
	case BRS_CopyFromTarget:
		OutBoneTransforms[TipBoneTransformIndex].Transform.SetRotation(CSEffectorTransform.GetRotation());
		break;
	case BRS_KeepComponentSpaceRotation:
		// Don't change the orientation at all
		break;
	default:
		break;
	}
}

bool FAnimNode_Fabrik::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones)
{
	// Allow evaluation if all parameters are initialized and TipBone is child of RootBone
	return
		(
		TipBone.IsValidToEvaluate(RequiredBones)
		&& RootBone.IsValidToEvaluate(RequiredBones)
		&& Precision > 0
		&& RequiredBones.BoneIsChildOf(TipBone.BoneIndex, RootBone.BoneIndex)
		);
}

void FAnimNode_Fabrik::ConditionalDebugDraw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* PreviewSkelMeshComp) const
{
#if WITH_EDITORONLY_DATA

	if(bEnableDebugDraw && PreviewSkelMeshComp && PreviewSkelMeshComp->GetWorld())
	{
		FVector const CSEffectorLocation = CachedEffectorCSTransform.GetLocation();

		// Show end effector position.
		DrawDebugBox(PreviewSkelMeshComp->GetWorld(), CSEffectorLocation, FVector(Precision), FColor::Green, true, 0.1f);
		DrawDebugCoordinateSystem(PreviewSkelMeshComp->GetWorld(), CSEffectorLocation, CachedEffectorCSTransform.GetRotation().Rotator(), 5.f, true, 0.1f);
	}
#endif
}

void FAnimNode_Fabrik::InitializeBoneReferences(const FBoneContainer& RequiredBones)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	TipBone.Initialize(RequiredBones);
	RootBone.Initialize(RequiredBones);
	EffectorTarget.InitializeBoneReferences(RequiredBones);
}

void FAnimNode_Fabrik::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	DebugData.AddDebugItem(DebugLine);
	ComponentPose.GatherDebugData(DebugData);
}

void FAnimNode_Fabrik::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	Super::Initialize_AnyThread(Context);
	EffectorTarget.Initialize(Context.AnimInstanceProxy);
}

