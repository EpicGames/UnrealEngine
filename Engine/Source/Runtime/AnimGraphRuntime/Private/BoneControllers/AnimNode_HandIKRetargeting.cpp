// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_HandIKRetargeting.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimTrace.h"
#if ENABLE_ANIM_DEBUG
#include "HAL/IConsoleManager.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_HandIKRetargeting)

#if ENABLE_ANIM_DEBUG
static TAutoConsoleVariable<bool> CVarAnimHandIKRetargetingEnable(TEXT("a.AnimNode.HandIKRetargeting.Enable"), true, TEXT("Enable / Disable Hand IK retargeting"));
#endif

/////////////////////////////////////////////////////
// FAnimNode_HandIKRetargeting

FAnimNode_HandIKRetargeting::FAnimNode_HandIKRetargeting()
: PerAxisAlpha(1.f,1.f,1.f)
, HandFKWeight(0.5f)
{
}

void FAnimNode_HandIKRetargeting::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	DebugLine += "(";
	AddDebugNodeData(DebugLine);
	DebugLine += FString::Printf(TEXT(" HandFKWeight: %f)"), HandFKWeight);
	for (int32 BoneIndex = 0; BoneIndex < IKBonesToMove.Num(); BoneIndex++)
	{
		DebugLine += FString::Printf(TEXT(", %s)"), *IKBonesToMove[BoneIndex].BoneName.ToString());
	}
	DebugLine += FString::Printf(TEXT(")"));
	DebugData.AddDebugItem(DebugLine);

	ComponentPose.GatherDebugData(DebugData);
}

void FAnimNode_HandIKRetargeting::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(HandIKRetargeting, !IsInGameThread());

#if ENABLE_ANIM_DEBUG
	if (!CVarAnimHandIKRetargetingEnable.GetValueOnAnyThread())
	{
		return;
	}
#endif 
	// Early exit if alpha is zero
	if (PerAxisAlpha.IsNearlyZero())
	{
		return;
	}

	checkSlow(OutBoneTransforms.Num() == 0);

	const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();

	FVector FKLocation = FVector::ZeroVector;
	FVector IKLocation = FVector::ZeroVector;

	if (FAnimWeight::IsFullWeight(HandFKWeight))
	{
		const FCompactPoseBoneIndex RightHandFKBoneIndex = RightHandFK.GetCompactPoseIndex(BoneContainer);
		const FCompactPoseBoneIndex RightHandIKBoneIndex = RightHandIK.GetCompactPoseIndex(BoneContainer);

		FKLocation = Output.Pose.GetComponentSpaceTransform(RightHandFKBoneIndex).GetTranslation();
		IKLocation = Output.Pose.GetComponentSpaceTransform(RightHandIKBoneIndex).GetTranslation();
	}
	else if (!FAnimWeight::IsRelevant(HandFKWeight))
	{
		const FCompactPoseBoneIndex LeftHandFKBoneIndex = LeftHandFK.GetCompactPoseIndex(BoneContainer);
		const FCompactPoseBoneIndex LeftHandIKBoneIndex = LeftHandIK.GetCompactPoseIndex(BoneContainer);

		FKLocation = Output.Pose.GetComponentSpaceTransform(LeftHandFKBoneIndex).GetTranslation();
		IKLocation = Output.Pose.GetComponentSpaceTransform(LeftHandIKBoneIndex).GetTranslation();
	}
	else
	{
		const FCompactPoseBoneIndex RightHandFKBoneIndex = RightHandFK.GetCompactPoseIndex(BoneContainer);
		const FCompactPoseBoneIndex RightHandIKBoneIndex = RightHandIK.GetCompactPoseIndex(BoneContainer);
		const FCompactPoseBoneIndex LeftHandFKBoneIndex = LeftHandFK.GetCompactPoseIndex(BoneContainer);
		const FCompactPoseBoneIndex LeftHandIKBoneIndex = LeftHandIK.GetCompactPoseIndex(BoneContainer);

		const FVector RightHandFKLocation = Output.Pose.GetComponentSpaceTransform(RightHandFKBoneIndex).GetTranslation();
		const FVector RightHandIKLocation = Output.Pose.GetComponentSpaceTransform(RightHandIKBoneIndex).GetTranslation();
		const FVector LeftHandFKLocation = Output.Pose.GetComponentSpaceTransform(LeftHandFKBoneIndex).GetTranslation();
		const FVector LeftHandIKLocation = Output.Pose.GetComponentSpaceTransform(LeftHandIKBoneIndex).GetTranslation();

		// Compute weight FK and IK hand location. And translation from IK to FK.
		FKLocation = FMath::Lerp<FVector>(LeftHandFKLocation, RightHandFKLocation, HandFKWeight);
		IKLocation = FMath::Lerp<FVector>(LeftHandIKLocation, RightHandIKLocation, HandFKWeight);
	}
	
	const FVector IK_To_FK_Translation = (FKLocation - IKLocation) * PerAxisAlpha;

	// If we're not translating, don't send any bones to update.
	if (!IK_To_FK_Translation.IsNearlyZero())
	{
		// Move desired bones
		for (const FBoneReference& BoneReference : IKBonesToMove)
		{
			if (BoneReference.IsValidToEvaluate(BoneContainer))
			{
				const FCompactPoseBoneIndex BoneIndex = BoneReference.GetCompactPoseIndex(BoneContainer);
				FTransform BoneTransform = Output.Pose.GetComponentSpaceTransform(BoneIndex);
				BoneTransform.AddToTranslation(IK_To_FK_Translation);

				OutBoneTransforms.Add(FBoneTransform(BoneIndex, BoneTransform));
			}
		}

		if (OutBoneTransforms.Num() > 0)
		{
			OutBoneTransforms.Sort(FCompareBoneTransformIndex());
		}
	}

#if ANIM_TRACE_ENABLED
	TRACE_ANIM_NODE_VALUE(Output, TEXT("Hand FK Weight"), HandFKWeight);
#endif
}

bool FAnimNode_HandIKRetargeting::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones)
{
	if (RightHandFK.IsValidToEvaluate(RequiredBones)
		&& LeftHandFK.IsValidToEvaluate(RequiredBones)
		&& RightHandIK.IsValidToEvaluate(RequiredBones)
		&& LeftHandIK.IsValidToEvaluate(RequiredBones))
	{
		// we need at least one bone to move valid.
		for (int32 BoneIndex = 0; BoneIndex < IKBonesToMove.Num(); BoneIndex++)
		{
			if (IKBonesToMove[BoneIndex].IsValidToEvaluate(RequiredBones))
			{
				return true;
			}
		}
	}

	return false;
}

void FAnimNode_HandIKRetargeting::InitializeBoneReferences(const FBoneContainer& RequiredBones)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	RightHandFK.Initialize(RequiredBones);
	LeftHandFK.Initialize(RequiredBones);
	RightHandIK.Initialize(RequiredBones);
	LeftHandIK.Initialize(RequiredBones);

	for (int32 BoneIndex = 0; BoneIndex < IKBonesToMove.Num(); BoneIndex++)
	{
		IKBonesToMove[BoneIndex].Initialize(RequiredBones);
	}
}

