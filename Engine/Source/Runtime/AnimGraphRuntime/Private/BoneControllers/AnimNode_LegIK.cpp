// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_LegIK.h"

#include "Components/SkeletalMeshComponent.h"
#include "DrawDebugHelpers.h"
#include "Engine/Engine.h"
#include "EngineGlobals.h"
#include "Animation/AnimInstanceProxy.h"
#include "SoftIK.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_LegIK)

#if ENABLE_ANIM_DEBUG
TAutoConsoleVariable<int32> CVarAnimNodeLegIKDebug(TEXT("a.AnimNode.LegIK.Debug"), 0, TEXT("Turn on debug for FAnimNode_LegIK"));
#endif

TAutoConsoleVariable<int32> CVarAnimLegIKEnable(TEXT("a.AnimNode.LegIK.Enable"), 1, TEXT("Toggle LegIK node."));
TAutoConsoleVariable<int32> CVarAnimLegIKMaxIterations(TEXT("a.AnimNode.LegIK.MaxIterations"), 0, TEXT("Leg IK MaxIterations override. 0 = node default, > 0 override."));
TAutoConsoleVariable<float> CVarAnimLegIKTargetReachStepPercent(TEXT("a.AnimNode.LegIK.TargetReachStepPercent"), 0.7f, TEXT("Leg IK TargetReachStepPercent."));
TAutoConsoleVariable<float> CVarAnimLegIKPullDistribution(TEXT("a.AnimNode.LegIK.PullDistribution"), 0.5f, TEXT("Leg IK PullDistribution. 0 = foot, 0.5 = balanced, 1.f = hip"));
TAutoConsoleVariable<int32> CVarAnimLegIKForceAlwaysSolve(TEXT("a.AnimNode.LegIK.ForceAlwaysSolve"), 0, TEXT("Leg IK Always Run IK Solver. 0 = default behavior, 1 = Run IK Solver every frame."));

/////////////////////////////////////////////////////
// FAnimAnimNode_LegIK

DECLARE_CYCLE_STAT(TEXT("LegIK Eval"), STAT_LegIK_Eval, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("LegIK FABRIK Eval"), STAT_LegIK_FABRIK_Eval, STATGROUP_Anim);

FAnimNode_LegIK::FAnimNode_LegIK()
	: MyAnimInstanceProxy(nullptr)
{
	ReachPrecision = 0.01f;
	MaxIterations = 12;
	SoftPercentLength = 1.0f;
	SoftAlpha = 1.0f;
}

void FAnimNode_LegIK::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	// 	DebugLine += "(";
	// 	AddDebugNodeData(DebugLine);
	// 	DebugLine += FString::Printf(TEXT(" Target: %s)"), *BoneToModify.BoneName.ToString());

	DebugData.AddDebugItem(DebugLine);
	ComponentPose.GatherDebugData(DebugData);
}

static FVector GetBoneWorldLocation(const FTransform& InBoneTransform, FAnimInstanceProxy* MyAnimInstanceProxy)
{
	const FVector MeshCompSpaceLocation = InBoneTransform.GetLocation();
	return MyAnimInstanceProxy->GetComponentTransform().TransformPosition(MeshCompSpaceLocation);
}

#if ENABLE_DRAW_DEBUG
static void DrawDebugLeg(const FAnimLegIKData& InLegData, FAnimInstanceProxy* MyAnimInstanceProxy, const FColor& InColor)
{
	const USkeletalMeshComponent* SkelMeshComp = MyAnimInstanceProxy->GetSkelMeshComponent();
	for (int32 Index = 0; Index < InLegData.NumBones - 1; Index++)
	{
		const FVector CurrentBoneWorldLoc = GetBoneWorldLocation(InLegData.FKLegBoneTransforms[Index], MyAnimInstanceProxy);
		const FVector ParentBoneWorldLoc = GetBoneWorldLocation(InLegData.FKLegBoneTransforms[Index + 1], MyAnimInstanceProxy);
		MyAnimInstanceProxy->AnimDrawDebugLine(CurrentBoneWorldLoc, ParentBoneWorldLoc, InColor, false, -1.f, 2.f);
	}
}
#endif // ENABLE_DRAW_DEBUG

void FAnimNode_LegIK::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_SkeletalControlBase::Initialize_AnyThread(Context);

	MyAnimInstanceProxy = Context.AnimInstanceProxy;
}

void FAnimLegIKData::InitializeTransforms(FAnimInstanceProxy* MyAnimInstanceProxy, FCSPose<FCompactPose>& MeshBases)
{
	// Initialize bone transforms
	IKFootTransform = MeshBases.GetComponentSpaceTransform(IKFootBoneIndex);

	FKLegBoneTransforms.Reset(NumBones);
	for (const FCompactPoseBoneIndex& LegBoneIndex : FKLegBoneIndices)
	{
		FKLegBoneTransforms.Add(MeshBases.GetComponentSpaceTransform(LegBoneIndex));
	}

#if ENABLE_ANIM_DEBUG && ENABLE_DRAW_DEBUG
	const bool bShowDebug = (CVarAnimNodeLegIKDebug.GetValueOnAnyThread() == 1);
	if (bShowDebug)
	{
		DrawDebugLeg(*this, MyAnimInstanceProxy, FColor::Red);
		MyAnimInstanceProxy->AnimDrawDebugSphere(GetBoneWorldLocation(IKFootTransform, MyAnimInstanceProxy), 4.f, 4, FColor::Red, false, -1.f, 2.f);
	}
#endif // ENABLE_ANIM_DEBUG && ENABLE_DRAW_DEBUG
}

void FAnimNode_LegIK::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	SCOPE_CYCLE_COUNTER(STAT_LegIK_Eval);

	check(OutBoneTransforms.Num() == 0);

	// Get transforms for each leg.
	for (int32 LimbIndex = 0; LimbIndex < LegsData.Num(); LimbIndex++)
	{
		FAnimLegIKData& LegData = LegsData[LimbIndex];

		LegData.InitializeTransforms(MyAnimInstanceProxy, Output.Pose);
		LegData.TwistOffsetDegrees = Output.Curve.Get(LegData.LegDefPtr->TwistOffsetCurveName);

		// rotate hips so foot aligns with effector.
		const bool bOrientedLegTowardsIK = OrientLegTowardsIK(LegData);

		// expand/compress leg, so foot reaches effector.
		const bool bDidLegReachIK = DoLegReachIK(LegData);

		// Adjust knee twist orientation
		const bool bAdjustedKneeTwist = LegData.LegDefPtr->bEnableKneeTwistCorrection ? AdjustKneeTwist(LegData) : false;

		// Override Foot FK Rotation with Foot IK Rotation.
		bool bModifiedLimb = bOrientedLegTowardsIK || bDidLegReachIK || bAdjustedKneeTwist;
		bool bOverrideFootFKRotation = false;
		const FQuat IKFootRotation = LegData.IKFootTransform.GetRotation();
		if (bModifiedLimb || !LegData.FKLegBoneTransforms[0].GetRotation().Equals(IKFootRotation))
		{
			LegData.FKLegBoneTransforms[0].SetRotation(IKFootRotation);
			bOverrideFootFKRotation = true;
			bModifiedLimb = true;
		}

		if (bModifiedLimb)
		{
			// Add modified transforms
			for (int32 Index = 0; Index < LegData.NumBones; Index++)
			{
				OutBoneTransforms.Add(FBoneTransform(LegData.FKLegBoneIndices[Index], LegData.FKLegBoneTransforms[Index]));
			}
		}

#if ENABLE_ANIM_DEBUG
		const bool bShowDebug = (CVarAnimNodeLegIKDebug.GetValueOnAnyThread() == 1);
		if (bShowDebug)
		{
			FString DebugString = FString::Printf(TEXT("Limb[%d/%d] (%s) bModifiedLimb(%d) bOrientedLegTowardsIK(%d) bDidLegReachIK(%d) bAdjustedKneeTwist(%d) bOverrideFootFKRotation(%d)"),
				LimbIndex + 1, LegsData.Num(), *LegData.LegDefPtr->FKFootBone.BoneName.ToString(), 
				bModifiedLimb, bOrientedLegTowardsIK, bDidLegReachIK, bAdjustedKneeTwist, bOverrideFootFKRotation);
			MyAnimInstanceProxy->AnimDrawDebugOnScreenMessage(DebugString, FColor::Red);
		}
#endif
	}

	// Sort OutBoneTransforms so indices are in increasing order.
	OutBoneTransforms.Sort(FCompareBoneTransformIndex());
}

static bool RotateLegByQuat(const FQuat& InDeltaRotation, FAnimLegIKData& InLegData)
{
	if (!InDeltaRotation.IsIdentity())
	{
		const FVector HipLocation = InLegData.FKLegBoneTransforms.Last().GetLocation();

		// Rotate Leg so it is aligned with IK Target
		for (FTransform& LegBoneTransform : InLegData.FKLegBoneTransforms)
		{
			LegBoneTransform.SetRotation(InDeltaRotation * LegBoneTransform.GetRotation());

			const FVector BoneLocation = LegBoneTransform.GetLocation();
			LegBoneTransform.SetLocation(HipLocation + InDeltaRotation.RotateVector(BoneLocation - HipLocation));
		}

		return true;
	}

	return false;
}

static bool RotateLegByDeltaNormals(const FVector& InInitialDir, const FVector& InTargetDir, FAnimLegIKData& InLegData)
{
	if (!InInitialDir.IsZero() && !InInitialDir.Equals(InTargetDir))
	{
		// Find Delta Rotation take takes us from Old to New dir
		const FQuat DeltaRotation = FQuat::FindBetweenNormals(InInitialDir, InTargetDir);
		return RotateLegByQuat(DeltaRotation, InLegData);
	}

	return false;
}

bool FAnimNode_LegIK::OrientLegTowardsIK(FAnimLegIKData& InLegData)
{
	check(InLegData.NumBones > 1);
	const FVector HipLocation = InLegData.FKLegBoneTransforms.Last().GetLocation();
	const FVector FootFKLocation = InLegData.FKLegBoneTransforms[0].GetLocation();
	const FVector FootIKLocation = InLegData.IKFootTransform.GetLocation();

	const FVector InitialDir = (FootFKLocation - HipLocation).GetSafeNormal();
	const FVector TargetDir = (FootIKLocation - HipLocation).GetSafeNormal();

	if (RotateLegByDeltaNormals(InitialDir, TargetDir, InLegData))
	{
#if ENABLE_ANIM_DEBUG
		const bool bShowDebug = (CVarAnimNodeLegIKDebug.GetValueOnAnyThread() == 1);
		if (bShowDebug)
		{
			DrawDebugLeg(InLegData, MyAnimInstanceProxy, FColor::Green);
		}
#endif
		return true;
	}

	return false;
}

void FIKChain::InitializeFromLegData(FAnimLegIKData& InLegData, FAnimInstanceProxy* InAnimInstanceProxy)
{
	if (Links.Num() != InLegData.NumBones)
	{
		Links.Init(FIKChainLink(), InLegData.NumBones);
	}
	
	TotalChainLength = 0.0;

	check(InLegData.NumBones > 1);
	for (int32 Index = 0; Index < InLegData.NumBones - 1; Index++)
	{
		const FVector BoneLocation = InLegData.FKLegBoneTransforms[Index].GetLocation();
		const FVector ParentLocation = InLegData.FKLegBoneTransforms[Index + 1].GetLocation();
		const double BoneLength = FVector::Dist(BoneLocation, ParentLocation);

		FIKChainLink& Link = Links[Index];
		Link.Location = BoneLocation;
		Link.Length = BoneLength;

		TotalChainLength += BoneLength;
	}

	// Add root bone last
	const int32 RootIndex = InLegData.NumBones - 1;
	Links[RootIndex].Location = InLegData.FKLegBoneTransforms[RootIndex].GetLocation();
	Links[RootIndex].Length = 0.f;

	NumLinks = Links.Num();
	check(NumLinks == InLegData.NumBones);

	if (InLegData.LegDefPtr != nullptr)
	{
		bEnableRotationLimit = InLegData.LegDefPtr->bEnableRotationLimit;
		if (bEnableRotationLimit)
		{
			MinRotationAngleRadians = FMath::DegreesToRadians(FMath::Clamp(InLegData.LegDefPtr->MinRotationAngle, 0.f, 90.f));
		}

		HingeRotationAxis = (InLegData.LegDefPtr->HingeRotationAxis != EAxis::None)
			? InLegData.FKLegBoneTransforms.Last().GetUnitAxis(InLegData.LegDefPtr->HingeRotationAxis)
			: FVector::ZeroVector;
	}

	MyAnimInstanceProxy = InAnimInstanceProxy;
	bInitialized = true;
}

TAutoConsoleVariable<int32> CVarAnimLegIKTwoBone(TEXT("a.AnimNode.LegIK.EnableTwoBone"), 1, TEXT("Enable Two Bone Code Path."));

void FIKChain::ReachTarget(
	const FVector& InTargetLocation,
	double InReachPrecision,
	int32 InMaxIterations,
	float SoftPercentLength,
	float SoftAlpha)
{
	if (!bInitialized)
	{
		return;
	}

	const FVector RootLocation = Links.Last().Location;

	// Optionally soften the target location to prevent knee popping
	FVector FinalTargetLocation = InTargetLocation;
	const bool bUsingSoftIK = SoftPercentLength < 1.0f && SoftAlpha > 0.f;
	if (bUsingSoftIK)
	{
		AnimationCore::SoftenIKEffectorPosition(RootLocation, TotalChainLength, SoftPercentLength, SoftAlpha, FinalTargetLocation);
	}

	// If we can't reach, we just go in a straight line towards the target,
	const bool bTargetIsReachable = FVector::DistSquared(RootLocation, InTargetLocation) < FMath::Square(GetMaximumReach());
	const bool bHasTwoOrFewerLinks = NumLinks <= 2;
	if (bHasTwoOrFewerLinks || (!bTargetIsReachable && !bUsingSoftIK))
	{
		const FVector Direction = (InTargetLocation - RootLocation).GetSafeNormal();
		OrientAllLinksToDirection(Direction);
	}
	// Two Bones, we can figure out solution instantly
	else if (NumLinks == 3 && (CVarAnimLegIKTwoBone.GetValueOnAnyThread() == 1))
	{
		SolveTwoBoneIK(FinalTargetLocation);
	}
	// Do iterative approach based on FABRIK
	else
	{
		SolveFABRIK(FinalTargetLocation, InReachPrecision, InMaxIterations);
	}
}

void FIKChain::ApplyTwistOffset(const float InTwistOffsetDegrees)
{
	const FVector& HeadLoc = Links[0].Location;
	const FVector HeadToTail = Links.Last().Location - HeadLoc;
	const FVector RotationAxis = HeadToTail.GetSafeNormal();

	// Only apply twist to non tail/head links.
 	for (int32 Index = 1; Index < Links.Num() - 1; ++Index)
	{
		FVector& LinkLoc = Links[Index].Location;
		const FVector LinkToHead = LinkLoc - HeadLoc;

		LinkLoc = HeadLoc + LinkToHead.RotateAngleAxis(InTwistOffsetDegrees, RotationAxis);
	}
}

void FIKChain::OrientAllLinksToDirection(const FVector& InDirection)
{
	for (int32 Index = Links.Num() - 2; Index >= 0; Index--)
	{
		Links[Index].Location = Links[Index + 1].Location + InDirection * Links[Index].Length;
	}
}

void FIKChain::SolveTwoBoneIK(const FVector& InTargetLocation)
{
	check(Links.Num() == 3);

	FVector& pA = Links[0].Location; // Foot
	FVector& pB = Links[1].Location; // Knee
	FVector& pC = Links[2].Location; // Hip / Root

	// Move foot directly to target.
	pA = InTargetLocation;

	const FVector HipToFoot = pA - pC;

	// Use Law of Cosines to work out solution.
	// At this point we know the target location is reachable, and we are already aligned with that location. So the leg is in the right plane.
	const double a = Links[1].Length;	// hip to knee
	const double b = HipToFoot.Size();	// hip to foot
	const double c = Links[0].Length;	// knee to foot

	const double Two_ab = 2.f * a * b;
	const double CosC = !FMath::IsNearlyZero(Two_ab) ? (a * a + b * b - c * c) / Two_ab : 0.0;
 	const double C = FMath::Acos(CosC);
	
	// Project Knee onto Hip to Foot line.
	const FVector HipToFootDir = !FMath::IsNearlyZero(b) ? HipToFoot / b : FVector::ZeroVector;
	const FVector HipToKnee = pB - pC;
	const FVector ProjKnee = pC + HipToKnee.ProjectOnToNormal(HipToFootDir);

	const FVector ProjKneeToKnee = (pB - ProjKnee);
	FVector BendDir = ProjKneeToKnee.GetSafeNormal(KINDA_SMALL_NUMBER);
	
	// If we have a HingeRotationAxis defined, we can cache 'BendDir'
	// and use it when we can't determine it. (When limb is straight without a bend).
	// We do this instead of using an explicit one, so we carry over the pole vector that animators use. 
	// So they can animate it, and we try to extract it from the animation.
	if ((HingeRotationAxis != FVector::ZeroVector) && (HipToFootDir != FVector::ZeroVector) && !FMath::IsNearlyZero(a))
	{
		const FVector HipToKneeDir = HipToKnee / a;
		const double KneeBendDot = HipToKneeDir | HipToFootDir;

		FVector& CachedRealBendDir = Links[1].RealBendDir;
		FVector& CachedBaseBendDir = Links[1].BaseBendDir;

		// Valid 'bend', cache 'BendDir'
		if ((BendDir != FVector::ZeroVector) && (KneeBendDot < 0.99))
		{
			CachedRealBendDir = BendDir;
			CachedBaseBendDir = HingeRotationAxis ^ HipToFootDir;
		}
		// Limb is too straight, can't determine BendDir accurately, so use cached value if possible.
		else 
		{
			// If we have cached 'BendDir', then reorient it based on 'HingeRotationAxis'
			if (CachedRealBendDir != FVector::ZeroVector)
			{
				const FVector CurrentBaseBendDir = HingeRotationAxis ^ HipToFootDir;
				const FQuat DeltaCachedToCurrBendDir = FQuat::FindBetweenNormals(CachedBaseBendDir, CurrentBaseBendDir);
				BendDir = DeltaCachedToCurrBendDir.RotateVector(CachedRealBendDir);
			}
		}
	}

	// We just combine both lines into one to save a multiplication.
	// const FVector NewProjectedKneeLoc = pC + HipToFootDir * a * CosC;
	// const FVector NewKneeLoc = NewProjectedKneeLoc + Dir_LegLineToKnee * a * FMath::Sin(C);
	const FVector NewKneeLoc = pC + a * (HipToFootDir * CosC + BendDir * FMath::Sin(C));
	pB = NewKneeLoc;
}

bool FAnimNode_LegIK::DoLegReachIK(FAnimLegIKData& InLegData)
{
	SCOPE_CYCLE_COUNTER(STAT_LegIK_FABRIK_Eval);

	const FVector FootFKLocation = InLegData.FKLegBoneTransforms[0].GetLocation();
	const FVector FootIKLocation = InLegData.IKFootTransform.GetLocation();

	// There's no work to do if:
	//	- We don't have a twist offset.
	//	- We don't need to run the solver.
	const bool bHasTwistOffset = !FMath::IsNearlyZero(InLegData.TwistOffsetDegrees);
	// The solver is needed if:
	//	- Our FK foot is not at the IK goal.
	//	- We're applying a rotation limit.
	//  - We're using Soft IK (even if foot is at goal, it may be bent by the soft IK if limb is fully extended)
	const bool bUsingSoftIK = SoftPercentLength < 1.0f && SoftAlpha > 0.f;
	const bool bFootAtGoal = FootFKLocation.Equals(FootIKLocation, ReachPrecision);
	const bool bUsingRotationLimit = InLegData.LegDefPtr->bEnableRotationLimit;
	const bool bNeedsSolver = !bFootAtGoal || bUsingRotationLimit || bUsingSoftIK || (CVarAnimLegIKForceAlwaysSolve.GetValueOnAnyThread() == 1);
	if (!bNeedsSolver && !bHasTwistOffset)
	{
		return false;
	}

	FIKChain& IKChain = InLegData.IKChain;
	IKChain.InitializeFromLegData(InLegData, MyAnimInstanceProxy);

	if (bNeedsSolver)
	{
		const int32 MaxIterationsOverride = CVarAnimLegIKMaxIterations.GetValueOnAnyThread() > 0 ? CVarAnimLegIKMaxIterations.GetValueOnAnyThread() : MaxIterations;
		IKChain.ReachTarget(FootIKLocation, ReachPrecision, MaxIterationsOverride, SoftPercentLength, SoftAlpha);
	}

	if (bHasTwistOffset)
	{
		IKChain.ApplyTwistOffset(InLegData.TwistOffsetDegrees);
	}

	// Update bone transforms based on IKChain

	// Rotations
	for (int32 LinkIndex = InLegData.NumBones - 2; LinkIndex >= 0; LinkIndex--)
	{
		const FIKChainLink& ParentLink = IKChain.Links[LinkIndex + 1];
		const FIKChainLink& CurrentLink = IKChain.Links[LinkIndex];

		FTransform& ParentTransform = InLegData.FKLegBoneTransforms[LinkIndex + 1];
		FTransform& CurrentTransform = InLegData.FKLegBoneTransforms[LinkIndex];

		// Calculate pre-translation vector between this bone and child
		const FVector InitialDir = (CurrentTransform.GetLocation() - ParentTransform.GetLocation()).GetSafeNormal();

		// Get vector from the post-translation bone to it's child
		const FVector TargetDir = (CurrentLink.Location - ParentLink.Location).GetSafeNormal();

		const FQuat DeltaRotation = FQuat::FindBetweenNormals(InitialDir, TargetDir);
		ParentTransform.SetRotation(DeltaRotation * ParentTransform.GetRotation());
	}

	// Translations
	for (int32 LinkIndex = InLegData.NumBones - 2; LinkIndex >= 0; LinkIndex--)
	{
		const FIKChainLink& CurrentLink = IKChain.Links[LinkIndex];
		FTransform& CurrentTransform = InLegData.FKLegBoneTransforms[LinkIndex];

		CurrentTransform.SetTranslation(CurrentLink.Location);
	}

#if ENABLE_ANIM_DEBUG
	const bool bShowDebug = (CVarAnimNodeLegIKDebug.GetValueOnAnyThread() == 1);
	if (bShowDebug)
	{
		DrawDebugLeg(InLegData, MyAnimInstanceProxy, FColor::Yellow);
	}
#endif

	return true;
}

void FIKChain::DrawDebugIKChain(const FIKChain& IKChain, const FColor& InColor)
{
#if ENABLE_DRAW_DEBUG
	if (IKChain.bInitialized && IKChain.MyAnimInstanceProxy)
	{
		for (int32 Index = 0; Index < IKChain.NumLinks - 1; Index++)
		{
			const FVector CurrentBoneWorldLoc = GetBoneWorldLocation(FTransform(IKChain.Links[Index].Location), IKChain.MyAnimInstanceProxy);
			const FVector ParentBoneWorldLoc = GetBoneWorldLocation(FTransform(IKChain.Links[Index + 1].Location), IKChain.MyAnimInstanceProxy);
			IKChain.MyAnimInstanceProxy->AnimDrawDebugLine(CurrentBoneWorldLoc, ParentBoneWorldLoc, InColor, false, -1.f, 1.f);
		}
	}
#endif // ENABLE_DRAW_DEBUG
}

void FIKChain::FABRIK_ApplyLinkConstraints_Forward(FIKChain& IKChain, int32 LinkIndex)
{
	if ((LinkIndex <= 0) || (LinkIndex >= IKChain.NumLinks - 1))
	{
		return;
	}

	const FIKChainLink& ChildLink = IKChain.Links[LinkIndex - 1];
	const FIKChainLink& CurrentLink = IKChain.Links[LinkIndex];
	FIKChainLink& ParentLink = IKChain.Links[LinkIndex + 1];

	const FVector ChildAxisX = (ChildLink.Location - CurrentLink.Location).GetSafeNormal();
	const FVector ChildAxisY = CurrentLink.LinkAxisZ ^ ChildAxisX;
	const FVector ParentAxisX = (ParentLink.Location - CurrentLink.Location).GetSafeNormal();

	const double ParentCos = (ParentAxisX | ChildAxisX);
	const double ParentSin = (ParentAxisX | ChildAxisY);

	const bool bNeedsReorient = (ParentSin < 0.0) || (ParentCos > FMath::Cos(IKChain.MinRotationAngleRadians));

	// Parent Link needs to be reoriented.
	if (bNeedsReorient)
	{
		// folding over itself.
		if (ParentCos > 0.f)
		{
			// Enforce minimum angle.
			ParentLink.Location = CurrentLink.Location + CurrentLink.Length * (FMath::Cos(IKChain.MinRotationAngleRadians) * ChildAxisX + FMath::Sin(IKChain.MinRotationAngleRadians) * ChildAxisY);
		}
		else
		{
			// When opening up leg, allow it to extend in a full straight line.
			ParentLink.Location = CurrentLink.Location - ChildAxisX * CurrentLink.Length;
		}
	}
}

void FIKChain::FABRIK_ApplyLinkConstraints_Backward(FIKChain& IKChain, int32 LinkIndex)
{
	if ((LinkIndex <= 0) || (LinkIndex >= IKChain.NumLinks - 1))
	{
		return;
	}

	FIKChainLink& ChildLink = IKChain.Links[LinkIndex - 1];
	const FIKChainLink& CurrentLink = IKChain.Links[LinkIndex];
	const FIKChainLink& ParentLink = IKChain.Links[LinkIndex + 1];

	const FVector ParentAxisX = (ParentLink.Location - CurrentLink.Location).GetSafeNormal();
	const FVector ParentAxisY = CurrentLink.LinkAxisZ ^ ParentAxisX;
	const FVector ChildAxisX = (ChildLink.Location - CurrentLink.Location).GetSafeNormal();

	const double ChildCos = (ChildAxisX | ParentAxisX);
	const double ChildSin = (ChildAxisX | ParentAxisY);

	const bool bNeedsReorient = (ChildSin > 0.f) || (ChildCos > FMath::Cos(IKChain.MinRotationAngleRadians));

	// Parent Link needs to be reoriented.
	if (bNeedsReorient)
	{
		// folding over itself.
		if (ChildCos > 0.f)
		{
			// Enforce minimum angle.
			ChildLink.Location = CurrentLink.Location + ChildLink.Length * (FMath::Cos(IKChain.MinRotationAngleRadians) * ParentAxisX - FMath::Sin(IKChain.MinRotationAngleRadians) * ParentAxisY);
		}
		else
		{
			// When opening up leg, allow it to extend in a full straight line.
			ChildLink.Location = CurrentLink.Location - ParentAxisX * ChildLink.Length;
		}
	}
}

void FIKChain::FABRIK_ForwardReach(const FVector& InTargetLocation, FIKChain& IKChain)
{
	// Move end effector towards target
	// If we are compressing the chain, limit displacement.
	// Due to how FABRIK works, if we push the target past the parent's joint, we flip the bone.
	{
		FVector EndEffectorToTarget = InTargetLocation - IKChain.Links[0].Location;

		FVector EndEffectorToTargetDir;
		double EndEffectToTargetSize;
		EndEffectorToTarget.ToDirectionAndLength(EndEffectorToTargetDir, EndEffectToTargetSize);

		const double ReachStepAlpha = FMath::Clamp(CVarAnimLegIKTargetReachStepPercent.GetValueOnAnyThread(), 0.01, 0.99);

		double Displacement = EndEffectToTargetSize;
		for (int32 LinkIndex = 1; LinkIndex < IKChain.NumLinks; LinkIndex++)
		{
			FVector EndEffectorToParent = IKChain.Links[LinkIndex].Location - IKChain.Links[0].Location;
			double ParentDisplacement = (EndEffectorToParent | EndEffectorToTargetDir);

			Displacement = (ParentDisplacement > 0.0) ? FMath::Min(Displacement, ParentDisplacement * ReachStepAlpha) : Displacement;
		}

		IKChain.Links[0].Location += EndEffectorToTargetDir * Displacement;
	}

	// "Forward Reaching" stage - adjust bones from end effector.
	for (int32 LinkIndex = 1; LinkIndex < IKChain.NumLinks; LinkIndex++)
	{
		FIKChainLink& ChildLink = IKChain.Links[LinkIndex - 1];
		FIKChainLink& CurrentLink = IKChain.Links[LinkIndex];

		CurrentLink.Location = ChildLink.Location + (CurrentLink.Location - ChildLink.Location).GetSafeNormal() * ChildLink.Length;

		if (IKChain.bEnableRotationLimit)
		{
			FABRIK_ApplyLinkConstraints_Forward(IKChain, LinkIndex);
		}
	}
}

void FIKChain::FABRIK_BackwardReach(const FVector& InRootTargetLocation, FIKChain& IKChain)
{
	// Move Root back towards RootTarget
	// If we are compressing the chain, limit displacement.
	// Due to how FABRIK works, if we push the target past the parent's joint, we flip the bone.
	{
		FVector RootToRootTarget = InRootTargetLocation - IKChain.Links.Last().Location;

		FVector RootToRootTargetDir;
		float RootToRootTargetSize;
		RootToRootTarget.ToDirectionAndLength(RootToRootTargetDir, RootToRootTargetSize);

		const double ReachStepAlpha = FMath::Clamp(CVarAnimLegIKTargetReachStepPercent.GetValueOnAnyThread(), 0.01, 0.99);

		double Displacement = RootToRootTargetSize;
		for (int32 LinkIndex = IKChain.NumLinks - 2; LinkIndex >= 0; LinkIndex--)
		{
			FVector RootToChild = IKChain.Links[IKChain.NumLinks - 2].Location - IKChain.Links.Last().Location;
			double ChildDisplacement = (RootToChild | RootToRootTargetDir);

			Displacement = (ChildDisplacement > 0.0) ? FMath::Min(Displacement, ChildDisplacement * ReachStepAlpha) : Displacement;
		}

		IKChain.Links.Last().Location += RootToRootTargetDir * Displacement;
	}

	// "Backward Reaching" stage - adjust bones from root.
	for (int32 LinkIndex = IKChain.NumLinks - 1; LinkIndex >= 1; LinkIndex--)
	{
		FIKChainLink& CurrentLink = IKChain.Links[LinkIndex];
		FIKChainLink& ChildLink = IKChain.Links[LinkIndex - 1];

		ChildLink.Location = CurrentLink.Location + (ChildLink.Location - CurrentLink.Location).GetSafeNormal() * ChildLink.Length;

		if (IKChain.bEnableRotationLimit)
		{
			FABRIK_ApplyLinkConstraints_Backward(IKChain, LinkIndex);
		}
	}
}

static FVector FindPlaneNormal(const TArray<FIKChainLink>& Links, const FVector& RootLocation, const FVector& TargetLocation)
{
	const FVector AxisX = (TargetLocation - RootLocation).GetSafeNormal();

	for (int32 LinkIndex = Links.Num() - 2; LinkIndex >= 0; LinkIndex--)
	{
		const FVector AxisY = (Links[LinkIndex].Location - RootLocation).GetSafeNormal();
		const FVector PlaneNormal = AxisX ^ AxisY;

		// Make sure we have a valid normal (Axes were not coplanar).
		if (PlaneNormal.SizeSquared() > SMALL_NUMBER)
		{
			return PlaneNormal.GetUnsafeNormal();
		}
	}

	// All links are co-planar?
	return FVector::UpVector;
}

TAutoConsoleVariable<int32> CVarAnimLegIKAveragePull(TEXT("a.AnimNode.LegIK.AveragePull"), 1, TEXT("Leg IK AveragePull"));

void FIKChain::SolveFABRIK(const FVector& InTargetLocation, double InReachPrecision, int32 InMaxIterations)
{
	// Make sure precision is not too small.
	const double ReachPrecision = FMath::Max(InReachPrecision, DOUBLE_KINDA_SMALL_NUMBER);

	const FVector RootTargetLocation = Links.Last().Location;
	const double PullDistributionAlpha = FMath::Clamp(CVarAnimLegIKPullDistribution.GetValueOnAnyThread(), 0.0, 1.0);

	// Check distance between foot and foot target location
	double Slop = FVector::Dist(Links[0].Location, InTargetLocation);
	if (Slop > ReachPrecision || bEnableRotationLimit)
	{
		if (bEnableRotationLimit)
		{
			// Since we've previously aligned the foot with the IK Target, we're solving IK in 2D space on a single plane.
			// Find Plane Normal, to use in rotation constraints.
			const FVector PlaneNormal = FindPlaneNormal(Links, RootTargetLocation, InTargetLocation);

			for (int32 LinkIndex = 1; LinkIndex < (NumLinks - 1); LinkIndex++)
			{
				const FIKChainLink& ChildLink = Links[LinkIndex - 1];
				FIKChainLink& CurrentLink = Links[LinkIndex];
				const FIKChainLink& ParentLink = Links[LinkIndex + 1];

				const FVector ChildAxisX = (ChildLink.Location - CurrentLink.Location).GetSafeNormal();
				const FVector ChildAxisY = PlaneNormal ^ ChildAxisX;
				const FVector ParentAxisX = (ParentLink.Location - CurrentLink.Location).GetSafeNormal();

				// Orient Z, so that ChildAxisY points 'up' and produces positive Sin values.
				CurrentLink.LinkAxisZ = (ParentAxisX | ChildAxisY) > 0.f ? PlaneNormal : -PlaneNormal;
			}
		}

#if ENABLE_ANIM_DEBUG
		const bool bShowDebug = (CVarAnimNodeLegIKDebug.GetValueOnAnyThread() == 1);
		if (bShowDebug)
		{
			DrawDebugIKChain(*this, FColor::Magenta);
		}
#endif

		// Re-position limb to distribute pull
		const FVector PullDistributionOffset = PullDistributionAlpha * (InTargetLocation - Links[0].Location) + (1.f - PullDistributionAlpha) * (RootTargetLocation - Links.Last().Location);
		for (int32 LinkIndex = 0; LinkIndex < NumLinks; LinkIndex++)
		{
			Links[LinkIndex].Location += PullDistributionOffset;
		}

		int32 IterationCount = 1;
		const int32 MaxIterations = FMath::Max(InMaxIterations, 1);
		do
		{
			const double PreviousSlop = Slop;

#if ENABLE_ANIM_DEBUG
			bool bDrawDebug = bShowDebug && (IterationCount == (MaxIterations - 1));
			if (bDrawDebug) { DrawDebugIKChain(*this, FColor::Red); }
#endif

			// Pull averaging only has a visual impact when we have more than 2 bones (3 links).
			if ((NumLinks > 3) && (CVarAnimLegIKAveragePull.GetValueOnAnyThread() == 1) && (Slop > 1.f))
			{
				FIKChain ForwardPull = *this;
				FABRIK_ForwardReach(InTargetLocation, ForwardPull);

				FIKChain BackwardPull = *this;
				FABRIK_BackwardReach(RootTargetLocation, BackwardPull);

				// Average pulls
				for (int32 LinkIndex = 0; LinkIndex < NumLinks; LinkIndex++)
				{
					Links[LinkIndex].Location = 0.5f * (ForwardPull.Links[LinkIndex].Location + BackwardPull.Links[LinkIndex].Location);
				}

#if ENABLE_ANIM_DEBUG
				if (bDrawDebug)
				{
					DrawDebugIKChain(ForwardPull, FColor::Green);
					DrawDebugIKChain(BackwardPull, FColor::Blue);
				}
#endif
			}
			else
			{
				FABRIK_ForwardReach(InTargetLocation, *this);

#if ENABLE_ANIM_DEBUG
				if (bDrawDebug) { DrawDebugIKChain(*this, FColor::Green); }
#endif

				FABRIK_BackwardReach(RootTargetLocation, *this);
#if ENABLE_ANIM_DEBUG
				if (bDrawDebug) { DrawDebugIKChain(*this, FColor::Blue); }
#endif
			}

			Slop = FVector::Dist(Links[0].Location, InTargetLocation) + FVector::Dist(Links.Last().Location, RootTargetLocation);

			// Abort if we're not getting closer and enter a deadlock.
			if (Slop > PreviousSlop)
			{
				break;
			}

		} while ((Slop > ReachPrecision) && (++IterationCount < MaxIterations));

		// Make sure our root is back at our root target.
		if (!Links.Last().Location.Equals(RootTargetLocation))
		{
			FABRIK_BackwardReach(RootTargetLocation, *this);
		}

		// If we reached, set target precisely
		if (Slop <= ReachPrecision)
		{
			Links[0].Location = InTargetLocation;
		}

#if ENABLE_ANIM_DEBUG
		if (bShowDebug)
		{
			DrawDebugIKChain(*this, FColor::Yellow);

			FString DebugString = FString::Printf(TEXT("FABRIK IterationCount: [%d]/[%d], Slop: [%f]/[%f]")
				, IterationCount, MaxIterations, Slop, ReachPrecision);
			MyAnimInstanceProxy->AnimDrawDebugOnScreenMessage(DebugString, FColor::Red);
		}
#endif
	}
}

bool FAnimNode_LegIK::AdjustKneeTwist(FAnimLegIKData& InLegData)
{
	const FVector FootFKLocation = InLegData.FKLegBoneTransforms[0].GetLocation();
	const FVector FootIKLocation = InLegData.IKFootTransform.GetLocation();

	const FVector HipLocation = InLegData.FKLegBoneTransforms.Last().GetLocation();
	const FVector FootAxisZ = (FootIKLocation - HipLocation).GetSafeNormal();

	FVector FootFKAxisX = InLegData.FKLegBoneTransforms[0].GetUnitAxis(InLegData.LegDefPtr->FootBoneForwardAxis);
	FVector FootIKAxisX = InLegData.IKFootTransform.GetUnitAxis(InLegData.LegDefPtr->FootBoneForwardAxis);

	// Reorient X Axis to be perpendicular with FootAxisZ
	FootFKAxisX = ((FootAxisZ ^ FootFKAxisX) ^ FootAxisZ);
	FootIKAxisX = ((FootAxisZ ^ FootIKAxisX) ^ FootAxisZ);

	// Compare Axis X to see if we need a rotation to be performed
	if (RotateLegByDeltaNormals(FootFKAxisX, FootIKAxisX, InLegData))
	{
#if ENABLE_ANIM_DEBUG
		const bool bShowDebug = (CVarAnimNodeLegIKDebug.GetValueOnAnyThread() == 1);
		if (bShowDebug)
		{
			DrawDebugLeg(InLegData, MyAnimInstanceProxy, FColor::Magenta);
		}
#endif
		return true;
	}

	return false;
}

bool FAnimNode_LegIK::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones)
{
	const bool bIsEnabled = (CVarAnimLegIKEnable.GetValueOnAnyThread() == 1);
	return bIsEnabled && (LegsData.Num() > 0);
}

static void PopulateLegBoneIndices(FAnimLegIKData& InLegData, const FCompactPoseBoneIndex& InFootBoneIndex, const int32& NumBonesInLimb, const FBoneContainer& RequiredBones)
{
	FCompactPoseBoneIndex BoneIndex = InFootBoneIndex;
	if (BoneIndex != INDEX_NONE)
	{
		InLegData.FKLegBoneIndices.Add(BoneIndex);
		FCompactPoseBoneIndex ParentBoneIndex = RequiredBones.GetParentBoneIndex(BoneIndex);

		int32 NumIterations = NumBonesInLimb;
		while ((NumIterations-- > 0) && (ParentBoneIndex != INDEX_NONE))
		{
			BoneIndex = ParentBoneIndex;
			InLegData.FKLegBoneIndices.Add(BoneIndex);
			ParentBoneIndex = RequiredBones.GetParentBoneIndex(BoneIndex);
		};
	}
}

void FAnimNode_LegIK::InitializeBoneReferences(const FBoneContainer& RequiredBones)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	// Preserve FIKChain for each leg, as we're trying to maintain CachedBendDir between LOD transitions.
	TMap<FName, FIKChain> IKChainLUT;
	for(const FAnimLegIKData& LegData : LegsData)
	{
		if (LegData.LegDefPtr)
		{
			IKChainLUT.Add(LegData.LegDefPtr->FKFootBone.BoneName, LegData.IKChain);
		}
	}

	LegsData.Reset();
	for (FAnimLegIKDefinition& LegDef : LegsDefinition)
	{
		LegDef.IKFootBone.Initialize(RequiredBones);
		LegDef.FKFootBone.Initialize(RequiredBones);

		FAnimLegIKData LegData;
		LegData.IKFootBoneIndex = LegDef.IKFootBone.GetCompactPoseIndex(RequiredBones);
		const FCompactPoseBoneIndex FKFootBoneIndex = LegDef.FKFootBone.GetCompactPoseIndex(RequiredBones);

		if ((LegData.IKFootBoneIndex != INDEX_NONE) && (FKFootBoneIndex != INDEX_NONE))
		{
			PopulateLegBoneIndices(LegData, FKFootBoneIndex, FMath::Max(LegDef.NumBonesInLimb, 1), RequiredBones);

			// We need at least three joints for this to work (hip, knee and foot).
			if (LegData.FKLegBoneIndices.Num() >= 3)
			{
				LegData.NumBones = LegData.FKLegBoneIndices.Num();
				if (FIKChain* IKChainPtr = IKChainLUT.Find(LegDef.FKFootBone.BoneName))
				{
					LegData.IKChain = *IKChainPtr;
				}
				LegData.LegDefPtr = &LegDef;
				LegsData.Add(LegData);
			}
		}
	}
}

