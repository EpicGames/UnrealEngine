// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_LookAt.h"
#include "Components/SkeletalMeshComponent.h"
#include "Materials/Material.h"
#include "PrimitiveDrawingUtils.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimStats.h"
#include "AnimationCoreLibrary.h"
#include "Engine/Engine.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_LookAt)

static const FVector DefaultLookAtAxis(0.f, 1.f, 0.f);
static const FVector DefaultLookUpAxis(1.f, 0.f, 0.f);

/////////////////////////////////////////////////////
// FAnimNode_LookAt

FAnimNode_LookAt::FAnimNode_LookAt()
	: LookAtLocation(FVector(100.f, 0.f, 0.f))
	, LookAt_Axis(DefaultLookAtAxis)
	, bUseLookUpAxis(false)
	, InterpolationType(EInterpolationBlend::Linear)
	, LookUp_Axis(DefaultLookUpAxis)
	, LookAtClamp(0.f)
	, InterpolationTime(0.f)
	, InterpolationTriggerThreashold(0.f)
#if WITH_EDITORONLY_DATA
	, LookAtAxis_DEPRECATED(EAxisOption::Y)
	, CustomLookAtAxis_DEPRECATED(FVector(0.f, 1.f, 0.f))
	, LookUpAxis_DEPRECATED(EAxisOption::X)
	, CustomLookUpAxis_DEPRECATED(FVector(1.f, 0.f, 0.f))
#endif
	, CurrentLookAtLocation(ForceInitToZero)
	, CurrentTargetLocation(ForceInitToZero)
	, PreviousTargetLocation(ForceInitToZero)
	, AccumulatedInterpoolationTime(0.f)
	, CachedCurrentTargetLocation(ForceInitToZero)
{
}

void FAnimNode_LookAt::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	DebugLine += "(";
	AddDebugNodeData(DebugLine);
	if (LookAtTarget.HasValidSetup())
	{
		DebugLine += FString::Printf(TEXT(" Bone: %s, Look At Target: %s, Look At Location: %s, Target Location : %s)"), *BoneToModify.BoneName.ToString(), *LookAtTarget.GetTargetSetup().ToString(), *LookAtLocation.ToString(), *CachedCurrentTargetLocation.ToString());
	}
	else
	{
		DebugLine += FString::Printf(TEXT(" Bone: %s, Look At Location : %s, Target Location : %s)"), *BoneToModify.BoneName.ToString(), *LookAtLocation.ToString(), *CachedCurrentTargetLocation.ToString());
	}

	DebugData.AddDebugItem(DebugLine);

	ComponentPose.GatherDebugData(DebugData);
}

float FAnimNode_LookAt::AlphaToBlendType(float InAlpha, EInterpolationBlend::Type BlendType)
{
	switch (BlendType)
	{
	case EInterpolationBlend::Sinusoidal: return FMath::Clamp<float>((FMath::Sin(InAlpha * PI - HALF_PI) + 1.f) / 2.f, 0.f, 1.f);
	case EInterpolationBlend::Cubic: return FMath::Clamp<float>(FMath::CubicInterp<float>(0.f, 0.f, 1.f, 0.f, InAlpha), 0.f, 1.f);
	case EInterpolationBlend::EaseInOutExponent2: return FMath::Clamp<float>(FMath::InterpEaseInOut<float>(0.f, 1.f, InAlpha, 2), 0.f, 1.f);
	case EInterpolationBlend::EaseInOutExponent3: return FMath::Clamp<float>(FMath::InterpEaseInOut<float>(0.f, 1.f, InAlpha, 3), 0.f, 1.f);
	case EInterpolationBlend::EaseInOutExponent4: return FMath::Clamp<float>(FMath::InterpEaseInOut<float>(0.f, 1.f, InAlpha, 4), 0.f, 1.f);
	case EInterpolationBlend::EaseInOutExponent5: return FMath::Clamp<float>(FMath::InterpEaseInOut<float>(0.f, 1.f, InAlpha, 5), 0.f, 1.f);
	}

	return InAlpha;
}

void FAnimNode_LookAt::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(LookAt, !IsInGameThread());

	check(OutBoneTransforms.Num() == 0);

	const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();
	const FCompactPoseBoneIndex ModifyBoneIndex = BoneToModify.GetCompactPoseIndex(BoneContainer);
	FTransform ComponentBoneTransform = Output.Pose.GetComponentSpaceTransform(ModifyBoneIndex);

	// get target location
	FTransform TargetTransform = LookAtTarget.GetTargetTransform(LookAtLocation, Output.Pose, Output.AnimInstanceProxy->GetComponentTransform());
	FVector TargetLocationInComponentSpace = TargetTransform.GetLocation();
	
	FVector OldCurrentTargetLocation = CurrentTargetLocation;
	FVector NewCurrentTargetLocation = TargetLocationInComponentSpace;

	if ((NewCurrentTargetLocation - OldCurrentTargetLocation).SizeSquared() > InterpolationTriggerThreashold*InterpolationTriggerThreashold)
	{
		if (AccumulatedInterpoolationTime >= InterpolationTime)
		{
			// reset current Alpha, we're starting to move
			AccumulatedInterpoolationTime = 0.f;
		}

		PreviousTargetLocation = OldCurrentTargetLocation;
		CurrentTargetLocation = NewCurrentTargetLocation;
	}
	else if (InterpolationTriggerThreashold == 0.f)
	{
		CurrentTargetLocation = NewCurrentTargetLocation;
	}

	if (InterpolationTime > 0.f)
	{
		float CurrentAlpha = AccumulatedInterpoolationTime/InterpolationTime;

		if (CurrentAlpha < 1.f)
		{
			float BlendAlpha = AlphaToBlendType(CurrentAlpha, InterpolationType);

			CurrentLookAtLocation = FMath::Lerp(PreviousTargetLocation, CurrentTargetLocation, BlendAlpha);
		}
	}
	else
	{
		CurrentLookAtLocation = CurrentTargetLocation;
	}

#if UE_ENABLE_DEBUG_DRAWING
	CachedOriginalTransform = ComponentBoneTransform;
	CachedTargetCoordinate = LookAtTarget.GetTargetTransform(FVector::ZeroVector, Output.Pose, Output.AnimInstanceProxy->GetComponentTransform());
	CachedPreviousTargetLocation = PreviousTargetLocation;
	CachedCurrentLookAtLocation = CurrentLookAtLocation;
#endif
	CachedCurrentTargetLocation = CurrentTargetLocation;

	// lookat vector
	FVector LookAtVector = LookAt_Axis.GetTransformedAxis(ComponentBoneTransform);
	// find look up vector in local space
	FVector LookUpVector = LookUp_Axis.GetTransformedAxis(ComponentBoneTransform);
	// Find new transform from look at info
	FQuat DeltaRotation = AnimationCore::SolveAim(ComponentBoneTransform, CurrentLookAtLocation, LookAtVector, bUseLookUpAxis, LookUpVector, LookAtClamp);

	ModifyPoseFromDeltaRotation(Output, OutBoneTransforms, ComponentBoneTransform, DeltaRotation);

	// Sort OutBoneTransforms so indices are in increasing order.
	OutBoneTransforms.Sort(FCompareBoneTransformIndex());

#if UE_ENABLE_DEBUG_DRAWING
	CachedLookAtTransform = ComponentBoneTransform;
#endif

	TRACE_ANIM_NODE_VALUE(Output, TEXT("Bone"), BoneToModify.BoneName);
	TRACE_ANIM_NODE_VALUE(Output, TEXT("Look At Target"), LookAtTarget.HasValidSetup() ? LookAtTarget.GetTargetSetup() : NAME_None);
	TRACE_ANIM_NODE_VALUE(Output, TEXT("Look At Location"), LookAtLocation);
	TRACE_ANIM_NODE_VALUE(Output, TEXT("Target Location"), CachedCurrentTargetLocation);
}

void FAnimNode_LookAt::EvaluateComponentSpaceInternal(FComponentSpacePoseContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateComponentSpaceInternal)
	Super::EvaluateComponentSpaceInternal(Context);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/*if (bEnableDebug)
	{
		const FTransform LocalToWorld = Context.AnimInstanceProxy->GetComponentTransform();
		FVector TargetWorldLoc = LocalToWorld.TransformPosition(CachedCurrentTargetLocation);
		FVector SourceWorldLoc = LocalToWorld.TransformPosition(CachedComponentBoneLocation);

		Context.AnimInstanceProxy->AnimDrawDebugLine(SourceWorldLoc, TargetWorldLoc, FColor::Green);
	}*/

#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
}

bool FAnimNode_LookAt::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) 
{
	// if both bones are valid
	return (BoneToModify.IsValidToEvaluate(RequiredBones) &&
		// or if name isn't set (use Look At Location) or Look at bone is valid 
		// do not call isValid since that means if look at bone isn't in LOD, we won't evaluate
		// we still should evaluate as long as the BoneToModify is valid even LookAtBone isn't included in required bones
		(!LookAtTarget.HasTargetSetup() || LookAtTarget.IsValidToEvaluate(RequiredBones)) );
}

#if WITH_EDITOR
// can't use World Draw functions because this is called from Render of viewport, AFTER ticking component, 
// which means LineBatcher already has ticked, so it won't render anymore
// to use World Draw functions, we have to call this from tick of actor
void FAnimNode_LookAt::ConditionalDebugDraw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* MeshComp) const
{
	auto CalculateLookAtMatrixFromTransform = [this](const FTransform& BaseTransform) -> FMatrix
	{
		FVector TransformedLookAtAxis = BaseTransform.TransformVector(LookAt_Axis.Axis);
		const FVector DefaultUpVector = BaseTransform.GetUnitAxis(EAxis::Z);
		FVector UpVector = (bUseLookUpAxis) ? BaseTransform.TransformVector(LookUp_Axis.Axis) : DefaultUpVector;
		// if parallel with up vector, find something else
		if (FMath::Abs(FVector::DotProduct(UpVector, TransformedLookAtAxis)) > (1.f - ZERO_ANIMWEIGHT_THRESH))
		{
			UpVector = BaseTransform.GetUnitAxis(EAxis::X);
		}

		FVector RightVector = FVector::CrossProduct(TransformedLookAtAxis, UpVector);
		FMatrix Matrix;
		FVector Location = BaseTransform.GetLocation();
		Matrix.SetAxes(&TransformedLookAtAxis, &RightVector, &UpVector, &Location);
		return Matrix;
	};

	// did not apply any of LocaltoWorld
	if(PDI && MeshComp)
	{
		FTransform LocalToWorld = MeshComp->GetComponentTransform();
		FTransform ComponentTransform = CachedOriginalTransform * LocalToWorld;
		FTransform LookAtTransform = CachedLookAtTransform * LocalToWorld;
		FTransform TargetTrasnform = CachedTargetCoordinate * LocalToWorld;
		FVector BoneLocation = LookAtTransform.GetLocation();

		// we're using interpolation, so print previous location
		const bool bUseInterpolation = InterpolationTime > 0.f;
		if(bUseInterpolation)
		{
			// this only will be different if we're interpolating
			DrawDashedLine(PDI, BoneLocation, LocalToWorld.TransformPosition(CachedPreviousTargetLocation), FColor(0, 255, 0), 5.f, SDPG_World);
		}

		// current look at location (can be clamped or interpolating)
		DrawDashedLine(PDI, BoneLocation, LocalToWorld.TransformPosition(CachedCurrentLookAtLocation), FColor::Yellow, 5.f, SDPG_World);
		DrawWireStar(PDI, CachedCurrentLookAtLocation, 5.0f, FColor::Yellow, SDPG_World);

		// draw current target information
		DrawDashedLine(PDI, BoneLocation, LocalToWorld.TransformPosition(CachedCurrentTargetLocation), FColor::Blue, 5.f, SDPG_World);
		DrawWireStar(PDI, CachedCurrentTargetLocation, 5.0f, FColor::Blue, SDPG_World);

		// draw the angular clamp
		if (LookAtClamp > 0.f)
		{
			float Angle = FMath::DegreesToRadians(LookAtClamp);
			float ConeSize = 30.f;
			DrawCone(PDI, FScaleMatrix(ConeSize) * CalculateLookAtMatrixFromTransform(ComponentTransform), Angle, Angle, 20, false, FLinearColor::Green, GEngine->DebugEditorMaterial->GetRenderProxy(), SDPG_World);
		}

		// draw directional  - lookat and look up
		DrawDirectionalArrow(PDI, CalculateLookAtMatrixFromTransform(LookAtTransform), FLinearColor::Red, 20, 5, SDPG_World);
		DrawCoordinateSystem(PDI, BoneLocation, LookAtTransform.GetRotation().Rotator(), 20.f, SDPG_Foreground);
		DrawCoordinateSystem(PDI, TargetTrasnform.GetLocation(), TargetTrasnform.GetRotation().Rotator(), 20.f, SDPG_Foreground);
	}
}
#endif // WITH_EDITOR

void FAnimNode_LookAt::InitializeBoneReferences(const FBoneContainer& RequiredBones)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	BoneToModify.Initialize(RequiredBones);
	LookAtTarget.InitializeBoneReferences(RequiredBones);
}

void FAnimNode_LookAt::UpdateInternal(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(UpdateInternal)
	FAnimNode_SkeletalControlBase::UpdateInternal(Context);

	AccumulatedInterpoolationTime = FMath::Clamp(AccumulatedInterpoolationTime+Context.GetDeltaTime(), 0.f, InterpolationTime);;
}

void FAnimNode_LookAt::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_SkeletalControlBase::Initialize_AnyThread(Context);

	LookAtTarget.Initialize(Context.AnimInstanceProxy);

	// initialize
	LookUp_Axis.Initialize();
	if (LookUp_Axis.Axis.IsZero())
	{
		UE_LOGF(LogAnimation, Warning, "Zero-length look-up axis specified in LookAt node. Reverting to default. Instance:%ls", *GetFullNameSafe(Context.GetAnimInstanceObject()));
		LookUp_Axis.Axis = DefaultLookUpAxis;
	}
	LookAt_Axis.Initialize();
	if (LookAt_Axis.Axis.IsZero())
	{
		UE_LOGF(LogAnimation, Warning, "Zero-length look-at axis specified in LookAt node. Reverting to default. Instance:%ls", *GetFullNameSafe(Context.GetAnimInstanceObject()));
		LookAt_Axis.Axis = DefaultLookAtAxis;
	}
}

void FAnimNode_LookAt::ModifyPoseFromDeltaRotation(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms, FTransform& InOutBoneToModifyTransform, const FQuat& DeltaRotation)
{
	InOutBoneToModifyTransform.SetRotation(DeltaRotation * InOutBoneToModifyTransform.GetRotation());
	const FCompactPoseBoneIndex BoneToModifyIndex = BoneToModify.GetCompactPoseIndex(Output.Pose.GetPose().GetBoneContainer());
	OutBoneTransforms.Add(FBoneTransform(BoneToModifyIndex, InOutBoneToModifyTransform));
}

