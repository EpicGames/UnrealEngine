// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_ModifyBone.h"
#include "AnimationRuntime.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimTrace.h"

/////////////////////////////////////////////////////
// FAnimNode_ModifyBone

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_ModifyBone)

FAnimNode_ModifyBone::FAnimNode_ModifyBone()
	: Translation(FVector::ZeroVector)
	, Rotation(FRotator::ZeroRotator)
	, Scale(FVector(1.0f))
	, TranslationMode(BMM_Ignore)
	, RotationMode(BMM_Ignore)
	, ScaleMode(BMM_Ignore)
	, TranslationSpace(BCS_ComponentSpace)
	, RotationSpace(BCS_ComponentSpace)
	, ScaleSpace(BCS_ComponentSpace)
{
}

void FAnimNode_ModifyBone::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	DebugLine += "(";
	AddDebugNodeData(DebugLine);
	DebugLine += FString::Printf(TEXT(" Target: %s)"), *BoneToModify.BoneName.ToString());
	DebugData.AddDebugItem(DebugLine);

	ComponentPose.GatherDebugData(DebugData);
}

void FAnimNode_ModifyBone::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(ModifyBone, !IsInGameThread());

	check(OutBoneTransforms.Num() == 0);

	// the way we apply transform is same as FMatrix or FTransform
	// we apply scale first, and rotation, and translation
	// if you'd like to translate first, you'll need two nodes that first node does translate and second nodes to rotate.
	const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();

	FCompactPoseBoneIndex CompactPoseBoneToModify = BoneToModify.GetCompactPoseIndex(BoneContainer);
	FTransform NewBoneTM = Output.Pose.GetComponentSpaceTransform(CompactPoseBoneToModify);
	FTransform ComponentTransform = Output.AnimInstanceProxy->GetComponentTransform();

	if (ScaleMode != BMM_Ignore)
	{
		// Convert to Bone Space.
		FAnimationRuntime::ConvertCSTransformToBoneSpace(ComponentTransform, Output.Pose, NewBoneTM, CompactPoseBoneToModify, ScaleSpace);

		if (ScaleMode == BMM_Additive)
		{
			NewBoneTM.SetScale3D(NewBoneTM.GetScale3D() * Scale);
		}
		else
		{
			NewBoneTM.SetScale3D(Scale);
		}

		// Convert back to Component Space.
		FAnimationRuntime::ConvertBoneSpaceTransformToCS(ComponentTransform, Output.Pose, NewBoneTM, CompactPoseBoneToModify, ScaleSpace);
	}

	if (RotationMode != BMM_Ignore)
	{
		// Convert to Bone Space.
		FAnimationRuntime::ConvertCSTransformToBoneSpace(ComponentTransform, Output.Pose, NewBoneTM, CompactPoseBoneToModify, RotationSpace);

		const FQuat BoneQuat(Rotation);
		if (RotationMode == BMM_Additive)
		{	
			NewBoneTM.SetRotation(BoneQuat * NewBoneTM.GetRotation());
		}
		else
		{
			NewBoneTM.SetRotation(BoneQuat);
		}

		// Convert back to Component Space.
		FAnimationRuntime::ConvertBoneSpaceTransformToCS(ComponentTransform, Output.Pose, NewBoneTM, CompactPoseBoneToModify, RotationSpace);
	}
	
	if (TranslationMode != BMM_Ignore)
	{
		// Convert to Bone Space.
		FAnimationRuntime::ConvertCSTransformToBoneSpace(ComponentTransform, Output.Pose, NewBoneTM, CompactPoseBoneToModify, TranslationSpace);

		if (TranslationMode == BMM_Additive)
		{
			NewBoneTM.AddToTranslation(Translation);
		}
		else
		{
			NewBoneTM.SetTranslation(Translation);
		}

		// Convert back to Component Space.
		FAnimationRuntime::ConvertBoneSpaceTransformToCS(ComponentTransform, Output.Pose, NewBoneTM, CompactPoseBoneToModify, TranslationSpace);
	}
	
	OutBoneTransforms.Add( FBoneTransform(BoneToModify.GetCompactPoseIndex(BoneContainer), NewBoneTM) );

	TRACE_ANIM_NODE_VALUE(Output, TEXT("Target"), BoneToModify.BoneName);
}

bool FAnimNode_ModifyBone::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) 
{
	// if both bones are valid
	return (BoneToModify.IsValidToEvaluate(RequiredBones));
}

void FAnimNode_ModifyBone::InitializeBoneReferences(const FBoneContainer& RequiredBones) 
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	BoneToModify.Initialize(RequiredBones);
}
