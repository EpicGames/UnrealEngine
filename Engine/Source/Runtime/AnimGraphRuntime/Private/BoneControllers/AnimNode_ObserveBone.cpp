// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_ObserveBone.h"
#include "AnimationRuntime.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_ObserveBone)

/////////////////////////////////////////////////////
// FAnimNode_ObserveBone

FAnimNode_ObserveBone::FAnimNode_ObserveBone()
	: DisplaySpace(BCS_ComponentSpace)
	, bRelativeToRefPose(false)
	, Translation(FVector::ZeroVector)
	, Rotation(FRotator::ZeroRotator)
	, Scale(FVector(1.0f))
{
}

void FAnimNode_ObserveBone::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	const FString DebugLine = FString::Printf(TEXT("(Bone: %s has T(%s), R(%s), S(%s))"), *BoneToObserve.BoneName.ToString(), *Translation.ToString(), *Rotation.Euler().ToString(), *Scale.ToString());

	DebugData.AddDebugItem(DebugLine);

	ComponentPose.GatherDebugData(DebugData);
}

void FAnimNode_ObserveBone::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(ObserveBone, !IsInGameThread());

	const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();

	const FCompactPoseBoneIndex BoneIndex = BoneToObserve.GetCompactPoseIndex(BoneContainer);
	FTransform BoneTM = Output.Pose.GetComponentSpaceTransform(BoneIndex);
	
	// Convert to the specific display space if necessary
	FAnimationRuntime::ConvertCSTransformToBoneSpace(Output.AnimInstanceProxy->GetComponentTransform(), Output.Pose, BoneTM, BoneIndex, DisplaySpace);

	// Convert to be relative to the ref pose if necessary
	if (bRelativeToRefPose)
	{
		const FTransform& SourceOrigRef = BoneContainer.GetRefPoseArray()[BoneToObserve.BoneIndex];
		BoneTM = BoneTM.GetRelativeTransform(SourceOrigRef);
	}

	// Cache off the values for display
	Translation = BoneTM.GetTranslation();
	Rotation = BoneTM.GetRotation().Rotator();
	Scale = BoneTM.GetScale3D();

	TRACE_ANIM_NODE_VALUE(Output, TEXT("Bone"), BoneToObserve.BoneName);
	TRACE_ANIM_NODE_VALUE(Output, TEXT("Translation"), Translation);
	TRACE_ANIM_NODE_VALUE(Output, TEXT("Rotation"), Rotation.Euler());
	TRACE_ANIM_NODE_VALUE(Output, TEXT("Scale"), Scale);
}

bool FAnimNode_ObserveBone::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones)
{
	return (BoneToObserve.IsValidToEvaluate(RequiredBones));
}

void FAnimNode_ObserveBone::InitializeBoneReferences(const FBoneContainer& RequiredBones)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	BoneToObserve.Initialize(RequiredBones);
}

