// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_ResetRoot.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimStats.h"
#include "AnimationRuntime.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_ResetRoot)

/////////////////////////////////////////////////////
// FAnimNode_ResetRoot

FAnimNode_ResetRoot::FAnimNode_ResetRoot()
{
}

void FAnimNode_ResetRoot::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	DebugLine += "(";
	AddDebugNodeData(DebugLine);
	DebugLine += FString::Printf(TEXT(")"));
	DebugData.AddDebugItem(DebugLine);

	ComponentPose.GatherDebugData(DebugData);
}

void FAnimNode_ResetRoot::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(ResetRoot, !IsInGameThread());

	check(OutBoneTransforms.IsEmpty());

	FCSPose<FCompactPose>& CSPose = Output.Pose;
	const FBoneContainer& BoneContainer = CSPose.GetPose().GetBoneContainer();

	// Reset Root
	const FCompactPoseBoneIndex RootBoneIndex = FCompactPoseBoneIndex(0);
	const FTransform RootTransform = BoneContainer.GetRefPoseTransform(RootBoneIndex);
	
	OutBoneTransforms.Reserve(RootChildren.Num() + 1);
	OutBoneTransforms.Add(FBoneTransform(RootBoneIndex, RootTransform));
	for (int32 Index = 0; Index < RootChildren.Num(); Index++)
	{
		OutBoneTransforms.Add(FBoneTransform(RootChildren[Index], CSPose.GetComponentSpaceTransform(RootChildren[Index])));
	}

	OutBoneTransforms.Sort(FCompareBoneTransformIndex());
}

bool FAnimNode_ResetRoot::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones)
{
	return true;
}

void FAnimNode_ResetRoot::OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance)
{
}

void FAnimNode_ResetRoot::InitializeBoneReferences(const FBoneContainer& RequiredBones)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	// Gather all direct Children of the Root.
	RootChildren.Reset();

	const int32 NumCompactPoseBones = RequiredBones.GetCompactPoseNumBones();
	for (int32 Index = 0; Index < NumCompactPoseBones; Index++)
	{
		const FCompactPoseBoneIndex BoneIndex = FCompactPoseBoneIndex(Index);
		const FCompactPoseBoneIndex ParentBoneIndex = RequiredBones.GetParentBoneIndex(BoneIndex);
		if (ParentBoneIndex == 0)
		{
			RootChildren.Add(BoneIndex);
		}
	}
}


