// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_RigidBody.h"
#include "Animation/AnimInstance.h"
#include "AnimationRuntime.h"
#include "Animation/AnimInstanceProxy.h"

#if WITH_CHAOS_VISUAL_DEBUGGER
#include "ChaosVDRuntimeModule.h"
#endif

#include "ChaosDebugDraw/ChaosDDScene.h"
#include "Chaos/Collision/CollisionContext.h"
#include "ClothCollisionSource.h"
#include "Engine/OverlapResult.h"
#include "Engine/SkeletalMesh.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PawnMovementComponent.h"
#include "HAL/Event.h"
#include "HAL/LowLevelMemStats.h"
#include "HAL/LowLevelMemTracker.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsActorHandle.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsAdapters.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsJointHandle.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsSimulation.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsStats.h"
#include "Physics/PhysicsInterfaceCore.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "PhysicsField/PhysicsFieldComponent.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "Logging/MessageLog.h"
#include "Logging/LogMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_RigidBody)

LLM_DEFINE_TAG(Animation_RigidBody);

//UE_DISABLE_OPTIMIZATION

/////////////////////////////////////////////////////
// FAnimNode_RigidBody

#define LOCTEXT_NAMESPACE "ImmediatePhysics"

DEFINE_STAT(STAT_RigidBodyNodeInitTime);
DEFINE_STAT(STAT_RigidBodyNodeInitTime_SetupSimulation);

CSV_DECLARE_CATEGORY_MODULE_EXTERN(ENGINE_API, Animation);

#if UE_BUILD_SHIPPING || UE_BUILD_TEST
DECLARE_LOG_CATEGORY_EXTERN(LogRBAN, Warning, Warning);
#else
DECLARE_LOG_CATEGORY_EXTERN(LogRBAN, Log, All);
#endif
DEFINE_LOG_CATEGORY(LogRBAN);

bool bEnableRigidBodyNode = true;
FAutoConsoleVariableRef CVarEnableRigidBodyNode(TEXT("p.RigidBodyNode"), bEnableRigidBodyNode, TEXT("Enables/disables the whole rigid body node system. When disabled, avoids all allocations and runtime costs. Can be used to disable RB Nodes on low-end platforms."), ECVF_Scalability);
TAutoConsoleVariable<int32> CVarEnableRigidBodyNodeSimulation(TEXT("p.RigidBodyNode.EnableSimulation"), 1, TEXT("Runtime Enable/Disable RB Node Simulation for debugging and testing (node is initialized and bodies and constraints are created, even when disabled.)"), ECVF_Default);
TAutoConsoleVariable<int32> CVarRigidBodyLODThreshold(TEXT("p.RigidBodyLODThreshold"), -1, TEXT("Max LOD that rigid body node is allowed to run on. Provides a global threshold that overrides per-node the LODThreshold property. -1 means no override."), ECVF_Scalability);

int32 RBAN_MaxSubSteps = 4;
bool bRBAN_EnableTimeBasedReset = true;
bool bRBAN_EnableComponentAcceleration = true;
int32 RBAN_WorldObjectExpiry = 4;
FAutoConsoleVariableRef CVarRigidBodyNodeMaxSteps(TEXT("p.RigidBodyNode.MaxSubSteps"), RBAN_MaxSubSteps, TEXT("Set the maximum number of simulation steps in the update loop"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeEnableTimeBasedReset(TEXT("p.RigidBodyNode.EnableTimeBasedReset"), bRBAN_EnableTimeBasedReset, TEXT("If true, Rigid Body nodes are reset when they have not been updated for a while (default true)"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeEnableComponentAcceleration(TEXT("p.RigidBodyNode.EnableComponentAcceleration"), bRBAN_EnableComponentAcceleration, TEXT("Enable/Disable the simple acceleration transfer system for component- or bone-space simulation"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeWorldObjectExpiry(TEXT("p.RigidBodyNode.WorldObjectExpiry"), RBAN_WorldObjectExpiry, TEXT("World objects are removed from the simulation if not detected after this many tests"), ECVF_Default);

bool bRBAN_IncludeClothColliders = true;
FAutoConsoleVariableRef CVarRigidBodyNodeIncludeClothColliders(TEXT("p.RigidBodyNode.IncludeClothColliders"), bRBAN_IncludeClothColliders, TEXT("Include cloth colliders as kinematic bodies in the immediate physics simulation."), ECVF_Default);

TAutoConsoleVariable<bool> CVarEnableRigidBodyNodePoseScalePropagation(TEXT("p.RigidBodyNode.EnablePoseScalePropagation"), false, TEXT("Runtime Enable/Disable applying bone transform scales to the output pose (after simulation)."), ECVF_Default);

// FSimSpaceSettings forced overrides for testing
bool bRBAN_SimSpace_EnableOverride = false;
float RBAN_SimSpaceOverride_WorldAlpha = -1.0f;
float RBAN_SimSpaceOverride_VelocityScaleZ = -1.0f;
float RBAN_SimSpaceOverride_DampingAlpha = -1.0f;
float RBAN_SimSpaceOverride_MaxLinearVelocity = -1.0f;
float RBAN_SimSpaceOverride_MaxAngularVelocity = -1.0f;
float RBAN_SimSpaceOverride_MaxLinearAcceleration = -1.0f;
float RBAN_SimSpaceOverride_MaxAngularAcceleration = -1.0f;
FAutoConsoleVariableRef CVarRigidBodyNodeSpaceOverride(TEXT("p.RigidBodyNode.Space.Override"), bRBAN_SimSpace_EnableOverride, TEXT("Force-enable the advanced simulation space movement forces"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeSpaceWorldAlpha(TEXT("p.RigidBodyNode.Space.WorldAlpha"), RBAN_SimSpaceOverride_WorldAlpha, TEXT("RBAN SimSpaceSettings overrides"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeSpaceVelScaleZ(TEXT("p.RigidBodyNode.Space.VelocityScaleZ"), RBAN_SimSpaceOverride_VelocityScaleZ, TEXT("RBAN SimSpaceSettings overrides"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeSpaceDampingAlpha(TEXT("p.RigidBodyNode.Space.DampingAlpha"), RBAN_SimSpaceOverride_DampingAlpha, TEXT("RBAN SimSpaceSettings overrides"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeSpaceMaxCompLinVel(TEXT("p.RigidBodyNode.Space.MaxLinearVelocity"), RBAN_SimSpaceOverride_MaxLinearVelocity, TEXT("RBAN SimSpaceSettings overrides"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeSpaceMaxCompAngVel(TEXT("p.RigidBodyNode.Space.MaxAngularVelocity"), RBAN_SimSpaceOverride_MaxAngularVelocity, TEXT("RBAN SimSpaceSettings overrides"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeSpaceMaxCompLinAcc(TEXT("p.RigidBodyNode.Space.MaxLinearAcceleration"), RBAN_SimSpaceOverride_MaxLinearAcceleration, TEXT("RBAN SimSpaceSettings overrides"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeSpaceMaxCompAngAcc(TEXT("p.RigidBodyNode.Space.MaxAngularAcceleration"), RBAN_SimSpaceOverride_MaxAngularAcceleration, TEXT("RBAN SimSpaceSettings overrides"), ECVF_Default);
float RBAN_Override_ComponentLinearAccScale = -1.0f;
float RBAN_Override_ComponentLinearVelScale = -1.0f;
float RBAN_Override_ComponentAppliedLinearAccClamp = -1.0f;
FAutoConsoleVariableRef CVarRigidBodyNodeOverrideComponentLinearAccScale(TEXT("p.RigidBodyNode.ComponentLinearAccScale"), RBAN_Override_ComponentLinearAccScale, TEXT("ComponentLinearAccScale override"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeOverrideComponentLinearVelScale(TEXT("p.RigidBodyNode.ComponentLinearVelcale"), RBAN_Override_ComponentLinearVelScale, TEXT("ComponentLinearVelcale override"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeOverrideComponentAppliedLinearAccClamp(TEXT("p.RigidBodyNode.ComponentAppliedLinearAccClamp"), RBAN_Override_ComponentAppliedLinearAccClamp, TEXT("ComponentAppliedLinearAccClamp override"), ECVF_Default);
float RBAN_GravityScale = 1.0f;
FAutoConsoleVariableRef CVarRigidBodyNodeGravityScale(TEXT("p.RigidBodyNode.GravityScale"), RBAN_GravityScale, TEXT("Multiplies the gravity on all RBANs"), ECVF_Default);

bool bRBAN_DeferredSimulationDefault = false;
FAutoConsoleVariableRef CVarRigidBodyNodeDeferredSimulationDefault(
	TEXT("p.RigidBodyNode.DeferredSimulationDefault"),
	bRBAN_DeferredSimulationDefault,
	TEXT("Whether rigid body simulations are deferred one frame for assets that don't opt into a specific simulation timing"),
	ECVF_Default);

bool bRBAN_DeferredSimulationForceDefault = false;
FAutoConsoleVariableRef CVarRigidBodyNodeDeferredSimulationForceDefault(TEXT("p.RigidBodyNode.DeferredSimulationForceDefault"), bRBAN_DeferredSimulationForceDefault, TEXT("When true, rigid body simulation will always use the value of p.RigidBodyNode.DeferredSimulationDefault to determine whether to defer the simulation work, ignoring the setting in the individual node."), ECVF_Default);

bool bRBAN_DebugDraw = false;
FAutoConsoleVariableRef CVarRigidBodyNodeDebugDraw(TEXT("p.RigidBodyNode.DebugDraw"), bRBAN_DebugDraw, TEXT("Whether to debug draw the rigid body simulation state. Requires p.Chaos.DebugDraw.Enabled 1 to function as well."), ECVF_Default);

// Temporary to avoid out of bounds access issue
bool bRBAN_InitializeBoneReferencesRangeCheckEnabled = true;
FAutoConsoleVariableRef CVarRigidBodyNodeInitializeBoneReferencesRangeCheckEnabled(TEXT("p.RigidBodyNode.InitializeBoneReferencesRangeCheckEnabled"), bRBAN_InitializeBoneReferencesRangeCheckEnabled, TEXT(""), ECVF_Default);

#if ENABLE_LOW_LEVEL_MEM_TRACKER
// This is used for memory tagging purposes
static FName GRBANClassFName("AnimNode_RigidBody");
#endif

// Array of priorities that can be indexed into with CVars, since task priorities cannot be set from scalability .ini
static UE::Tasks::ETaskPriority GRigidBodyNodeTaskPriorities[] =
{
	UE::Tasks::ETaskPriority::High,
	UE::Tasks::ETaskPriority::Normal,
	UE::Tasks::ETaskPriority::BackgroundHigh,
	UE::Tasks::ETaskPriority::BackgroundNormal,
	UE::Tasks::ETaskPriority::BackgroundLow
};

static int32 GRigidBodyNodeSimulationTaskPriority = 0;
FAutoConsoleVariableRef CVarRigidBodyNodeSimulationTaskPriority(
	TEXT("p.RigidBodyNode.TaskPriority.Simulation"),
	GRigidBodyNodeSimulationTaskPriority,
	TEXT("Task priority for running the rigid body node simulation task (0 = foreground/high, 1 = foreground/normal, 2 = background/high, 3 = background/normal, 4 = background/low)."),
	ECVF_Default
);

// This is to validate our declaration of TIsPODType in the header, which
// was done to ensure that STRUCT_IsPlainOldData is set, which allows scripts
// and reflection based clients to copy via memcpy:
static_assert(std::is_trivially_copyable<FSimSpaceSettings>::value);

FSimSpaceSettings::FSimSpaceSettings()
	: WorldAlpha(0)
	, VelocityScaleZ(1)
	, DampingAlpha(1)
	, MaxLinearVelocity(10000)
	, MaxAngularVelocity(10000)
	, MaxLinearAcceleration(10000)
	, MaxAngularAcceleration(10000)
#if WITH_EDITORONLY_DATA
	, ExternalLinearDrag_DEPRECATED(0)
#endif
	, ExternalLinearDragV(FVector::ZeroVector)
	, ExternalLinearVelocity(FVector::ZeroVector)
	, ExternalAngularVelocity(FVector::ZeroVector)
{
}

void FSimSpaceSettings::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		if (ExternalLinearDrag_DEPRECATED != 0.0f)
		{
			ExternalLinearDragV = FVector(ExternalLinearDrag_DEPRECATED, ExternalLinearDrag_DEPRECATED, ExternalLinearDrag_DEPRECATED);
		}
	}
#endif
}


FAnimNode_RigidBody::FAnimNode_RigidBody()
	: OverridePhysicsAsset(nullptr)
	, PreviousCompWorldSpaceTM()
	, CurrentTransform()
	, PreviousTransform()
	, UsePhysicsAsset(nullptr)
	, OverrideWorldGravity(0.0f)
	, ExternalForce(0.0f)
	, ComponentLinearAccScale(0.0f)
	, ComponentLinearVelScale(0.0f)
	, ComponentAppliedLinearAccClamp(10000.0f)
	, SimSpaceSettings()
	, CachedBoundsScale(1.2f)
	, BaseBoneRef()
	, OverlapChannel(ECC_WorldStatic)
	, SimulationSpace(ESimulationSpace::ComponentSpace)
	, bForceDisableCollisionBetweenConstraintBodies(false)
	, bUseExternalClothCollision(false)
	, ResetSimulatedTeleportType(ETeleportType::None)
	, bEnableWorldGeometry(false)
	, bOverrideWorldGravity(false)
	, bTransferBoneVelocities(false)
	, bFreezeIncomingPoseOnStart(false)
	, bClampLinearTranslationLimitToRefPose(false)
	, bAllowPoseScalePropagation(true)
	, WorldSpaceMinimumScale(0.01f)
	, EvaluationResetTime(0.01f)
	, bEnabled(false)
	, bSimulationStarted(false)
	, bCheckForBodyTransformInit(false)
#if WITH_EDITORONLY_DATA
	, bComponentSpaceSimulation_DEPRECATED(true)
#endif
	, SimulationTiming(ESimulationTiming::Default)
	, WorldTimeSeconds(0.0)
	, LastEvalTimeSeconds(0.0)
	, AccumulatedDeltaTime(0.0f)
	, AnimPhysicsMinDeltaTime(0.0f)
	, bSimulateAnimPhysicsAfterReset(false)
	, SkelMeshCompWeakPtr()
	, PhysicsSimulation(nullptr)
	, SolverSettings()
	, SolverIterations()
	, SimulationTask()
	, OutputBoneData()
	, Bodies()
	, SkeletonBoneIndexToBodyIndex()
	, BodyAnimData()
	, Constraints()
	, PendingRadialForces()
	, PerSolverField()
	, ComponentsInSim()
	, ComponentsInSimTick(0)
	, WorldSpaceGravity(0.0f)
	, TotalMass(0.0)
	, CachedBounds(FVector::ZeroVector, 0.0f)
	, QueryParams(NAME_None, FCollisionQueryParams::GetUnknownStatId())
	, PhysScene(nullptr)
	, UnsafeWorld(nullptr)
	, UnsafeOwner(nullptr)
	, CapturedBoneVelocityBoneContainer()
	, CapturedBoneVelocityPose()
	, CapturedFrozenPose()
	, CapturedFrozenCurves()
	, PreviousComponentLinearVelocity(0.0f)
	, PreviousSimulationSpaceTransform()
	, PreviousPreviousSimulationSpaceTransform()
	, PreviousDt(1.0f/30.0f)
{
}

FAnimNode_RigidBody::~FAnimNode_RigidBody()
{
	DestroyPhysicsSimulation();
}

void FAnimNode_RigidBody::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	DebugLine += "(";
	AddDebugNodeData(DebugLine);
	DebugLine += ")";

	DebugData.AddDebugItem(DebugLine);

	const bool bUsingFrozenPose = bFreezeIncomingPoseOnStart && bSimulationStarted && (CapturedFrozenPose.GetPose().GetNumBones() > 0);
	if (!bUsingFrozenPose)
	{
		ComponentPose.GatherDebugData(DebugData);
	}
}

void FAnimNode_RigidBody::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	Super::Initialize_AnyThread(Context);

#if WITH_EDITOR
	if(GIsReinstancing)
	{
		InitPhysics(Cast<UAnimInstance>(Context.GetAnimInstanceObject()));
	}
#endif
}

void FAnimNode_RigidBody::UpdateComponentPose_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(UpdateComponentPose_AnyThread)
	// Only freeze update graph after initial update, as we want to get that pose through.
	if (bFreezeIncomingPoseOnStart && bSimulationStarted && ResetSimulatedTeleportType == ETeleportType::None)
	{
		// If we have a Frozen Pose captured, 
		// then we don't need to update the rest of the graph.
		if (CapturedFrozenPose.GetPose().GetNumBones() > 0)
		{
		}
		else
		{
			// Create a new context with zero deltatime to freeze time in rest of the graph.
			// This will be used to capture a frozen pose.
			FAnimationUpdateContext FrozenContext = Context.FractionalWeightAndTime(1.f, 0.f);

			Super::UpdateComponentPose_AnyThread(FrozenContext);
		}
	}
	else
	{
		Super::UpdateComponentPose_AnyThread(Context);
	}
}

void FAnimNode_RigidBody::EvaluateComponentPose_AnyThread(FComponentSpacePoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateComponentPose_AnyThread)
	if (bFreezeIncomingPoseOnStart && bSimulationStarted)
	{
		// If we have a Frozen Pose captured, use it.
		// Only after our intialize setup. As we need new pose for that.
		if (ResetSimulatedTeleportType == ETeleportType::None && (CapturedFrozenPose.GetPose().GetNumBones() > 0))
		{
			Output.Pose.CopyPose(CapturedFrozenPose);
			Output.Curve.CopyFrom(CapturedFrozenCurves);
		}
		// Otherwise eval graph to capture it.
		else
		{
			Super::EvaluateComponentPose_AnyThread(Output);
			CapturedFrozenPose.CopyPose(Output.Pose);
			CapturedFrozenCurves.CopyFrom(Output.Curve);
		}
	}
	else
	{
		Super::EvaluateComponentPose_AnyThread(Output);
	}

	// Capture incoming pose if 'bTransferBoneVelocities' is set.
	// That is, until simulation starts.
	if (bTransferBoneVelocities && !bSimulationStarted)
	{
		CapturedBoneVelocityPose.CopyPose(Output.Pose);
		CapturedBoneVelocityPose.CopyAndAssignBoneContainer(CapturedBoneVelocityBoneContainer);
	}
}

void FAnimNode_RigidBody::InitializeNewBodyTransformsDuringSimulation(FComponentSpacePoseContext& Output, const FTransform& ComponentTransform, const FTransform& BaseBoneTM)
{
	for (const FOutputBoneData& OutputData : OutputBoneData)
	{
		const int32 BodyIndex = OutputData.BodyIndex;
		FBodyAnimData& BodyData = BodyAnimData[BodyIndex];
		if (!BodyData.bBodyTransformInitialized)
		{
			BodyData.bBodyTransformInitialized = true;

			// If we have a parent body, we need to grab relative transforms to it.
			if (OutputData.ParentBodyIndex != INDEX_NONE)
			{
				ensure(BodyAnimData[OutputData.ParentBodyIndex].bBodyTransformInitialized);

				FTransform BodyRelativeTransform = FTransform::Identity;
				for (const FCompactPoseBoneIndex& CompactBoneIndex : OutputData.BoneIndicesToParentBody)
				{
					const FTransform& LocalSpaceTM = Output.Pose.GetLocalSpaceTransform(CompactBoneIndex);
					BodyRelativeTransform = BodyRelativeTransform * LocalSpaceTM;
				}

				const FTransform WSBodyTM = BodyRelativeTransform * Bodies[OutputData.ParentBodyIndex]->GetWorldTransform();
				Bodies[BodyIndex]->InitWorldTransform(WSBodyTM);
				BodyAnimData[BodyIndex].RefPoseLength = static_cast<float>(BodyRelativeTransform.GetLocation().Size());
			}
			// If we don't have a parent body, then we can just grab the incoming pose in component space.
			else
			{
				const FTransform& ComponentSpaceTM = Output.Pose.GetComponentSpaceTransform(OutputData.CompactPoseBoneIndex);
				const FTransform BodyTM = ConvertCSTransformToSimSpace(SimulationSpace, ComponentSpaceTM, ComponentTransform, BaseBoneTM);

				Bodies[BodyIndex]->InitWorldTransform(BodyTM);
			}
		}
	}
}

void FAnimNode_RigidBody::InitSimulationSpace(
	const FTransform& ComponentToWorld,
	const FTransform& BoneToComponent)
{
	FTransform SimulationSpaceTransform = SpaceToWorldTransform(SimulationSpace, ComponentToWorld, BoneToComponent);
	PreviousSimulationSpaceTransform = SimulationSpaceTransform;
	PreviousPreviousSimulationSpaceTransform = SimulationSpaceTransform;
	PreviousDt = 1.0f / 30.0f;
}

void FAnimNode_RigidBody::CalculateSimulationSpaceMotion(
	ESimulationSpace Space, 
	const FTransform& SimulationSpaceTransform,
	const FTransform& ComponentToWorld,
	const float Dt,
	const FSimSpaceSettings& Settings,
	FVector& SpaceLinearVel, 
	FVector& SpaceAngularVel, 
	FVector& SpaceLinearAcc, 
	FVector& SpaceAngularAcc)
{
	SpaceLinearVel = FVector::ZeroVector;
	SpaceAngularVel = FVector::ZeroVector;
	SpaceLinearAcc = FVector::ZeroVector;
	SpaceAngularAcc = FVector::ZeroVector;

	// If the system is disabled, nothing else to do
	if ((Settings.WorldAlpha == 0.0f) || (Dt < UE_SMALL_NUMBER))
	{
		return;
	}

	// If the simulation is in world space, the simulation space is stationary
	if (Space == ESimulationSpace::WorldSpace)
	{
		SpaceLinearVel = Settings.ExternalLinearVelocity;
		SpaceAngularVel = Settings.ExternalAngularVelocity;
		return;
	}

	// World-space component linear velocity and acceleration
	FVector PrevSpaceLinearVel = Chaos::FVec3::CalculateVelocity(PreviousPreviousSimulationSpaceTransform.GetTranslation(), PreviousSimulationSpaceTransform.GetTranslation(), PreviousDt);
	SpaceLinearVel = Chaos::FVec3::CalculateVelocity(PreviousSimulationSpaceTransform.GetTranslation(), SimulationSpaceTransform.GetTranslation(), Dt);
	SpaceLinearAcc = (SpaceLinearVel - PrevSpaceLinearVel) / Dt;

	// World-space component angular velocity and acceleration
	FVector PrevSpaceAngularVel = Chaos::FRotation3::CalculateAngularVelocity(PreviousPreviousSimulationSpaceTransform.GetRotation(), PreviousSimulationSpaceTransform.GetRotation(), PreviousDt);
	SpaceAngularVel = Chaos::FRotation3::CalculateAngularVelocity(PreviousSimulationSpaceTransform.GetRotation(), SimulationSpaceTransform.GetRotation(), Dt);
	SpaceAngularAcc = (SpaceAngularVel - PrevSpaceAngularVel) / Dt;

	// Apply Z scale
	SpaceLinearVel.Z *= Settings.VelocityScaleZ;
	SpaceLinearAcc.Z *= Settings.VelocityScaleZ;

	// Clamped world-space motion of the simulation space
	SpaceLinearVel = SpaceLinearVel.GetClampedToMaxSize(Settings.MaxLinearVelocity) + Settings.ExternalLinearVelocity;
	SpaceAngularVel = SpaceAngularVel.GetClampedToMaxSize(Settings.MaxAngularVelocity) + Settings.ExternalAngularVelocity;
	SpaceLinearAcc = SpaceLinearAcc.GetClampedToMaxSize(Settings.MaxLinearAcceleration);
	SpaceAngularAcc = SpaceAngularAcc.GetClampedToMaxSize(Settings.MaxAngularAcceleration);

	// Transform world-space motion into simulation space
	SpaceLinearVel = SimulationSpaceTransform.InverseTransformVector(SpaceLinearVel);
	SpaceAngularVel = SimulationSpaceTransform.InverseTransformVectorNoScale(SpaceAngularVel);
	SpaceLinearAcc = SimulationSpaceTransform.InverseTransformVector(SpaceLinearAcc);
	SpaceAngularAcc = SimulationSpaceTransform.InverseTransformVectorNoScale(SpaceAngularAcc);

	// Apply WorldAlpha to simulation space motion (usually to reduce it)
	SpaceLinearVel *= Settings.WorldAlpha;
	SpaceLinearAcc *= Settings.WorldAlpha;
	SpaceAngularVel *= Settings.WorldAlpha;
	SpaceAngularAcc *= Settings.WorldAlpha;

	PreviousPreviousSimulationSpaceTransform = PreviousSimulationSpaceTransform;
	PreviousSimulationSpaceTransform = SimulationSpaceTransform;
	PreviousDt = Dt;
}


DECLARE_CYCLE_STAT(TEXT("RigidBody_Eval"), STAT_RigidBody_Eval, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("RigidBodyNode_Simulation"), STAT_RigidBodyNode_Simulation, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("RigidBodyNode_SimulationWait"), STAT_RigidBodyNode_SimulationWait, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("FAnimNode_RigidBody::EvaluateSkeletalControl_AnyThread"), STAT_ImmediateEvaluateSkeletalControl, STATGROUP_ImmediatePhysics);

void FAnimNode_RigidBody::RunPhysicsSimulation(float DeltaSeconds, const FVector& SimSpaceGravity)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigidBody"));
	SCOPE_CYCLE_COUNTER(STAT_RigidBodyNode_Simulation);
	CSV_SCOPED_TIMING_STAT(Animation, RigidBodyNodeSimulation);

#if ENABLE_LOW_LEVEL_MEM_TRACKER
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(OwningAssetPackageName, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(GRBANClassFName, ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(OwningAssetName, GRBANClassFName, OwningAssetPackageName);
#endif

	FScopeCycleCounterUObject AdditionalScope(UsePhysicsAsset, GET_STATID(STAT_RigidBodyNode_Simulation));

	const int32 MaxSteps = RBAN_MaxSubSteps;
	const float MaxDeltaSeconds = 1.f / 30.f;

	PhysicsSimulation->Simulate_AssumesLocked(DeltaSeconds, MaxDeltaSeconds, MaxSteps, SimSpaceGravity);
}

void FAnimNode_RigidBody::FlushDeferredSimulationTask()
{
	if (SimulationTask.IsValid() && !SimulationTask.IsCompleted())
	{
		SCOPE_CYCLE_COUNTER(STAT_RigidBodyNode_SimulationWait);
		CSV_SCOPED_TIMING_STAT(Animation, RigidBodyNodeSimulationWait);
		SimulationTask.Wait();
	}
}

void FAnimNode_RigidBody::DestroyPhysicsSimulation()
{
	ClothColliders.Reset();
	FlushDeferredSimulationTask();
	delete PhysicsSimulation;
	PhysicsSimulation = nullptr;
}

void FAnimNode_RigidBody::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigidBody"));
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	SCOPE_CYCLE_COUNTER(STAT_RigidBody_Eval);
	CSV_SCOPED_TIMING_STAT(Animation, RigidBodyEval);
	SCOPE_CYCLE_COUNTER(STAT_ImmediateEvaluateSkeletalControl);

#if ENABLE_LOW_LEVEL_MEM_TRACKER
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(OwningAssetPackageName, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(GRBANClassFName, ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(OwningAssetName, GRBANClassFName, OwningAssetPackageName);
#endif

	//SCOPED_NAMED_EVENT_TEXT("FAnimNode_RigidBody::EvaluateSkeletalControl_AnyThread", FColor::Magenta);

	if (CVarEnableRigidBodyNodeSimulation.GetValueOnAnyThread() == 0)
	{
		return;
	}

	const float DeltaSeconds = AccumulatedDeltaTime;
	AccumulatedDeltaTime = 0.f;

	if (bEnabled && PhysicsSimulation)	
	{
		FlushDeferredSimulationTask();

		const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();
		const FTransform CompWorldSpaceTM = Output.AnimInstanceProxy->GetComponentTransform();

		bool bFirstEvalSinceReset = !Output.AnimInstanceProxy->GetEvaluationCounter().HasEverBeenUpdated();

		// First-frame initialization
		if (bFirstEvalSinceReset)
		{
			PreviousCompWorldSpaceTM = CompWorldSpaceTM;
			ResetSimulatedTeleportType = ETeleportType::ResetPhysics;
		}

		// See if we need to reset physics because too much time passed since our last update (e.g., because we we off-screen for a while), 
		// in which case the current sim state may be too far from the current anim pose. This is mostly a problem with world-space 
		// simulation, whereas bone- and component-space sims can be fairly robust against missing updates.
		// Don't do this on first frame or if time-based reset is disabled. 
		if ((EvaluationResetTime > 0.0f) && !bFirstEvalSinceReset)
		{
			// NOTE: under normal conditions, when this anim node is being serviced at the usual rate (which may not be every frame
			// if URO is enabled), we expect that WorldTimeSeconds == (LastEvalTimeSeconds + DeltaSeconds). DeltaSeconds is the 
			// accumulated time since the last update, including frames dropped by URO, but not frames dropped because of
			// being off-screen or LOD changes.
			if (WorldTimeSeconds - (LastEvalTimeSeconds + DeltaSeconds) > EvaluationResetTime)
			{
				UE_LOGF(LogRBAN, Verbose, "%ls Time-Based Reset", *Output.AnimInstanceProxy->GetAnimInstanceName());
				ResetSimulatedTeleportType = ETeleportType::ResetPhysics;
			}
		}

		// Update the evaluation time to the current time
		LastEvalTimeSeconds = WorldTimeSeconds;

		// Disable simulation below minimum scale in world space mode. World space sim doesn't play nice with scale anyway - we do not scale joint offets or collision shapes.
		if ((SimulationSpace == ESimulationSpace::WorldSpace) && (CompWorldSpaceTM.GetScale3D().SizeSquared() < WorldSpaceMinimumScale * WorldSpaceMinimumScale))
		{
			return;
		}

		bool bApplyBoneTransformScaleToOutputPose = bAllowPoseScalePropagation && CVarEnableRigidBodyNodePoseScalePropagation.GetValueOnAnyThread();

		// Remove any scaling from BaseBoneTM if we are intending to apply the bone transform scale to the output pose. This prevents the bone transform scale
		// being applied to the physics simulation which means that when the scale is applied to the output pose it has not effectivly been applied twice.
		const FTransform& BaseBoneComponentSpaceTM = Output.Pose.GetComponentSpaceTransform(BaseBoneRef.GetCompactPoseIndex(BoneContainer));
		const FTransform BaseBoneTM(BaseBoneComponentSpaceTM.GetRotation(), BaseBoneComponentSpaceTM.GetTranslation(), bApplyBoneTransformScaleToOutputPose ? FVector(1.0f) : BaseBoneComponentSpaceTM.GetScale3D());

		// Initialize potential new bodies because of LOD change.
		if (ResetSimulatedTeleportType == ETeleportType::None && bCheckForBodyTransformInit)
		{
			bCheckForBodyTransformInit = false;
			InitializeNewBodyTransformsDuringSimulation(Output, CompWorldSpaceTM, BaseBoneTM);
		}

		// If time advances, update simulation
		// Reset if necessary
		const bool bResetOrTeleportBodies = (ResetSimulatedTeleportType != ETeleportType::None);
		if (bResetOrTeleportBodies)
		{
			// Capture bone velocities if we have captured a bone velocity pose.
			if (bTransferBoneVelocities && (CapturedBoneVelocityPose.GetPose().GetNumBones() > 0))
			{
				for (const FOutputBoneData& OutputData : OutputBoneData)
				{
					const int32 BodyIndex = OutputData.BodyIndex;
					FBodyAnimData& BodyData = BodyAnimData[BodyIndex];

					if (BodyData.bIsSimulated)
					{
						const FCompactPoseBoneIndex NextCompactPoseBoneIndex = OutputData.CompactPoseBoneIndex;
						// Convert CompactPoseBoneIndex to SkeletonBoneIndex...
						const FSkeletonPoseBoneIndex PoseSkeletonBoneIndex = BoneContainer.GetSkeletonPoseIndexFromCompactPoseIndex(NextCompactPoseBoneIndex);
						// ... So we can convert to the captured pose CompactPoseBoneIndex. 
						// In case there was a LOD change, and poses are not compatible anymore.
						const FCompactPoseBoneIndex PrevCompactPoseBoneIndex = CapturedBoneVelocityBoneContainer.GetCompactPoseIndexFromSkeletonPoseIndex(PoseSkeletonBoneIndex);

						if (PrevCompactPoseBoneIndex.IsValid())
						{
							const FTransform PrevCSTM = CapturedBoneVelocityPose.GetComponentSpaceTransform(PrevCompactPoseBoneIndex);
							const FTransform NextCSTM = Output.Pose.GetComponentSpaceTransform(NextCompactPoseBoneIndex);

							const FTransform PrevSSTM = ConvertCSTransformToSimSpace(SimulationSpace, PrevCSTM, CompWorldSpaceTM, BaseBoneTM);
							const FTransform NextSSTM = ConvertCSTransformToSimSpace(SimulationSpace, NextCSTM, CompWorldSpaceTM, BaseBoneTM);

							if(DeltaSeconds > 0.0f)
							{
								// Linear Velocity
								BodyData.TransferedBoneLinearVelocity = ((NextSSTM.GetLocation() - PrevSSTM.GetLocation()) / DeltaSeconds);
								
								// Angular Velocity
								const FQuat DeltaRotation = (NextSSTM.GetRotation().Inverse() * PrevSSTM.GetRotation());
								const double RotationAngle = DeltaRotation.GetAngle() / DeltaSeconds;
								BodyData.TransferedBoneAngularVelocity = (FQuat(DeltaRotation.GetRotationAxis(), RotationAngle)); 
							}
							else
							{
								BodyData.TransferedBoneLinearVelocity = (FVector::ZeroVector);
								BodyData.TransferedBoneAngularVelocity = (FQuat::Identity); 
							}

						}
					}
				}
			}

			
			switch(ResetSimulatedTeleportType)
			{
				case ETeleportType::TeleportPhysics:
				{
					UE_LOGF(LogRBAN, Verbose, "%ls TeleportPhysics (Scale: %f %f %f)", *Output.AnimInstanceProxy->GetAnimInstanceName(), CompWorldSpaceTM.GetScale3D().X, CompWorldSpaceTM.GetScale3D().Y, CompWorldSpaceTM.GetScale3D().Z);

					// Teleport bodies.
					for (const FOutputBoneData& OutputData : OutputBoneData)
					{
						const int32 BodyIndex = OutputData.BodyIndex;
						BodyAnimData[BodyIndex].bBodyTransformInitialized = true;

						FTransform BodyTM = Bodies[BodyIndex]->GetWorldTransform();
						FTransform ComponentSpaceTM;

						switch(SimulationSpace)
						{
							case ESimulationSpace::ComponentSpace: ComponentSpaceTM = BodyTM; break;
							case ESimulationSpace::WorldSpace: ComponentSpaceTM = BodyTM.GetRelativeTransform(PreviousCompWorldSpaceTM); break;
							case ESimulationSpace::BaseBoneSpace: ComponentSpaceTM = BodyTM * BaseBoneTM; break;
							default: ensureMsgf(false, TEXT("Unsupported Simulation Space")); ComponentSpaceTM = BodyTM;
						}

						BodyTM = ConvertCSTransformToSimSpace(SimulationSpace, ComponentSpaceTM, CompWorldSpaceTM, BaseBoneTM);
						Bodies[BodyIndex]->SetWorldTransform(BodyTM);
						if (OutputData.ParentBodyIndex != INDEX_NONE)
						{
							BodyAnimData[BodyIndex].RefPoseLength = static_cast<float>(BodyTM.GetRelativeTransform(Bodies[OutputData.ParentBodyIndex]->GetWorldTransform()).GetLocation().Size());
						}
					}
				}
				break;

				case ETeleportType::ResetPhysics:
				{
					UE_LOGF(LogRBAN, Verbose, "%ls ResetPhysics (Scale: %f %f %f)", *Output.AnimInstanceProxy->GetAnimInstanceName(), CompWorldSpaceTM.GetScale3D().X, CompWorldSpaceTM.GetScale3D().Y, CompWorldSpaceTM.GetScale3D().Z);

					InitSimulationSpace(CompWorldSpaceTM, BaseBoneTM);

					// Completely reset bodies.
					for (const FOutputBoneData& OutputData : OutputBoneData)
					{
						const int32 BodyIndex = OutputData.BodyIndex;
						BodyAnimData[BodyIndex].bBodyTransformInitialized = true;

						const FTransform& ComponentSpaceTM = Output.Pose.GetComponentSpaceTransform(OutputData.CompactPoseBoneIndex);
						const FTransform BodyTM = ConvertCSTransformToSimSpace(SimulationSpace, ComponentSpaceTM, CompWorldSpaceTM, BaseBoneTM);
						Bodies[BodyIndex]->InitWorldTransform(BodyTM);
						if (OutputData.ParentBodyIndex != INDEX_NONE)
						{
							BodyAnimData[BodyIndex].RefPoseLength = static_cast<float>(BodyTM.GetRelativeTransform(Bodies[OutputData.ParentBodyIndex]->GetWorldTransform()).GetLocation().Size());
						}
					}
				}
				break;
			}

			// Always reset after a teleport
			PreviousCompWorldSpaceTM = CompWorldSpaceTM;
			PreviousComponentLinearVelocity = FVector::ZeroVector;
		}

		// Assets can override config for deferred simulation
		const bool bUseDeferredSimulationTask =
			((SimulationTiming == ESimulationTiming::Default) || bRBAN_DeferredSimulationForceDefault) ? bRBAN_DeferredSimulationDefault : (SimulationTiming == ESimulationTiming::Deferred);

		FVector SimSpaceGravity(0.f);

		// Only need to tick physics if we didn't reset and we have some time to simulate
		const bool bNeedsSimulationTick = ((bSimulateAnimPhysicsAfterReset || (ResetSimulatedTeleportType != ETeleportType::ResetPhysics)) && DeltaSeconds > AnimPhysicsMinDeltaTime);
		if (bNeedsSimulationTick)
		{
			// Transfer bone velocities previously captured.
			if (bTransferBoneVelocities && (CapturedBoneVelocityPose.GetPose().GetNumBones() > 0))
			{
				for (const FOutputBoneData& OutputData : OutputBoneData)
				{
					const int32 BodyIndex = OutputData.BodyIndex;
					const FBodyAnimData& BodyData = BodyAnimData[BodyIndex];

					if (BodyData.bIsSimulated)
					{
						ImmediatePhysics::FActorHandle* Body = Bodies[BodyIndex];
						Body->SetLinearVelocity(BodyData.TransferedBoneLinearVelocity);

						const FQuat AngularVelocity = BodyData.TransferedBoneAngularVelocity;
						Body->SetAngularVelocity(AngularVelocity.GetRotationAxis() * AngularVelocity.GetAngle());
					}
				}

				// Free up our captured pose after it's been used.
				CapturedBoneVelocityPose.Empty();
			}
			else if ((SimulationSpace != ESimulationSpace::WorldSpace) && bRBAN_EnableComponentAcceleration)
			{
				const FVector UseComponentLinearVelScale = (RBAN_Override_ComponentLinearVelScale >= 0) ? FVector(RBAN_Override_ComponentLinearVelScale) : ComponentLinearVelScale;
				const FVector UseComponentLinearAccScale = (RBAN_Override_ComponentLinearAccScale >= 0) ? FVector(RBAN_Override_ComponentLinearAccScale) : ComponentLinearAccScale;
				const FVector UseComponentAppliedLinearAccClamp = (RBAN_Override_ComponentAppliedLinearAccClamp >= 0) ? FVector(RBAN_Override_ComponentAppliedLinearAccClamp) : ComponentAppliedLinearAccClamp;

				if (!UseComponentLinearVelScale.IsNearlyZero() || !UseComponentLinearAccScale.IsNearlyZero())
				{
					// Calc linear velocity
					const FVector ComponentDeltaLocation = CurrentTransform.GetTranslation() - PreviousTransform.GetTranslation();
					const FVector ComponentLinearVelocity = ComponentDeltaLocation / DeltaSeconds;
					// Apply acceleration that opposed velocity (basically 'drag')
					FVector LinearAccelToApply = WorldVectorToSpaceNoScale(SimulationSpace, -ComponentLinearVelocity, CompWorldSpaceTM, BaseBoneTM) * UseComponentLinearVelScale;

					// Calc linear acceleration
					const FVector ComponentLinearAcceleration = (ComponentLinearVelocity - PreviousComponentLinearVelocity) / DeltaSeconds;
					PreviousComponentLinearVelocity = ComponentLinearVelocity;
					// Apply opposite acceleration to bodies
					LinearAccelToApply += WorldVectorToSpaceNoScale(SimulationSpace, -ComponentLinearAcceleration, CompWorldSpaceTM, BaseBoneTM) * UseComponentLinearAccScale;

					// Clamp if desired
					if (!UseComponentAppliedLinearAccClamp.IsNearlyZero())
					{
						LinearAccelToApply = LinearAccelToApply.BoundToBox(-UseComponentAppliedLinearAccClamp, UseComponentAppliedLinearAccClamp);
					}

					// Iterate over bodies
					for (const FOutputBoneData& OutputData : OutputBoneData)
					{
						const int32 BodyIndex = OutputData.BodyIndex;
						const FBodyAnimData& BodyData = BodyAnimData[BodyIndex];

						if (BodyData.bIsSimulated)
						{
							ImmediatePhysics::FActorHandle* Body = Bodies[BodyIndex];
							const double BodyInvMass = Body->GetInverseMass();
							if (BodyInvMass > 0.0)
							{
								// Apply to body
								Body->AddForce(LinearAccelToApply / BodyInvMass);
							}
						}
					}
				}
			}

			{
				bool bInputPoseContainsScaledTransforms = false;

				for (const FOutputBoneData& OutputData : OutputBoneData)
				{
					const int32 BodyIndex = OutputData.BodyIndex;
					const bool bIsSimulated = BodyAnimData[BodyIndex].bIsSimulated;
				
					if (!bIsSimulated || bApplyBoneTransformScaleToOutputPose)
					{
						const FTransform& ComponentSpaceTM = Output.Pose.GetComponentSpaceTransform(OutputData.CompactPoseBoneIndex);

						if(!bIsSimulated)
						{
							// @todo(ccaulfield): We should be interpolating kinematic targets for each sub-step below
							const FTransform BodyTM = ConvertCSTransformToSimSpace(SimulationSpace, ComponentSpaceTM, CompWorldSpaceTM, BaseBoneTM);
							Bodies[BodyIndex]->SetKinematicTarget(BodyTM);
						}

						// Check for scaling on input pose transforms.
						bInputPoseContainsScaledTransforms |= !ComponentSpaceTM.GetScale3D().Equals(FVector(1.0f), SMALL_NUMBER);
					}	
				}

				// Avoid unnecessary scaling of simulation results when applying them to the output pose if the scale is identity.
				bApplyBoneTransformScaleToOutputPose &= bInputPoseContainsScaledTransforms;
			}

			UpdateWorldForces(CompWorldSpaceTM, BaseBoneTM, DeltaSeconds);
			SimSpaceGravity = WorldVectorToSpaceNoScale(SimulationSpace, WorldSpaceGravity, CompWorldSpaceTM, BaseBoneTM);
			SimSpaceGravity *= RBAN_GravityScale;

			FSimSpaceSettings UseSimSpaceSettings = SimSpaceSettings;
			if (bRBAN_SimSpace_EnableOverride)
			{
				if (RBAN_SimSpaceOverride_WorldAlpha >= 0.0f) UseSimSpaceSettings.WorldAlpha = RBAN_SimSpaceOverride_WorldAlpha;
				if (RBAN_SimSpaceOverride_VelocityScaleZ >= 0.0f) UseSimSpaceSettings.VelocityScaleZ = RBAN_SimSpaceOverride_VelocityScaleZ;
				if (RBAN_SimSpaceOverride_DampingAlpha >= 0.0f) UseSimSpaceSettings.DampingAlpha = RBAN_SimSpaceOverride_DampingAlpha;
				if (RBAN_SimSpaceOverride_MaxLinearVelocity >= 0.0f) UseSimSpaceSettings.MaxLinearVelocity = RBAN_SimSpaceOverride_MaxLinearVelocity;
				if (RBAN_SimSpaceOverride_MaxAngularVelocity >= 0.0f) UseSimSpaceSettings.MaxAngularVelocity = RBAN_SimSpaceOverride_MaxAngularVelocity;
				if (RBAN_SimSpaceOverride_MaxLinearAcceleration >= 0.0f) UseSimSpaceSettings.MaxLinearAcceleration = RBAN_SimSpaceOverride_MaxLinearAcceleration;
				if (RBAN_SimSpaceOverride_MaxAngularAcceleration >= 0.0f) UseSimSpaceSettings.MaxAngularAcceleration = RBAN_SimSpaceOverride_MaxAngularAcceleration;
			}

			const FTransform SpaceTransform = SpaceToWorldTransform(SimulationSpace, CompWorldSpaceTM, BaseBoneTM);
			FVector SimulationSpaceLinearVelocity;
			FVector SimulationSpaceAngularVelocity;
			FVector SimulationSpaceLinearAcceleration;
			FVector SimulationSpaceAngularAcceleration;
			CalculateSimulationSpaceMotion(
				SimulationSpace, 
				SpaceTransform,
				CompWorldSpaceTM,
				DeltaSeconds,
				UseSimSpaceSettings,
				SimulationSpaceLinearVelocity,
				SimulationSpaceAngularVelocity,
				SimulationSpaceLinearAcceleration,
				SimulationSpaceAngularAcceleration);

			UpdateWorldObjects(SpaceTransform);
			UpdateClothColliderObjects(SpaceTransform);

			PhysicsSimulation->UpdateSimulationSpace(
				SpaceTransform,
				SimulationSpaceLinearVelocity,
				SimulationSpaceAngularVelocity,
				SimulationSpaceLinearAcceleration,
				SimulationSpaceAngularAcceleration);

			PhysicsSimulation->SetSimulationSpaceSettings(
				(UseSimSpaceSettings.WorldAlpha > 0.0f),
				UseSimSpaceSettings.DampingAlpha,
				UseSimSpaceSettings.ExternalLinearDragV);

			PhysicsSimulation->SetSolverSettings(
				SolverSettings.FixedTimeStep,
				SolverSettings.CullDistance,
				SolverSettings.MaxDepenetrationVelocity,
				SolverSettings.bUseLinearJointSolver,
				SolverSettings.PositionIterations,
				SolverSettings.VelocityIterations,
				SolverSettings.ProjectionIterations,
				SolverSettings.bUseManifolds);

			Chaos::FCollisionDetectorSettings CollisionDetectorSettings = PhysicsSimulation->GetCollisionDetectorSettings();
			CollisionDetectorSettings.BoundsVelocityInflation = BoundsVelocityMultiplier;
			CollisionDetectorSettings.MaxVelocityBoundsExpansion = MaxVelocityBoundsExpansion;
			CollisionDetectorSettings.bAllowCCD = bAllowCCD;
			PhysicsSimulation->SetCollisionDetectorSettings(CollisionDetectorSettings);

			if (!bUseDeferredSimulationTask)
			{
				RunPhysicsSimulation(DeltaSeconds, SimSpaceGravity);
			}

			// Draw here even if the simulation is deferred since we want the shapes drawn relative to the current transform
			if (bRBAN_DebugDraw)
			{
				PhysicsSimulation->DebugDraw();
			}
		}
		
		TArray<FTransform> ScaledBodyTransforms;

		if(bApplyBoneTransformScaleToOutputPose)
		{
			// Local record of body transforms after the input pose transform scale has been applied.
			// Only used if bApplyBoneTransformScaleToOutputPose is true.
			ScaledBodyTransforms.Init(FTransform::Identity, Bodies.Num());
		}

		//write back to animation system
		const FTransform& SimulationWorldSpaceTM = bUseDeferredSimulationTask ? PreviousCompWorldSpaceTM : CompWorldSpaceTM;
		for (const FOutputBoneData& OutputData : OutputBoneData)
		{
			const int32 BodyIndex = OutputData.BodyIndex;
			if (BodyAnimData[BodyIndex].bIsSimulated)
			{
				FTransform BodyTM = Bodies[BodyIndex]->GetWorldTransform();
				if (ensure(!BodyTM.ContainsNaN()))
				{
					// if we clamp translation, we only do this when all linear translation are locked
					// 
					// @todo(ccaulfield): this shouldn't be required with Chaos - projection should be handling it...
					if (bClampLinearTranslationLimitToRefPose
						&&BodyAnimData[BodyIndex].LinearXMotion == ELinearConstraintMotion::LCM_Locked
						&& BodyAnimData[BodyIndex].LinearYMotion == ELinearConstraintMotion::LCM_Locked
						&& BodyAnimData[BodyIndex].LinearZMotion == ELinearConstraintMotion::LCM_Locked)
					{
						// grab local space of length from ref pose 
						// we have linear limit value - see if that works
						// calculate current local space from parent
						// find parent transform
						const int32 ParentBodyIndex = OutputData.ParentBodyIndex;
						FTransform ParentTransform = FTransform::Identity;
						if (ParentBodyIndex != INDEX_NONE)
						{
							ParentTransform = Bodies[ParentBodyIndex]->GetWorldTransform();
						}

						// get local transform
						FTransform LocalTransform = BodyTM.GetRelativeTransform(ParentTransform);
						const float CurrentLength = static_cast<float>(LocalTransform.GetTranslation().Size());

						// this is inconsistent with constraint. The actual linear limit is set by constraint
						if (!FMath::IsNearlyEqual(CurrentLength, BodyAnimData[BodyIndex].RefPoseLength, KINDA_SMALL_NUMBER))
						{
							float RefPoseLength = BodyAnimData[BodyIndex].RefPoseLength;
							if (CurrentLength > RefPoseLength)
							{
								float Scale = (CurrentLength > KINDA_SMALL_NUMBER) ? RefPoseLength / CurrentLength : 0.f;
								// we don't use 1.f here because 1.f can create pops based on float issue. 
								// so we only activate clamping when less than 90%
								if (Scale < 0.9f)
								{
									LocalTransform.ScaleTranslation(Scale);
									BodyTM = LocalTransform * ParentTransform;
									Bodies[BodyIndex]->SetWorldTransform(BodyTM);
								}
							}
						}
					}

					// Apply scale in input pose to body transforms before writing to the output pose.
					if (bApplyBoneTransformScaleToOutputPose)
					{
						const int32 ParentBodyIndex = OutputData.ParentBodyIndex;
												
						if (ParentBodyIndex != INDEX_NONE)
						{	
							const FTransform ParentTransform = Bodies[ParentBodyIndex]->GetWorldTransform();

							if (!BodyAnimData[ParentBodyIndex].bIsSimulated)
							{
								// BoneIndicesToParentBody contains all indexs up to but not including the parent bone index. The last element should be a direct child of the parent.
								const FCompactPoseBoneIndex& CompactParentBoneIndex = Output.AnimInstanceProxy->GetRequiredBones().GetParentBoneIndex(OutputData.BoneIndicesToParentBody.Last());
	
								ScaledBodyTransforms[ParentBodyIndex] = ParentTransform;
								ScaledBodyTransforms[ParentBodyIndex].SetScale3D(Output.Pose.GetComponentSpaceTransform(CompactParentBoneIndex).GetScale3D());
							}

							BodyTM = BodyTM.GetRelativeTransform(ParentTransform) * ScaledBodyTransforms[ParentBodyIndex];
						}

						ScaledBodyTransforms[BodyIndex] = BodyTM;
					}

					FTransform ComponentSpaceTM;

					switch(SimulationSpace)
					{
						case ESimulationSpace::ComponentSpace: ComponentSpaceTM = BodyTM; break;
						case ESimulationSpace::WorldSpace: ComponentSpaceTM = BodyTM.GetRelativeTransform(SimulationWorldSpaceTM); break;
						case ESimulationSpace::BaseBoneSpace: ComponentSpaceTM = BodyTM * BaseBoneTM; break;
						default: ensureMsgf(false, TEXT("Unsupported Simulation Space")); ComponentSpaceTM = BodyTM;
					}
					
					OutBoneTransforms.Add(FBoneTransform(OutputData.CompactPoseBoneIndex, ComponentSpaceTM));
				}
			}
		}

		// Deferred task must be started after we read actor poses to avoid a race
		if (bNeedsSimulationTick && bUseDeferredSimulationTask)
		{
			// FlushDeferredSimulationTask() should have already ensured task is done.
			ensure(SimulationTask.IsCompleted());
			const int32 PriorityIndex = FMath::Clamp<int32>(GRigidBodyNodeSimulationTaskPriority, 0, UE_ARRAY_COUNT(GRigidBodyNodeTaskPriorities) - 1);
			const UE::Tasks::ETaskPriority TaskPriority = GRigidBodyNodeTaskPriorities[PriorityIndex];
			SimulationTask = UE::Tasks::Launch(
				TEXT("RigidBodyNodeSimulationTask"),
				[this, DeltaSeconds, SimSpaceGravity] { RunPhysicsSimulation(DeltaSeconds, SimSpaceGravity); },
				TaskPriority);
		}

		PreviousCompWorldSpaceTM = CompWorldSpaceTM;
		ResetSimulatedTeleportType = ETeleportType::None;
	}
}

void ComputeBodyInsertionOrder(TArray<FBoneIndexType>& InsertionOrder, const USkeletalMeshComponent& SKC)
{
	//We want to ensure simulated bodies are sorted by LOD so that the first simulated bodies are at the highest LOD.
	//Since LOD2 is a subset of LOD1 which is a subset of LOD0 we can change the number of simulated bodies without any reordering
	//For this to work we must first insert all simulated bodies in the right order. We then insert all the kinematic bodies in the right order

	InsertionOrder.Reset();

	if (SKC.GetSkeletalMeshAsset() == nullptr)
	{
		return;
	}

	const int32 NumLODs = SKC.GetNumLODs();
	if(NumLODs > 0)
	{
		TArray<FBoneIndexType> RequiredBones0;
		TArray<FBoneIndexType> ComponentSpaceTMs0;
		SKC.ComputeRequiredBones(RequiredBones0, ComponentSpaceTMs0, 0, /*bIgnorePhysicsAsset=*/ true);

		TArray<bool> InSortedOrder;
		InSortedOrder.AddZeroed(SKC.GetSkeletalMeshAsset()->GetRefSkeleton().GetNum());

		auto MergeIndices = [&InsertionOrder, &InSortedOrder](const TArray<FBoneIndexType>& RequiredBones) -> void
		{
			for (FBoneIndexType BoneIdx : RequiredBones)
			{
				if (!InSortedOrder[BoneIdx])
				{
					InsertionOrder.Add(BoneIdx);
				}

				InSortedOrder[BoneIdx] = true;
			}
		};


		for(int32 LodIdx = NumLODs - 1; LodIdx > 0; --LodIdx)
		{
			TArray<FBoneIndexType> RequiredBones;
			TArray<FBoneIndexType> ComponentSpaceTMs;
			SKC.ComputeRequiredBones(RequiredBones, ComponentSpaceTMs, LodIdx, /*bIgnorePhysicsAsset=*/ true);
			MergeIndices(RequiredBones);
		}

		MergeIndices(RequiredBones0);
	}
}

UPhysicsAsset* FAnimNode_RigidBody::GetPhysicsAssetToBeUsed(const UAnimInstance* InAnimInstance) const
{
	if (IsValid(OverridePhysicsAsset))
	{
		return ToRawPtr(OverridePhysicsAsset);
	}

	if (InAnimInstance)
	{
		const USkeletalMeshComponent* SkeletalMeshComp = InAnimInstance->GetSkelMeshComponent();
		if (bDefaultToSkeletalMeshPhysicsAsset && SkeletalMeshComp)
		{
			return SkeletalMeshComp->GetPhysicsAsset();
		}
	}

	return nullptr;
}

void FAnimNode_RigidBody::InitPhysics(const UAnimInstance* InAnimInstance)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigidBody")); 
	
	SCOPE_CYCLE_COUNTER(STAT_RigidBodyNodeInitTime);

	DestroyPhysicsSimulation();

	const USkeletalMeshComponent* SkeletalMeshComp = InAnimInstance->GetSkelMeshComponent();
	const USkeletalMesh* SkeletalMeshAsset = SkeletalMeshComp->GetSkeletalMeshAsset();
	USkeleton* SkeletonAsset = InAnimInstance->CurrentSkeleton;

	if(!SkeletalMeshAsset || !SkeletonAsset)
	{
		// Without both the skeleton and the mesh we can't create a new simulation.
		// The previous simulation has just been cleaned up above so we can return early here and not instantiate a new one
		return;
	}

#if ENABLE_LOW_LEVEL_MEM_TRACKER
	OwningAssetPackageName = SkeletalMeshAsset->GetPackage()->GetFName();
	OwningAssetName = SkeletalMeshComp->GetFName();

	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(OwningAssetPackageName, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(GRBANClassFName, ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(OwningAssetName, GRBANClassFName, OwningAssetPackageName);
#endif

	const FReferenceSkeleton& SkelMeshRefSkel = SkeletalMeshAsset->GetRefSkeleton();
	UsePhysicsAsset = GetPhysicsAssetToBeUsed(InAnimInstance);

	ensure(SkeletonAsset == SkeletalMeshAsset->GetSkeleton());

	const FSkeletonToMeshLinkup& LinkupTable = SkeletonAsset->FindOrAddMeshLinkupData(SkeletalMeshAsset);
	const TArray<int32>& MeshToSkeletonBoneIndex = LinkupTable.MeshToSkeletonTable;
	
	const int32 NumSkeletonBones = SkeletonAsset->GetReferenceSkeleton().GetNum();
	SkeletonBoneIndexToBodyIndex.Reset(NumSkeletonBones);
	SkeletonBoneIndexToBodyIndex.Init(INDEX_NONE, NumSkeletonBones);

	PreviousTransform = SkeletalMeshComp->GetComponentToWorld();
	
	RemoveClothColliderObjects();
	
	ComponentsInSim.Reset();
	ComponentsInSimTick = 0;

	if (UPhysicsSettings* Settings = UPhysicsSettings::Get())
	{
		AnimPhysicsMinDeltaTime = Settings->AnimPhysicsMinDeltaTime;
		bSimulateAnimPhysicsAfterReset = Settings->bSimulateAnimPhysicsAfterReset;
	}
	else
	{
		AnimPhysicsMinDeltaTime = 0.f;
		bSimulateAnimPhysicsAfterReset = false;
	}
	
	bEnabled = (UsePhysicsAsset && bEnableRigidBodyNode && SkeletalMeshComp->GetAllowRigidBodyAnimNode());
	if(bEnabled)
	{
		SCOPE_CYCLE_COUNTER(STAT_RigidBodyNodeInitTime_SetupSimulation);

		PhysicsSimulation = new ImmediatePhysics::FSimulation();

#if WITH_CHAOS_VISUAL_DEBUGGER
		PhysicsSimulation->GetChaosVDContextData().Id = FChaosVDRuntimeModule::Get().GenerateUniqueID();
		PhysicsSimulation->GetChaosVDContextData().Type = static_cast<int32>(EChaosVDContextType::Solver);
#endif

#if CHAOS_DEBUG_DRAW
		if ((SkeletalMeshComp->GetWorld() != nullptr) && (SkeletalMeshComp->GetWorld()->GetPhysicsScene() != nullptr))
		{
			const FString DDName = FString::Format(TEXT("RBAN {0}"), { SkeletalMeshComp->GetName()});
			PhysicsSimulation->SetDebugDrawScene(DDName, SkeletalMeshComp->GetWorld()->GetPhysicsScene()->GetDebugDrawScene());
		}
#endif

		const int32 NumBodies = UsePhysicsAsset->SkeletalBodySetups.Num();
		Bodies.Empty(NumBodies);
		BodyAnimData.Reset(NumBodies);
		BodyAnimData.AddDefaulted(NumBodies);
		TotalMass = 0.0;

		// Instantiate a FBodyInstance/FConstraintInstance set that will be cloned into the Immediate Physics sim.
		// NOTE: We do not have a skeleton at the moment, so we have to use the ref pose
		TArray<FBodyInstance*> HighLevelBodyInstances;
		TArray<FConstraintInstance*> HighLevelConstraintInstances;

		// Chaos relies on the initial pose to set up constraint positions
		constexpr bool bCreateBodiesInRefPose = true;
		SkeletalMeshComp->InstantiatePhysicsAssetRefPose(
			*UsePhysicsAsset, 
			SimulationSpace == ESimulationSpace::WorldSpace ? SkeletalMeshComp->GetComponentToWorld().GetScale3D() : FVector(1.f), 
			HighLevelBodyInstances, 
			HighLevelConstraintInstances, 
			nullptr, 
			nullptr, 
			INDEX_NONE, 
			FPhysicsAggregateHandle(),
			bCreateBodiesInRefPose);

		TMap<FName, ImmediatePhysics::FActorHandle*> NamesToHandles;
		TArray<ImmediatePhysics::FActorHandle*> IgnoreCollisionActors;

		TArray<FBoneIndexType> InsertionOrder;
		ComputeBodyInsertionOrder(InsertionOrder, *SkeletalMeshComp);

		// NOTE: NumBonesLOD0 may be less than NumBonesTotal, and it may be middle bones that are missing from LOD0.
		// In this case, LOD0 bone indices may be >= NumBonesLOD0, but always < NumBonesTotal. Arrays indexed by
		// bone index must be size NumBonesTotal.
		const int32 NumBonesLOD0 = InsertionOrder.Num();
		const int32 NumBonesTotal = SkelMeshRefSkel.GetNum();

		// If our skeleton is not the one that was used to build the PhysicsAsset, some bodies may be missing, or rearranged.
		// We need to map the original indices to the new bodies for use by the CollisionDisableTable.
		// NOTE: This array is indexed by the original BodyInstance body index (BodyInstance->InstanceBodyIndex)
		TArray<ImmediatePhysics::FActorHandle*> BodyIndexToActorHandle;
		BodyIndexToActorHandle.AddZeroed(HighLevelBodyInstances.Num());

		TArray<FBodyInstance*> BodiesSorted;
		BodiesSorted.AddZeroed(NumBonesTotal);

		for (FBodyInstance* BI : HighLevelBodyInstances)
		{
			if(BI->IsValidBodyInstance())
			{
				BodiesSorted[BI->InstanceBoneIndex] = BI;
			}
		}

		// Create the immediate physics bodies
		for (FBoneIndexType InsertBone : InsertionOrder)
		{
			if (FBodyInstance* BodyInstance = BodiesSorted[InsertBone])
			{
				UBodySetup* BodySetup = UsePhysicsAsset->SkeletalBodySetups[BodyInstance->InstanceBodyIndex];

				bool bSimulated = (BodySetup->PhysicsType == EPhysicsType::PhysType_Simulated) || (bUseDefaultAsSimulated && BodySetup->PhysicsType == EPhysicsType::PhysType_Default);
				ImmediatePhysics::EActorType ActorType = bSimulated ?  ImmediatePhysics::EActorType::DynamicActor : ImmediatePhysics::EActorType::KinematicActor;
				ImmediatePhysics::FActorHandle* NewBodyHandle = PhysicsSimulation->CreateActor(ImmediatePhysics::MakeActorSetup(ActorType, BodyInstance, BodyInstance->GetUnrealWorldTransform()));
				if (NewBodyHandle)
				{
					NewBodyHandle->SetCCDEnabled(BodySetup->DefaultInstance.bUseCCD);

					if (bSimulated)
					{
						const double InvMass = NewBodyHandle->GetInverseMass();
						TotalMass += InvMass > 0.0 ? 1.0 / InvMass : 0.0;
					}
					const int32 BodyIndex = Bodies.Add(NewBodyHandle);
					const int32 SkeletonBoneIndex = MeshToSkeletonBoneIndex[InsertBone];
					if (ensure(SkeletonBoneIndex >= 0))
					{
						SkeletonBoneIndexToBodyIndex[SkeletonBoneIndex] = BodyIndex;
					}
					BodyAnimData[BodyIndex].bIsSimulated = bSimulated;
					NamesToHandles.Add(BodySetup->BoneName, NewBodyHandle);
					BodyIndexToActorHandle[BodyInstance->InstanceBodyIndex] = NewBodyHandle;

					if (BodySetup->CollisionReponse == EBodyCollisionResponse::BodyCollision_Disabled)
					{
						IgnoreCollisionActors.Add(NewBodyHandle);
					}

					NewBodyHandle->SetName(BodySetup->BoneName);
				}
			}
		}

		//Insert joints so that they coincide body order. That is, if we stop simulating all bodies past some index, we can simply ignore joints past a corresponding index without any re-order
		//For this to work we consider the most last inserted bone in each joint
		TArray<int32> InsertionOrderPerBone;
		InsertionOrderPerBone.AddUninitialized(NumBonesTotal);

		for(int32 Position = 0; Position < NumBonesLOD0; ++Position)
		{
			InsertionOrderPerBone[InsertionOrder[Position]] = Position;
		}

		HighLevelConstraintInstances.Sort([&InsertionOrderPerBone, &SkelMeshRefSkel](const FConstraintInstance& LHS, const FConstraintInstance& RHS)
		{
			if(LHS.IsValidConstraintInstance() && RHS.IsValidConstraintInstance())
			{
				const int32 BoneIdxLHS1 = SkelMeshRefSkel.FindBoneIndex(LHS.ConstraintBone1);
				const int32 BoneIdxLHS2 = SkelMeshRefSkel.FindBoneIndex(LHS.ConstraintBone2);

				const int32 BoneIdxRHS1 = SkelMeshRefSkel.FindBoneIndex(RHS.ConstraintBone1);
				const int32 BoneIdxRHS2 = SkelMeshRefSkel.FindBoneIndex(RHS.ConstraintBone2);

				const int32 MaxPositionLHS = FMath::Max(InsertionOrderPerBone[BoneIdxLHS1], InsertionOrderPerBone[BoneIdxLHS2]);
				const int32 MaxPositionRHS = FMath::Max(InsertionOrderPerBone[BoneIdxRHS1], InsertionOrderPerBone[BoneIdxRHS2]);

				return MaxPositionLHS < MaxPositionRHS;
			}
			
			return false;
		});


		TArray<ImmediatePhysics::FSimulation::FIgnorePair> IgnorePairs;
		if(NamesToHandles.Num() > 0)
		{
			//constraints
			for(int32 ConstraintIdx = 0; ConstraintIdx < HighLevelConstraintInstances.Num(); ++ConstraintIdx)
			{
				FConstraintInstance* CI = HighLevelConstraintInstances[ConstraintIdx];
				ImmediatePhysics::FActorHandle* Body1Handle = NamesToHandles.FindRef(CI->ConstraintBone1);
				ImmediatePhysics::FActorHandle* Body2Handle = NamesToHandles.FindRef(CI->ConstraintBone2);

				if(Body1Handle && Body2Handle)
				{
					if (Body1Handle->IsSimulated() || Body2Handle->IsSimulated())
					{
						PhysicsSimulation->CreateJoint(ImmediatePhysics::MakeJointSetup(CI, Body1Handle, Body2Handle));
						if (bForceDisableCollisionBetweenConstraintBodies)
						{
							int32 BodyIndex1 = UsePhysicsAsset->FindBodyIndex(CI->ConstraintBone1);
							int32 BodyIndex2 = UsePhysicsAsset->FindBodyIndex(CI->ConstraintBone2);
							if (BodyIndex1 != INDEX_NONE && BodyIndex2 != INDEX_NONE)
							{
								UsePhysicsAsset->DisableCollision(BodyIndex1, BodyIndex2);
							}
						}

						int32 BodyIndex;
						if (Bodies.Find(Body1Handle, BodyIndex))
						{
							BodyAnimData[BodyIndex].LinearXMotion = CI->GetLinearXMotion();
							BodyAnimData[BodyIndex].LinearYMotion = CI->GetLinearYMotion();
							BodyAnimData[BodyIndex].LinearZMotion = CI->GetLinearZMotion();
							BodyAnimData[BodyIndex].LinearLimit = CI->GetLinearLimit();

							//set limit to ref pose 
							FTransform Body1Transform = Body1Handle->GetWorldTransform();
							FTransform Body2Transform = Body2Handle->GetWorldTransform();
							BodyAnimData[BodyIndex].RefPoseLength = static_cast<float>(Body1Transform.GetRelativeTransform(Body2Transform).GetLocation().Size());
						}

						if (CI->IsCollisionDisabled())
						{
							ImmediatePhysics::FSimulation::FIgnorePair Pair;
							Pair.A = Body1Handle;
							Pair.B = Body2Handle;
							IgnorePairs.Add(Pair);
						}
					}
				}
			}

			ResetSimulatedTeleportType = ETeleportType::ResetPhysics;
		}

		// Terminate all the constraint instances
		for (FConstraintInstance* CI : HighLevelConstraintInstances)
		{
			CI->TermConstraint();
			delete CI;
		}

		// Terminate all of the instances, cannot be done during insert or we may break constraint chains
		for(FBodyInstance* Instance : HighLevelBodyInstances)
		{
			if(Instance->IsValidBodyInstance())
			{
				Instance->TermBody(true);
			}

			delete Instance;
		}

		HighLevelConstraintInstances.Empty();
		HighLevelBodyInstances.Empty();
		BodiesSorted.Empty();

		const TMap<FRigidBodyIndexPair, bool>& DisableTable = UsePhysicsAsset->CollisionDisableTable;
		for(auto ConstItr = DisableTable.CreateConstIterator(); ConstItr; ++ConstItr)
		{
			int32 IndexA = ConstItr.Key().Indices[0];
			int32 IndexB = ConstItr.Key().Indices[1];
			if ((IndexA < BodyIndexToActorHandle.Num()) && (IndexB < BodyIndexToActorHandle.Num()))
			{
				if ((BodyIndexToActorHandle[IndexA] != nullptr) && (BodyIndexToActorHandle[IndexB] != nullptr))
				{
					ImmediatePhysics::FSimulation::FIgnorePair Pair;
					Pair.A = BodyIndexToActorHandle[IndexA];
					Pair.B = BodyIndexToActorHandle[IndexB];
					IgnorePairs.Add(Pair);
				}
			}
		}

		PhysicsSimulation->SetIgnoreCollisionPairTable(IgnorePairs);
		PhysicsSimulation->SetIgnoreCollisionActors(IgnoreCollisionActors);

		SolverSettings = UsePhysicsAsset->SolverSettings;
		PhysicsSimulation->SetSolverSettings(
			SolverSettings.FixedTimeStep,
			SolverSettings.CullDistance,
			SolverSettings.MaxDepenetrationVelocity,
			SolverSettings.bUseLinearJointSolver,
			SolverSettings.PositionIterations,
			SolverSettings.VelocityIterations,
			SolverSettings.ProjectionIterations,
			SolverSettings.bUseManifolds);

		SolverIterations = UsePhysicsAsset->SolverIterations;
	}
}

DECLARE_CYCLE_STAT(TEXT("FAnimNode_RigidBody::UpdateWorldGeometry"), STAT_ImmediateUpdateWorldGeometry, STATGROUP_ImmediatePhysics);

void FAnimNode_RigidBody::UpdateWorldGeometry(const UWorld& World, const USkeletalMeshComponent& SKC)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigidBody")); 
	
	SCOPE_CYCLE_COUNTER(STAT_ImmediateUpdateWorldGeometry);
	QueryParams = FCollisionQueryParams(SCENE_QUERY_STAT(RagdollNodeFindGeometry), /*bTraceComplex=*/false);
#if WITH_EDITOR
	if(!World.IsGameWorld())
	{
		QueryParams.MobilityType = EQueryMobilityType::Any;	//If we're in some preview world trace against everything because things like the preview floor are not static
		QueryParams.AddIgnoredComponent(&SKC);
	}
	else
#endif
	{
		QueryParams.MobilityType = EQueryMobilityType::Static;	//We only want static actors
	}

	// Check for deleted world objects and flag for removal (later in anim task)
	ExpireWorldObjects();

	// If we have moved outside of the bounds we checked for world objects we need to gather new world objects
	FSphere Bounds = SKC.CalcBounds(SKC.GetComponentToWorld()).GetSphere();
	if (!Bounds.IsInside(CachedBounds))
	{
		// Since the cached bounds are no longer valid, update them.
		CachedBounds = Bounds;
		CachedBounds.W *= CachedBoundsScale;

		// Cache the PhysScene and World for use in UpdateWorldForces and CollectWorldObjects
		// When these are non-null it is an indicator that we need to update the collected world objects list
		PhysScene = World.GetPhysicsScene();
		UnsafeWorld = &World;
		UnsafeOwner = SKC.GetOwner();

		// A timer to track objects we haven't detected in a while
		++ComponentsInSimTick;
	}
}

DECLARE_CYCLE_STAT(TEXT("FAnimNode_RigidBody::UpdateWorldForces"), STAT_ImmediateUpdateWorldForces, STATGROUP_ImmediatePhysics);

void FAnimNode_RigidBody::UpdateWorldForces(const FTransform& ComponentToWorld, const FTransform& BaseBoneTM, const float DeltaSeconds)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigidBody")); 
	SCOPE_CYCLE_COUNTER(STAT_ImmediateUpdateWorldForces);

	if(TotalMass > 0.0)
	{
		for (const USkeletalMeshComponent::FPendingRadialForces& PendingRadialForce : PendingRadialForces)
		{
			const FVector RadialForceOrigin = WorldPositionToSpace(SimulationSpace, PendingRadialForce.Origin, ComponentToWorld, BaseBoneTM);
			for(ImmediatePhysics::FActorHandle* Body : Bodies)
			{
				const double InvMass = Body->GetInverseMass();
				if(InvMass > 0.0)
				{
					const double StrengthPerBody = PendingRadialForce.bIgnoreMass ? PendingRadialForce.Strength : PendingRadialForce.Strength / (TotalMass * InvMass);
					ImmediatePhysics::EForceType ForceType;
					if (PendingRadialForce.Type == USkeletalMeshComponent::FPendingRadialForces::AddImpulse)
					{
						ForceType = PendingRadialForce.bIgnoreMass ? ImmediatePhysics::EForceType::AddVelocity : ImmediatePhysics::EForceType::AddImpulse;
					}
					else
					{
						ForceType = PendingRadialForce.bIgnoreMass ? ImmediatePhysics::EForceType::AddAcceleration : ImmediatePhysics::EForceType::AddForce;
					}
					
					Body->AddRadialForce(RadialForceOrigin, StrengthPerBody, PendingRadialForce.Radius, PendingRadialForce.Falloff, ForceType);
				}
			}
		}

		if(!ExternalForce.IsNearlyZero())
		{
			const FVector ExternalForceInSimSpace = WorldVectorToSpaceNoScale(SimulationSpace, ExternalForce, ComponentToWorld, BaseBoneTM);
			for (ImmediatePhysics::FActorHandle* Body : Bodies)
			{
				const double InvMass = Body->GetInverseMass();
				if (InvMass > 0.0)
				{
					Body->AddForce(ExternalForceInSimSpace);
				}
			}
		}
		if(DeltaSeconds != 0.0)
		{
			if(!PerSolverField.IsEmpty())
			{
				TArray<FVector>& SamplePositions = PerSolverField.GetSamplePositions();
				TArray<FFieldContextIndex>& SampleIndices = PerSolverField.GetSampleIndices();

				SamplePositions.SetNum(Bodies.Num(), EAllowShrinking::No);
				SampleIndices.SetNum(Bodies.Num(), EAllowShrinking::No);

				int32 Index = 0;
				for (ImmediatePhysics::FActorHandle* Body : Bodies)
				{
					SamplePositions[Index] = (Body->GetWorldTransform() * SpaceToWorldTransform(SimulationSpace, ComponentToWorld, BaseBoneTM)).GetLocation();
					SampleIndices[Index] = FFieldContextIndex(Index, Index);
					++Index;
				}
				PerSolverField.ComputeFieldRigidImpulse(WorldTimeSeconds);

				const TArray<FVector>& LinearVelocities = PerSolverField.GetOutputResults(EFieldCommandOutputType::LinearVelocity);
				const TArray<FVector>& LinearForces = PerSolverField.GetOutputResults(EFieldCommandOutputType::LinearForce);
				const TArray<FVector>& AngularVelocities = PerSolverField.GetOutputResults(EFieldCommandOutputType::AngularVelocity);
				const TArray<FVector>& AngularTorques = PerSolverField.GetOutputResults(EFieldCommandOutputType::AngularTorque);

				if (LinearVelocities.Num() == Bodies.Num())
				{
					Index = 0;
					for (ImmediatePhysics::FActorHandle* Body : Bodies)
					{
						const FVector ExternalForceInSimSpace = WorldVectorToSpaceNoScale(SimulationSpace, LinearVelocities[Index++], ComponentToWorld, BaseBoneTM) * Body->GetMass() / DeltaSeconds;
						Body->AddForce(ExternalForceInSimSpace);
					}
				}
				if (LinearForces.Num() == Bodies.Num())
				{
					Index = 0;
					for (ImmediatePhysics::FActorHandle* Body : Bodies)
					{
						const FVector ExternalForceInSimSpace = WorldVectorToSpaceNoScale(SimulationSpace, LinearForces[Index++], ComponentToWorld, BaseBoneTM);
						Body->AddForce(ExternalForceInSimSpace);
					}
				}
				if (AngularVelocities.Num() == Bodies.Num())
				{
					Index = 0;
					for (ImmediatePhysics::FActorHandle* Body : Bodies)
					{
						const FVector ExternalTorqueInSimSpace = WorldVectorToSpaceNoScale(SimulationSpace, AngularVelocities[Index++], ComponentToWorld, BaseBoneTM) * Body->GetInertia() / DeltaSeconds;
						Body->AddTorque(ExternalTorqueInSimSpace);
					}
				}
				if (AngularTorques.Num() == Bodies.Num())
				{
					Index = 0;
					for (ImmediatePhysics::FActorHandle* Body : Bodies)
					{
						const FVector ExternalTorqueInSimSpace = WorldVectorToSpaceNoScale(SimulationSpace, AngularTorques[Index++], ComponentToWorld, BaseBoneTM);
						Body->AddTorque(ExternalTorqueInSimSpace);
					}
				}
			}
		}
	}
}

bool FAnimNode_RigidBody::NeedsDynamicReset() const
{
	return true;
}

void FAnimNode_RigidBody::ResetDynamics(ETeleportType InTeleportType)
{
	// This will be picked up next evaluate and reset our simulation.
	// Teleport type can only go higher - i.e. if we have requested a reset, then a teleport will still reset fully
	ResetSimulatedTeleportType = ((InTeleportType > ResetSimulatedTeleportType) ? InTeleportType : ResetSimulatedTeleportType);
}

void FAnimNode_RigidBody::SetOverridePhysicsAsset(UPhysicsAsset* PhysicsAsset)
{
	OverridePhysicsAsset = PhysicsAsset;
}

DECLARE_CYCLE_STAT(TEXT("RigidBody_PreUpdate"), STAT_RigidBody_PreUpdate, STATGROUP_Anim);

void FAnimNode_RigidBody::PreUpdate(const UAnimInstance* InAnimInstance)
{
	// Detect changes in the physics asset to be used. This can happen when using the override physics asset as a pin on the anim graph node.
	UPhysicsAsset* PhysicsAssetToBeUsed = GetPhysicsAssetToBeUsed(InAnimInstance);
	if (UsePhysicsAsset != PhysicsAssetToBeUsed)
	{
		InitPhysics(InAnimInstance);

		// Update the bone references after a change in the physics asset. This needs to happen after initializing physics as the Bodies set up in InitPhysics() need to be up to date.
		InitializeBoneReferences(InAnimInstance->GetRequiredBones());
	}

	// Don't update geometry if RBN is disabled
	if(!bEnabled)
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_RigidBody_PreUpdate);

	USkeletalMeshComponent* SKC = InAnimInstance->GetSkelMeshComponent();
	APawn* PawnOwner = InAnimInstance->TryGetPawnOwner();
	UPawnMovementComponent* MovementComp = PawnOwner ? PawnOwner->GetMovementComponent() : nullptr;

#if WITH_EDITOR
	if (bEnableWorldGeometry && SimulationSpace != ESimulationSpace::WorldSpace && SKC && SKC->GetRelativeScale3D() != FVector(1.f, 1.f, 1.f))
	{
		FMessageLog("PIE").Warning(FText::Format(LOCTEXT("WorldCollisionComponentSpace", "Trying to use world collision without world space simulation for scaled ''{0}''. This is not supported, please change SimulationSpace to WorldSpace"),
			FText::FromString(GetPathNameSafe(SKC))));
	}
#endif

	UWorld* World = InAnimInstance->GetWorld();
	if (World)
	{
		WorldSpaceGravity = bOverrideWorldGravity ? OverrideWorldGravity : (MovementComp ? FVector(0.f, 0.f, MovementComp->GetGravityZ()) : FVector(0.f, 0.f, World->GetGravityZ()));

		if(SKC)
		{
			// Store game time for use in parallel evaluation. This may be the totol time (inc pauses) or the time the game has been unpaused.
			WorldTimeSeconds = SKC->PrimaryComponentTick.bTickEvenWhenPaused ? World->UnpausedTimeSeconds : World->TimeSeconds;

			if (PhysicsSimulation && bEnableWorldGeometry)
			{ 
				// @todo: this logic can be simplified now. We used to run PurgeExpiredWorldObjects and CollectWorldObjects
				// in UpdateAnimation, but we can't access the world actor's geometry there
				UpdateWorldGeometry(*World, *SKC);

				// Remove expired objects from the sim
				PurgeExpiredWorldObjects();

				// Find nearby world objects to add to the sim (gated on UnsafeWorld - see UpdateWorldGeometry)
				CollectWorldObjects();
			}

			PendingRadialForces = SKC->GetPendingRadialForces();

			PreviousTransform = CurrentTransform;
			CurrentTransform = SKC->GetComponentToWorld();

			if (World->PhysicsField)
			{
				const FBox BoundingBox = SKC->CalcBounds(SKC->GetComponentTransform()).GetBox();

				PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS
				World->PhysicsField->FillTransientCommands(false, BoundingBox, WorldTimeSeconds, PerSolverField.GetTransientCommands());
				World->PhysicsField->FillPersistentCommands(false, BoundingBox, WorldTimeSeconds, PerSolverField.GetPersistentCommands());
				PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS
			}
		}
	}

	if (bUseExternalClothCollision && ClothColliders.IsEmpty())
	{
		// The Cloth Collider assets are part of the SkelMeshComponent and can be initialized after the first call to InitPhysics. Keep checking here until some 
		// are found, following the behavior of the cloth system (see USkeletalMeshComponent::UpdateClothTransformImp())
		CollectClothColliderObjects(SKC);
	}
}

int32 FAnimNode_RigidBody::GetLODThreshold() const
{
	if (bUseLocalLODThresholdOnly || CVarRigidBodyLODThreshold.GetValueOnAnyThread() == -1)
	{
		return LODThreshold;
	}
	else
	{
		if(LODThreshold != -1)
		{
			return FMath::Min(LODThreshold, CVarRigidBodyLODThreshold.GetValueOnAnyThread());
		}
		else
		{
			return CVarRigidBodyLODThreshold.GetValueOnAnyThread();
		}
	}
}

DECLARE_CYCLE_STAT(TEXT("RigidBody_Update"), STAT_RigidBody_Update, STATGROUP_Anim);

void FAnimNode_RigidBody::UpdateInternal(const FAnimationUpdateContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigidBody")); 
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(UpdateInternal)
	// Avoid this work if RBN is disabled, as the results would be discarded
	if(!bEnabled)
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_RigidBody_Update);
	
	// Must flush the simulation since we may be making changes to the scene
	FlushDeferredSimulationTask();

	// Accumulate deltatime elapsed during update. To be used during evaluation.
	AccumulatedDeltaTime += Context.AnimInstanceProxy->GetDeltaSeconds();

	if (UnsafeWorld != nullptr)
	{
		// Node is valid to evaluate. Simulation is starting.
		bSimulationStarted = true;
	}

	// These get set again if our bounds change. Subsequent calls to CollectWorldObjects will early-out until then
	UnsafeWorld = nullptr;
	UnsafeOwner = nullptr;
	PhysScene = nullptr;
}

void FAnimNode_RigidBody::CollectClothColliderObjects(const USkeletalMeshComponent* SkeletalMeshComp)
{
	if (bUseExternalClothCollision && bRBAN_IncludeClothColliders && SkeletalMeshComp && PhysicsSimulation)
	{
		const TArray<FClothCollisionSource>& SkeletalMeshClothCollisionSources = SkeletalMeshComp->GetClothCollisionSources();
		
		for (const FClothCollisionSource& ClothCollisionSource : SkeletalMeshClothCollisionSources)
		{
			const USkeletalMeshComponent* const SourceComponent = ClothCollisionSource.SourceComponent.Get();
			const UPhysicsAsset* const PhysicsAsset = ClothCollisionSource.SourcePhysicsAsset.Get();

			if (SourceComponent && PhysicsAsset)
			{
				TArray<FBodyInstance*> BodyInstances;
				SourceComponent->InstantiatePhysicsAssetBodiesRefPose(*PhysicsAsset, BodyInstances);

				for (uint32 BodyInstanceIndex = 0, BodyInstanceMax = BodyInstances.Num(); BodyInstanceIndex < BodyInstanceMax; ++BodyInstanceIndex)
				{
					FBodyInstance* const BodyInstance = BodyInstances[BodyInstanceIndex];

					if (BodyInstance->InstanceBoneIndex != INDEX_NONE) // Invalid Bone index can occur if the physics asset references a bone name that does not exist in the Cloth Collision Source.
					{
						ImmediatePhysics::FActorHandle* const ActorHandle = PhysicsSimulation->CreateActor(ImmediatePhysics::MakeKinematicActorSetup(BodyInstance, BodyInstance->GetUnrealWorldTransform()));
						PhysicsSimulation->AddToCollidingPairs(ActorHandle); // <-allow collision between this actor and all dynamic actors.
						ClothColliders.Add(FClothCollider(ActorHandle, SourceComponent, BodyInstance->InstanceBoneIndex));
					}

					// Terminate the instance.
					if (BodyInstance->IsValidBodyInstance())
					{
						BodyInstance->TermBody(true);
					}

					delete BodyInstance;
					BodyInstances[BodyInstanceIndex] = nullptr;
				}

				BodyInstances.Reset();
			}
		}
	}
}

void FAnimNode_RigidBody::RemoveClothColliderObjects()
{
	for (const FClothCollider& ClothCollider : ClothColliders)
	{
		PhysicsSimulation->DestroyActor(ClothCollider.ActorHandle);
	}
	
	ClothColliders.Reset();
}

void FAnimNode_RigidBody::UpdateClothColliderObjects(const FTransform& SpaceTransform)
{
	for (FClothCollider& ClothCollider : ClothColliders)
	{
		if (ClothCollider.ActorHandle && ClothCollider.SkeletalMeshComponent)
		{
			// Calculate the sim-space transform of this object
			const FTransform CompWorldTransform = ClothCollider.SkeletalMeshComponent->GetBoneTransform(ClothCollider.BoneIndex);
			FTransform CompSpaceTransform;
			CompSpaceTransform.SetTranslation(SpaceTransform.InverseTransformPosition(CompWorldTransform.GetLocation()));
			CompSpaceTransform.SetRotation(SpaceTransform.InverseTransformRotation(CompWorldTransform.GetRotation()));
			CompSpaceTransform.SetScale3D(FVector::OneVector);	// TODO - sort out scale for world objects in local sim

			// Update the sim's copy of the world object
			ClothCollider.ActorHandle->SetKinematicTarget(CompSpaceTransform);
		}
	}
}

void FAnimNode_RigidBody::CollectWorldObjects()
{
	if ((UnsafeWorld != nullptr) && (PhysScene != nullptr))
	{
		// @todo(ccaulfield): should this use CachedBounds?
		TArray<FOverlapResult> Overlaps;
		UnsafeWorld->OverlapMultiByChannel(Overlaps, CachedBounds.Center, FQuat::Identity, OverlapChannel, FCollisionShape::MakeSphere(static_cast<float>(CachedBounds.W)), QueryParams, FCollisionResponseParams(ECR_Overlap));

		for (const FOverlapResult& Overlap : Overlaps)
		{
			if (UPrimitiveComponent* OverlapComp = Overlap.GetComponent())
			{
				FWorldObject* WorldObject = ComponentsInSim.Find(OverlapComp);
				if (WorldObject != nullptr)
				{
					// Existing object - reset its age
					WorldObject->LastSeenTick = ComponentsInSimTick;
				}
				else
				{
					// New object - add it to the sim
					const bool bIsSelf = (UnsafeOwner == OverlapComp->GetOwner());
					if (!bIsSelf)
					{
						// Create a kinematic actor. Not using Static as world-static objects may
						// move in the simulation's frame of reference
						ImmediatePhysics::FActorHandle* ActorHandle = PhysicsSimulation->CreateActor(
								ImmediatePhysics::MakeKinematicActorSetup(
									&OverlapComp->BodyInstance, OverlapComp->GetComponentTransform()));
						PhysicsSimulation->AddToCollidingPairs(ActorHandle);
						ComponentsInSim.Add(
							OverlapComp, FWorldObject(ActorHandle, ComponentsInSimTick, Overlap.Component));
					}
				}
			}
		}
	}
}

// Flag invalid objects for purging
void FAnimNode_RigidBody::ExpireWorldObjects()
{
	// Invalidate deleted and expired world objects
	TArray<const UPrimitiveComponent*> PrunedEntries;
	for (auto& WorldEntry : ComponentsInSim)
	{
		FWorldObject& WorldObject = WorldEntry.Value;
		const UPrimitiveComponent* WorldComp = WorldObject.Component.Get();

		// Do we need to expire this object?
		const int32 ExpireTickCount = RBAN_WorldObjectExpiry;
		bool bIsInvalid =
			((ComponentsInSimTick - WorldObject.LastSeenTick) > ExpireTickCount)	// Haven't seen this object for a while
			|| !IsValid(WorldComp)
			|| (WorldComp->GetBodyInstance() == nullptr)
			|| (!WorldComp->GetBodyInstance()->IsValidBodyInstance());

		// Remove from sim if necessary
		if (bIsInvalid)
		{
			WorldObject.bExpired = true;
		}
	}
}

void FAnimNode_RigidBody::PurgeExpiredWorldObjects()
{
	// Destroy expired simulated objects
	TArray<const UPrimitiveComponent*> PurgedEntries;
	for (auto& WorldEntry : ComponentsInSim)
	{
		FWorldObject& WorldObject = WorldEntry.Value;

		if (WorldObject.bExpired)
		{
			PhysicsSimulation->DestroyActor(WorldObject.ActorHandle);
			WorldObject.ActorHandle = nullptr;

			PurgedEntries.Add(WorldEntry.Key);
		}
	}

	// Remove purged map entries
	for (const UPrimitiveComponent* PurgedEntry : PurgedEntries)
	{
		ComponentsInSim.Remove(PurgedEntry);
	}
}

// Update the transforms of the world objects we added to the sim. This is required
// if we have a component- or bone-space simulation as even world-static objects
// will be moving in the simulation's frame of reference.
void FAnimNode_RigidBody::UpdateWorldObjects(const FTransform& SpaceTransform)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigidBody")); 


	if (SimulationSpace != ESimulationSpace::WorldSpace)
	{
		for (TPair<const UPrimitiveComponent*, FWorldObject>& WorldEntry : ComponentsInSim)
		{ 
			const UPrimitiveComponent* OverlapComp = WorldEntry.Value.Component.Get();
			if (OverlapComp != nullptr)
			{
				FWorldObject& WorldObject = WorldEntry.Value;
				ImmediatePhysics::FActorHandle* ActorHandle = WorldObject.ActorHandle;

				// Calculate the sim-space transform of this object
				const FTransform CompWorldTransform = OverlapComp->GetComponentTransform();
				FTransform CompSpaceTransform;
				CompSpaceTransform.SetTranslation(SpaceTransform.InverseTransformPosition(CompWorldTransform.GetLocation()));
				CompSpaceTransform.SetRotation(SpaceTransform.InverseTransformRotation(CompWorldTransform.GetRotation()));
				CompSpaceTransform.SetScale3D(FVector::OneVector);	// TODO - sort out scale for world objects in local sim

				// Update the sim's copy of the world object
				ActorHandle->SetKinematicTarget(CompSpaceTransform);

				// We need to update the particle's transform in the right space for the first time. 
				// When actor is created in CollectWorldObjects the space transform is still unknown.
				if (WorldObject.bNew)
				{
					WorldObject.bNew = false;
					ActorHandle->InitWorldTransform(CompSpaceTransform);
				}
			}
		}
	}
}

void FAnimNode_RigidBody::InitializeBoneReferences(const FBoneContainer& RequiredBones) 
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	/** We only need to update simulated bones and children of simulated bones*/
	const int32 NumBodies = Bodies.Num();
	OutputBoneData.Empty(NumBodies);

	if (!RequiredBones.IsValid())
	{
		UE_LOGF(LogRBAN, Warning, "FAnimNode_RigidBody::InitializeBoneReferences: Required bones is invalid for Skeleton %ls / SkeletalMesh %ls.",
			*GetNameSafe(RequiredBones.GetSkeletonAsset()), *GetNameSafe(RequiredBones.GetSkeletalMeshAsset()));
		return;
	}

	const FReferenceSkeleton& RefSkeleton = RequiredBones.GetReferenceSkeleton();
	const TArray<FBoneIndexType>& RequiredBoneIndices = RequiredBones.GetBoneIndicesArray();
	const int32 NumRequiredBoneIndices = RequiredBoneIndices.Num();

	int32 NumSimulatedBodies = 0;
	TArray<int32> SimulatedBodyIndices;
	// if no name is entered, use root
	if (BaseBoneRef.BoneName == NAME_None)
	{
		BaseBoneRef.BoneName = RefSkeleton.GetBoneName(0);
	}

	// If the user specified a simulation root that is not used by the skelmesh, issue a warning 
	// (FAnimNode_RigidBody::IsValidToEvaluate will return false and the simulation will not run)
	InitializeAndValidateBoneRef(BaseBoneRef, RequiredBones);

	bool bHasInvalidBoneReference = false;
	for (int32 Index = 0; Index < NumRequiredBoneIndices; ++Index)
	{
		const FCompactPoseBoneIndex CompactPoseBoneIndex(Index);
		const int32 SkeletonBoneIndex = RequiredBones.GetSkeletonIndex(CompactPoseBoneIndex);

		// If we have a missing bone in our skeleton, we don't want to have an out of bounds access.
		if (!SkeletonBoneIndexToBodyIndex.IsValidIndex(SkeletonBoneIndex))
		{
			bHasInvalidBoneReference = true;
			break;
		}

		const int32 BodyIndex = SkeletonBoneIndexToBodyIndex[SkeletonBoneIndex];

		if (BodyIndex != INDEX_NONE)
		{
			// Avoid and track down issues with out-of-bounds access of BodyAnimData
			if (bRBAN_InitializeBoneReferencesRangeCheckEnabled)
			{
				if (!ensure(BodyAnimData.IsValidIndex(BodyIndex)))
				{
					UE_LOGF(LogRBAN, Warning, "FAnimNode_RigidBody::InitializeBoneReferences: BodyIndex out of range. BodyIndex=%d/%d, SkeletonBoneIndex=%d/%d, CompactPoseBoneIndex=%d, RequiredBoneIndex=%d",
								 BodyIndex, BodyAnimData.Num(), SkeletonBoneIndex, SkeletonBoneIndexToBodyIndex.Num(), CompactPoseBoneIndex.GetInt(), Index);

					bHasInvalidBoneReference = true;
					break;
				}
			}

			//If we have a body we need to save it for later
			FOutputBoneData* OutputData = new (OutputBoneData) FOutputBoneData();
			OutputData->BodyIndex = BodyIndex;
			OutputData->CompactPoseBoneIndex = CompactPoseBoneIndex;

			if (BodyAnimData[BodyIndex].bIsSimulated)
			{
				++NumSimulatedBodies;
				SimulatedBodyIndices.AddUnique(BodyIndex);
			}

			OutputData->BoneIndicesToParentBody.Add(CompactPoseBoneIndex);

			// Walk up parent chain until we find parent body.
			OutputData->ParentBodyIndex = INDEX_NONE;
			FCompactPoseBoneIndex CompactParentIndex = RequiredBones.GetParentBoneIndex(CompactPoseBoneIndex);
			while (CompactParentIndex != INDEX_NONE)
			{
				const int32 SkeletonParentBoneIndex = RequiredBones.GetSkeletonIndex(CompactParentIndex);

				// Must check our parent as well for a missing bone.
				if (!SkeletonBoneIndexToBodyIndex.IsValidIndex(SkeletonParentBoneIndex))
				{
					bHasInvalidBoneReference = true;
					break;
				}

				OutputData->ParentBodyIndex = SkeletonBoneIndexToBodyIndex[SkeletonParentBoneIndex];
				if (OutputData->ParentBodyIndex != INDEX_NONE)
				{
					break;
				}

				OutputData->BoneIndicesToParentBody.Add(CompactParentIndex);
				CompactParentIndex = RequiredBones.GetParentBoneIndex(CompactParentIndex);
			}

			if (bHasInvalidBoneReference)
			{
				break;
			}
		}
	}

	if (bHasInvalidBoneReference)
	{
		// If a bone was missing, let us know which asset it happened on, and clear our bone container to make the bad asset visible.
		UE_LOGF(LogRBAN, Warning, "FAnimNode_RigidBody::InitializeBoneReferences: The Skeleton %ls, is missing bones that SkeletalMesh %ls needs. Skeleton might need to be resaved.",
			*GetNameSafe(RequiredBones.GetSkeletonAsset()), *GetNameSafe(RequiredBones.GetSkeletalMeshAsset()));
		ensure(false);
		OutputBoneData.Empty();
	}
	else
	{
		// New bodies potentially introduced with new LOD
		// We'll have to initialize their transform.
		bCheckForBodyTransformInit = true;

		if (PhysicsSimulation)
		{
			PhysicsSimulation->SetNumActiveBodies(NumSimulatedBodies, SimulatedBodyIndices);
		}

		// We're switching to a new LOD, this invalidates our captured poses.
		CapturedFrozenPose.Empty();
		CapturedFrozenCurves.Empty();
	}
}

void FAnimNode_RigidBody::AddImpulseAtLocation(FVector Impulse, FVector Location, FName BoneName)
{
	// Find the body. This is currently only used in the editor and will need optimizing if used in game
	for (int32 BodyIndex = 0; BodyIndex < Bodies.Num(); ++BodyIndex)
	{
		ImmediatePhysics::FActorHandle* Body = Bodies[BodyIndex];
		if (Body->GetName() == BoneName)
		{
			Body->AddImpulseAtLocation(Impulse, Location);
		}
	}
}

void FAnimNode_RigidBody::OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance)
{
	InitPhysics(InAnimInstance);
}

#if WITH_EDITORONLY_DATA
void FAnimNode_RigidBody::PostSerialize(const FArchive& Ar)
{
	if(bComponentSpaceSimulation_DEPRECATED == false)
	{
		//If this is not the default value it means we have old content where we were simulating in world space
		SimulationSpace = ESimulationSpace::WorldSpace;
		bComponentSpaceSimulation_DEPRECATED = true;
	}
}
#endif

bool FAnimNode_RigidBody::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones)
{
	return BaseBoneRef.IsValidToEvaluate(RequiredBones);
}

bool RigidBodyNodeIsPoseScalePropagationEnabledGlobally()
{
	return CVarEnableRigidBodyNodePoseScalePropagation.GetValueOnAnyThread();
}

#undef LOCTEXT_NAMESPACE

