// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_RigidBody_Library.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_RigidBody_Library)

FRigidBodyAnimNodeReference UAnimNodeRigidBodyLibrary::ConvertToRigidBodyAnimNode(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result)
{
	return FAnimNodeReference::ConvertToType<FRigidBodyAnimNodeReference>(Node, Result);
}

void UAnimNodeRigidBodyLibrary::ConvertToRigidBodyAnimNodePure(const FAnimNodeReference& Node, FRigidBodyAnimNodeReference& RigidBodyAnimNode, bool& Result)
{
	EAnimNodeReferenceConversionResult ConversionResult;
	RigidBodyAnimNode = ConvertToRigidBodyAnimNode(Node, ConversionResult);
	Result = (ConversionResult == EAnimNodeReferenceConversionResult::Succeeded);
}

FRigidBodyAnimNodeReference UAnimNodeRigidBodyLibrary::SetOverridePhysicsAsset(const FRigidBodyAnimNodeReference& Node, UPhysicsAsset* PhysicsAsset)
{
	Node.CallAnimNodeFunction<FAnimNode_RigidBody>(
		TEXT("SetOverridePhysicsAsset"),
		[PhysicsAsset](FAnimNode_RigidBody& InRigidBodyNode)
		{
			InRigidBodyNode.SetOverridePhysicsAsset(PhysicsAsset);
		});

	return Node;
}
