// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_ScaleChainLength.h"
#include "PrimitiveDrawingUtils.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimStats.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_ScaleChainLength)

/////////////////////////////////////////////////////
// FAnimNode_ScaleChainLength

FAnimNode_ScaleChainLength::FAnimNode_ScaleChainLength()
	: DefaultChainLength(0.0f)
	, TargetLocation(ForceInitToZero)
	, Alpha(1.f)
	, ActualAlpha(0.0f)
	, ChainInitialLength(EScaleChainInitialLength::FixedDefaultLengthValue)
	, bBoneIndicesCached(false)
{
}

void FAnimNode_ScaleChainLength::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_Base::Initialize_AnyThread(Context);
	InputPose.Initialize(Context);
}

void FAnimNode_ScaleChainLength::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)
	GetEvaluateGraphExposedInputs().Execute(Context);
	InputPose.Update(Context);

	ActualAlpha = AlphaScaleBias.ApplyTo(Alpha);
}

void FAnimNode_ScaleChainLength::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	InputPose.CacheBones(Context);

	// LOD change, recache bone indices.
	bBoneIndicesCached = false;
}

void FAnimNode_ScaleChainLength::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(ScaleChainLength, !IsInGameThread());

	// Evaluate incoming pose into our output buffer.
	InputPose.Evaluate(Output);

	if (!FAnimWeight::IsRelevant(ActualAlpha))
	{
		return;
	}

	const FBoneContainer& BoneContainer = Output.Pose.GetBoneContainer();

	if (!bBoneIndicesCached)
	{
		bBoneIndicesCached = true;

		ChainStartBone.Initialize(BoneContainer);
		ChainEndBone.Initialize(BoneContainer);
		ChainBoneIndices.Reset();

		// Make sure we have valid start/end bones, and that end is a child of start.
		// Cache this, so we only evaluate on init and LOD changes.
		const bool bBoneSetupIsValid = ChainStartBone.IsValidToEvaluate(BoneContainer) && ChainEndBone.IsValidToEvaluate(BoneContainer) &&
			BoneContainer.BoneIsChildOf(ChainEndBone.GetCompactPoseIndex(BoneContainer), ChainStartBone.GetCompactPoseIndex(BoneContainer));

		if (bBoneSetupIsValid)
		{
			const FCompactPoseBoneIndex StartBoneIndex = ChainStartBone.GetCompactPoseIndex(BoneContainer);
			FCompactPoseBoneIndex BoneIndex = ChainEndBone.GetCompactPoseIndex(BoneContainer);
			ChainBoneIndices.Add(BoneIndex);
			if (BoneIndex != INDEX_NONE)
			{
				FCompactPoseBoneIndex ParentBoneIndex = BoneContainer.GetParentBoneIndex(BoneIndex);
				while ((ParentBoneIndex != INDEX_NONE) && (ParentBoneIndex != StartBoneIndex))
				{
					BoneIndex = ParentBoneIndex;
					ParentBoneIndex = BoneContainer.GetParentBoneIndex(BoneIndex);
					ChainBoneIndices.Insert(BoneIndex, 0);
				};
				ChainBoneIndices.Insert(StartBoneIndex, 0);
			}
		}
	}

	// Need at least Start/End bones to be valid.
	if (ChainBoneIndices.Num() < 2)
	{
		return;
	}

	const FVector TargetLocationCompSpace = Output.AnimInstanceProxy->GetComponentTransform().InverseTransformPosition(TargetLocation);

	// Allocate transforms to get component space transform of chain start bone.
	FCSPose<FCompactPose> CSPose;
	CSPose.InitPose(Output.Pose);

	const FTransform StartTransformCompSpace = CSPose.GetComponentSpaceTransform(ChainBoneIndices[0]);

	const double DesiredChainLength = (TargetLocationCompSpace - StartTransformCompSpace.GetLocation()).Size();
	const double InitialChainLength = GetInitialChainLength(Output.Pose, CSPose);
	const double ChainLengthScale = !FMath::IsNearlyZero(InitialChainLength) ? (DesiredChainLength / InitialChainLength) : 1.0;
	const double ChainLengthScaleWithAlpha = FMath::LerpStable(1.0, ChainLengthScale, ActualAlpha);

	// If we're not going to scale anything, early out.
	if (FMath::IsNearlyEqual(ChainLengthScaleWithAlpha, 1.0))
	{
		return;
	}

	// Scale translation of all bones in local space.
	FCompactPose& LSPose = Output.Pose;
	for (const FCompactPoseBoneIndex& BoneIndex : ChainBoneIndices)
	{
		// Get bone space transform, scale transition.
		LSPose[BoneIndex].ScaleTranslation(ChainLengthScaleWithAlpha);
	}
}

double FAnimNode_ScaleChainLength::GetInitialChainLength(FCompactPose& InLSPose, FCSPose<FCompactPose>& InCSPose) const
{
	switch (ChainInitialLength)
	{
	case EScaleChainInitialLength::Distance : 
	{
		const FVector ChainStartLocation = InCSPose.GetComponentSpaceTransform(ChainBoneIndices[0]).GetLocation();
		const FVector ChainEndLocation = InCSPose.GetComponentSpaceTransform(ChainBoneIndices.Last()).GetLocation();
		return (ChainEndLocation - ChainStartLocation).Size();
	}

	case EScaleChainInitialLength::ChainLength :
	{
		double ChainLength = 0.0;
		for (const FCompactPoseBoneIndex& BoneIndex : ChainBoneIndices)
		{
			ChainLength += InLSPose[BoneIndex].GetTranslation().Size();
		}
		return ChainLength;
	}
	};

	// Fallback is using fixed value DefaultChainLength.
	return DefaultChainLength;
}

void FAnimNode_ScaleChainLength::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("Alpha (%.1f%%)"), ActualAlpha * 100.f);
	DebugData.AddDebugItem(DebugLine);

	InputPose.GatherDebugData(DebugData);
}

