// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimTrace.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimStats.h"
#include "Engine/SkeletalMeshSocket.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_SkeletalControlBase)

#define LOCTEXT_NAMESPACE "AnimNode_SkeletalControlBase"

#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	DECLARE_LOG_CATEGORY_EXTERN(LogSkeletalControlBase, Warning, Warning);
#else
	DECLARE_LOG_CATEGORY_EXTERN(LogSkeletalControlBase, Log, All);
#endif
DEFINE_LOG_CATEGORY(LogSkeletalControlBase);

/////////////////////////////////////////////////////
// FAnimNode_SkeletalControlBase

void FAnimNode_SkeletalControlBase::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_Base::Initialize_AnyThread(Context);

	ComponentPose.Initialize(Context);

	AlphaBoolBlend.Reinitialize();
	AlphaScaleBiasClamp.Reinitialize();
}

void FAnimNode_SkeletalControlBase::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
#if WITH_EDITOR
	ClearValidationVisualWarnings();
#endif

	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	FAnimNode_Base::CacheBones_AnyThread(Context);
	InitializeBoneReferences(Context.AnimInstanceProxy->GetRequiredBones());
	ComponentPose.CacheBones(Context);
}

void FAnimNode_SkeletalControlBase::UpdateInternal(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(UpdateInternal)
}

void FAnimNode_SkeletalControlBase::UpdateComponentPose_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(UpdateComponentPose_AnyThread)
	ComponentPose.Update(Context);
}

void FAnimNode_SkeletalControlBase::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	//////////////////////////////////////////////////////////////////////////
	// PERFORMANCE CRITICAL NOTE
	// 
	// This function is called recursively as we traverse nodes, as such, it is critical to keep the
	// amount of stack space used to a minimum as many nodes can be traversed. Using too much stack
	// here can quickly lead to stack overflows.
	// 
	// We explicitly disable inlineing for virtual calls. Normally, virtual calls are never inlined
	// but when PGO and LTO are enabled, the compiler can speculatively de-virtualize the calls. It does
	// this by comparing the v-table pointer and inlineing the call directly in here with a static jump.
	// As a result, it can significantly increase the amount of stack space used.
	//////////////////////////////////////////////////////////////////////////

	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)
	UE_DONT_INLINE_CALL UpdateComponentPose_AnyThread(Context);

	ActualAlpha = 0.f;
	if (IsLODEnabled(Context.AnimInstanceProxy))
	{
		GetEvaluateGraphExposedInputs().Execute(Context);

		// Apply the skeletal control if it's valid
		switch (AlphaInputType)
		{
		case EAnimAlphaInputType::Float : 
			ActualAlpha = AlphaScaleBias.ApplyTo(AlphaScaleBiasClamp.ApplyTo(Alpha, Context.GetDeltaTime()));
			break;
		case EAnimAlphaInputType::Bool :
			ActualAlpha = AlphaBoolBlend.ApplyTo(bAlphaBoolEnabled, Context.GetDeltaTime());
			break;
		case EAnimAlphaInputType::Curve :
			if (UAnimInstance* AnimInstance = Cast<UAnimInstance>(Context.AnimInstanceProxy->GetAnimInstanceObject()))
			{
				ActualAlpha = AlphaScaleBiasClamp.ApplyTo(AnimInstance->GetCurveValue(AlphaCurveName), Context.GetDeltaTime());
			}
			break;
		};

		// Make sure Alpha is clamped between 0 and 1.
		ActualAlpha = FMath::Clamp<float>(ActualAlpha, 0.f, 1.f);

		if (FAnimWeight::IsRelevant(ActualAlpha))
		{
			const USkeleton* Skeleton = Context.AnimInstanceProxy->GetSkeleton();
			const FBoneContainer& RequiredBones = Context.AnimInstanceProxy->GetRequiredBones();

			bool bIsValidToEvaluate;
			UE_DONT_INLINE_CALL bIsValidToEvaluate = IsValidToEvaluate(Skeleton, RequiredBones);
			if (bIsValidToEvaluate)
			{
				UE_DONT_INLINE_CALL UpdateInternal(Context);
			}
		}
	}

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Alpha"), ActualAlpha);
}

bool ContainsNaN(const TArray<FBoneTransform> & BoneTransforms)
{
	for (int32 i = 0; i < BoneTransforms.Num(); ++i)
	{
		if (BoneTransforms[i].Transform.ContainsNaN())
		{
			return true;
		}
	}

	return false;
}

void FAnimNode_SkeletalControlBase::EvaluateComponentPose_AnyThread(FComponentSpacePoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateComponentPose_AnyThread)
	// Evaluate the input
	ComponentPose.EvaluateComponentSpace(Output);
}

void FAnimNode_SkeletalControlBase::EvaluateComponentSpaceInternal(FComponentSpacePoseContext& Context)
{
}

void FAnimNode_SkeletalControlBase::EvaluateComponentSpace_AnyThread(FComponentSpacePoseContext& Output)
{
	//////////////////////////////////////////////////////////////////////////
	// PERFORMANCE CRITICAL NOTE
	// 
	// This function is called recursively as we traverse nodes, as such, it is critical to keep the
	// amount of stack space used to a minimum as many nodes can be traversed. Using too much stack
	// here can quickly lead to stack overflows.
	// 
	// We explicitly disable inlineing for virtual calls. Normally, virtual calls are never inlined
	// but when PGO and LTO are enabled, the compiler can speculatively de-virtualize the calls. It does
	// this by comparing the v-table pointer and inlineing the call directly in here with a static jump.
	// As a result, it can significantly increase the amount of stack space used.
	//////////////////////////////////////////////////////////////////////////

	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateComponentSpace_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(SkeletalControlBase, !IsInGameThread());

	// Cache the incoming node IDs in a base context
	FAnimationBaseContext CachedContext(Output);

	UE_DONT_INLINE_CALL EvaluateComponentPose_AnyThread(Output);

#if WITH_EDITORONLY_DATA
	// save current pose before applying skeletal control to compute the exact gizmo location in AnimGraphNode
	ForwardedPose.CopyPose(Output.Pose);
#endif // #if WITH_EDITORONLY_DATA

#if DO_CHECK
	// this is to ensure Source data does not contain NaN
	ensure(Output.ContainsNaN() == false);
#endif

	// Apply the skeletal control if it's valid
	if (FAnimWeight::IsRelevant(ActualAlpha))
	{
		const USkeleton* Skeleton = Output.AnimInstanceProxy->GetSkeleton();
		const FBoneContainer& RequiredBones = Output.AnimInstanceProxy->GetRequiredBones();

		bool bIsValidToEvaluate;
		UE_DONT_INLINE_CALL bIsValidToEvaluate = IsValidToEvaluate(Skeleton, RequiredBones);
		if (bIsValidToEvaluate)
		{
			Output.SetNodeIds(CachedContext);

			UE_DONT_INLINE_CALL EvaluateComponentSpaceInternal(Output);

			BoneTransforms.Reset(BoneTransforms.Num());
			UE_DONT_INLINE_CALL EvaluateSkeletalControl_AnyThread(Output, BoneTransforms);

			if (BoneTransforms.Num() > 0)
			{
				const float BlendWeight = FMath::Clamp<float>(ActualAlpha, 0.f, 1.f);
				Output.Pose.LocalBlendCSBoneTransforms(BoneTransforms, BlendWeight);
			}

			// we check NaN when you get out of this function in void FComponentSpacePoseLink::EvaluateComponentSpace(FComponentSpacePoseContext& Output)
		}
	}
}

void FAnimNode_SkeletalControlBase::AddDebugNodeData(FString& OutDebugData)
{
	OutDebugData += FString::Printf(TEXT("Alpha: %.1f%%"), ActualAlpha*100.f);
}

void FAnimNode_SkeletalControlBase::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
}

void FAnimNode_SkeletalControlBase::SetAlpha(float InAlpha)
{
	Alpha = InAlpha;
	ActualAlpha = InAlpha;
}

float FAnimNode_SkeletalControlBase::GetAlpha() const
{
	return ActualAlpha;
}

void FAnimNode_SkeletalControlBase::InitializeAndValidateBoneRef(FBoneReference& BoneRef, const FBoneContainer& RequiredBones)
{
	if (BoneRef.BoneName != NAME_None)
	{
		BoneRef.Initialize(RequiredBones);
	}

	if (!BoneRef.HasValidSetup())
	{
		const FText ErrorText = FText::Format(LOCTEXT("SkeletalControlBoneError", "Referenced Bone {0} does not exist on SkeletalMesh {1}."),
			FText::AsCultureInvariant(BoneRef.BoneName.ToString()),
			FText::AsCultureInvariant(GetNameSafe(RequiredBones.GetSkeletalMeshAsset())));

#if WITH_EDITOR
		AddValidationVisualWarning(ErrorText);
#endif // WITH_EDITOR

		// If the user specified a simulation root that is not used by the skelmesh, issue a warning 
		UE_LOGF(LogSkeletalControlBase, Log, "%ls", *ErrorText.ToString());
	}
}

#if WITH_EDITOR
void FAnimNode_SkeletalControlBase::AddBoneRefMissingVisualWarning(const FString& BoneName, const FString& SkeletalMeshName)
{
	const FText ErrorText = FText::Format(LOCTEXT("SkeletalControlBoneVisualWarning", "Simulation Base Bone {0} does not exist on SkeletalMesh {1}."), FText::FromString(BoneName), FText::FromString(SkeletalMeshName));
	
	AddValidationVisualWarning(ErrorText);
}

void FAnimNode_SkeletalControlBase::AddValidationVisualWarning(FText ValidationVisualWarning)
{
#if WITH_EDITORONLY_DATA
	if (ValidationVisualWarningMessage.IsEmpty())
	{
		ValidationVisualWarningMessage = ValidationVisualWarning;
	}
	else
	{
		ValidationVisualWarningMessage = FText::Format(FText::FromString(TEXT("{0}\n{1}")), ValidationVisualWarningMessage, ValidationVisualWarning);
	}
#endif
}

void FAnimNode_SkeletalControlBase::ClearValidationVisualWarnings()
{
	ValidationVisualWarningMessage = FText::GetEmpty();
}

bool FAnimNode_SkeletalControlBase::HasValidationVisualWarnings() const
{
	return ValidationVisualWarningMessage.IsEmpty() == false;
}

FText FAnimNode_SkeletalControlBase::GetValidationVisualWarningMessage() const
{
	return ValidationVisualWarningMessage;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
