// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_SplineIK.h"
#include "AnimationRuntime.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimStats.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "SplineIK.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_SplineIK)

FAnimNode_SplineIK::FAnimNode_SplineIK() 
	: BoneAxis(ESplineBoneAxis::X)
	, bAutoCalculateSpline(true)
	, PointCount(2)
	, Roll(0.0f)
	, TwistStart(0.0f)
	, TwistEnd(0.0f)
	, Stretch(0.0f)
	, Offset(0.0f)
	, OriginalSplineLength(0.0f)
{
}

void FAnimNode_SplineIK::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	DebugLine += "(";
	AddDebugNodeData(DebugLine);
	DebugLine += FString::Printf(TEXT(" StartBone: %s, EndBone: %s)"), *StartBone.BoneName.ToString(), *EndBone.BoneName.ToString());
	DebugData.AddDebugItem(DebugLine);

	ComponentPose.GatherDebugData(DebugData);
}

void FAnimNode_SplineIK::OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance)
{
	if (InAnimInstance->GetSkelMeshComponent() && InAnimInstance->GetSkelMeshComponent()->GetSkeletalMeshAsset())
	{
		GatherBoneReferences(InAnimInstance->GetSkelMeshComponent()->GetSkeletalMeshAsset()->GetRefSkeleton());
	}
}

struct FSplineIKScratchArea : public TThreadSingleton<FSplineIKScratchArea>
{
	TArray<FTransform> InTransforms;
	TArray<FTransform> OutTransforms;
	TArray<FCompactPoseBoneIndex> CompactPoseBoneIndices;
};

void FAnimNode_SplineIK::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(SplineIK, !IsInGameThread());

	if (CachedBoneReferences.Num() > 0)
	{
		const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();

		TransformSpline();

		const float TotalSplineLength = TransformedSpline.GetSplineLength();
		const float TotalSplineAlpha = TransformedSpline.ReparamTable.Points.Last().OutVal;
		TwistBlend.SetValueRange(TwistStart, TwistEnd);

		TArray<FTransform>& InTransforms = FSplineIKScratchArea::Get().InTransforms;
		TArray<FTransform>& OutTransforms = FSplineIKScratchArea::Get().OutTransforms;
		TArray<FCompactPoseBoneIndex>& CompactPoseBoneIndices = FSplineIKScratchArea::Get().CompactPoseBoneIndices;
		InTransforms.Reset();
		OutTransforms.Reset();
		CompactPoseBoneIndices.Reset();

		const int32 BoneCount = CachedBoneReferences.Num();
		for (int32 BoneIndex = 0; BoneIndex < BoneCount; BoneIndex++)
		{
			const FSplineIKCachedBoneData& BoneData = CachedBoneReferences[BoneIndex];
			if (!BoneData.Bone.IsValidToEvaluate(BoneContainer))
			{
				break;
			}

			int32 CompactPoseIndexIndex = CompactPoseBoneIndices.Add(BoneData.Bone.GetCompactPoseIndex(BoneContainer));
			InTransforms.Add(Output.Pose.GetComponentSpaceTransform(CompactPoseBoneIndices[CompactPoseIndexIndex]));
		}

		AnimationCore::SolveSplineIK(InTransforms, TransformedSpline.Position, TransformedSpline.Rotation, TransformedSpline.Scale, TotalSplineAlpha, TotalSplineLength, FFloatMapping::CreateRaw(this, &FAnimNode_SplineIK::GetTwist, TotalSplineAlpha), Roll, Stretch, Offset, (EAxis::Type)BoneAxis, FFindParamAtFirstSphereIntersection::CreateRaw(this, &FAnimNode_SplineIK::FindParamAtFirstSphereIntersection), CachedOffsetRotations, CachedBoneLengths, OriginalSplineLength, OutTransforms);

		check(InTransforms.Num() == OutTransforms.Num());
		check(InTransforms.Num() == CompactPoseBoneIndices.Num());

		const int32 NumOutputBones = CompactPoseBoneIndices.Num();
		for (int32 OutBoneIndex = 0; OutBoneIndex < NumOutputBones; OutBoneIndex++)
		{
			OutBoneTransforms.Emplace(CompactPoseBoneIndices[OutBoneIndex], OutTransforms[OutBoneIndex]);
		}
	}

	TRACE_ANIM_NODE_VALUE(Output, TEXT("Start Bone"), StartBone.BoneName);
	TRACE_ANIM_NODE_VALUE(Output, TEXT("End Bone"), EndBone.BoneName);
}

bool FAnimNode_SplineIK::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) 
{
	// If any bone references are valid, evaluate.
	if (CachedBoneReferences.Num() > 0)
	{
		for (FSplineIKCachedBoneData& CachedBoneData : CachedBoneReferences)
		{
			if (CachedBoneData.Bone.IsValidToEvaluate(RequiredBones))
			{
				return true;
			}
		}
	}
	
	return false;
}

void FAnimNode_SplineIK::InitializeBoneReferences(const FBoneContainer& RequiredBones) 
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	StartBone.Initialize(RequiredBones);
	EndBone.Initialize(RequiredBones);

	GatherBoneReferences(RequiredBones.GetReferenceSkeleton());

	for (FSplineIKCachedBoneData& CachedBoneData : CachedBoneReferences)
	{
		CachedBoneData.Bone.Initialize(RequiredBones);
	}
}

FTransform FAnimNode_SplineIK::GetTransformedSplinePoint(int32 TransformIndex) const
{
	if (TransformedSpline.Rotation.Points.IsValidIndex(TransformIndex))
	{
		return FTransform(TransformedSpline.Rotation.Points[TransformIndex].OutVal, TransformedSpline.Position.Points[TransformIndex].OutVal, TransformedSpline.Scale.Points[TransformIndex].OutVal);
	}

	return FTransform::Identity;
}

FTransform FAnimNode_SplineIK::GetControlPoint(int32 TransformIndex) const
{
	if (ControlPoints.IsValidIndex(TransformIndex))
	{
		return ControlPoints[TransformIndex];
	}

	return FTransform::Identity;
}

void FAnimNode_SplineIK::SetControlPoint(int32 TransformIndex, const FTransform& InTransform)
{
	if (ControlPoints.IsValidIndex(TransformIndex))
	{
		ControlPoints[TransformIndex] = InTransform; 
	}
}

void FAnimNode_SplineIK::SetControlPointLocation(int32 TransformIndex, const FVector& InLocation)
{
	if (ControlPoints.IsValidIndex(TransformIndex))
	{
		ControlPoints[TransformIndex].SetLocation(InLocation);
	}
}

void FAnimNode_SplineIK::SetControlPointRotation(int32 TransformIndex, const FQuat& InRotation)
{
	if (ControlPoints.IsValidIndex(TransformIndex))
	{
		ControlPoints[TransformIndex].SetRotation(InRotation);
	}
}

void FAnimNode_SplineIK::SetControlPointScale(int32 TransformIndex, const FVector& InScale)
{
	if (ControlPoints.IsValidIndex(TransformIndex))
	{
		ControlPoints[TransformIndex].SetScale3D(InScale);
	}
}

void FAnimNode_SplineIK::TransformSpline()
{
	TransformedSpline.Position.Reset();
	TransformedSpline.Rotation.Reset();
	TransformedSpline.Scale.Reset();

	for (int32 PointIndex = 0; PointIndex < BoneSpline.Position.Points.Num(); PointIndex++)
	{
		FTransform ControlPoint;
		if (ControlPoints.IsValidIndex(PointIndex))
		{
			ControlPoint = ControlPoints[PointIndex];
		}

		FTransform PointTransform;
		PointTransform.SetLocation(BoneSpline.Position.Points[PointIndex].OutVal + ControlPoint.GetLocation());
		PointTransform.SetRotation(ControlPoint.GetRotation() * BoneSpline.Rotation.Points[PointIndex].OutVal);
		PointTransform.SetScale3D(BoneSpline.Scale.Points[PointIndex].OutVal * ControlPoint.GetScale3D());

		TransformedSpline.Position.Points.Emplace(BoneSpline.Position.Points[PointIndex]);
		TransformedSpline.Rotation.Points.Emplace(BoneSpline.Rotation.Points[PointIndex]);
		TransformedSpline.Scale.Points.Emplace(BoneSpline.Scale.Points[PointIndex]);

		TransformedSpline.Position.Points[PointIndex].OutVal = PointTransform.GetLocation();
		TransformedSpline.Rotation.Points[PointIndex].OutVal = PointTransform.GetRotation();
		TransformedSpline.Scale.Points[PointIndex].OutVal = PointTransform.GetScale3D();
	}

	TransformedSpline.UpdateSpline();

	FSplinePositionLinearApproximation::Build(TransformedSpline, LinearApproximation);
}

float FAnimNode_SplineIK::FindParamAtFirstSphereIntersection(const FVector& InOrigin, float InRadius, int32& StartingLinearIndex)
{
	const double RadiusSquared = InRadius * InRadius;
	const int32 LinearCount = LinearApproximation.Num() - 1;
	for (int32 LinearIndex = StartingLinearIndex; LinearIndex < LinearCount; ++LinearIndex)
	{
		const FSplinePositionLinearApproximation& LinearPoint = LinearApproximation[LinearIndex];
		const FSplinePositionLinearApproximation& NextLinearPoint = LinearApproximation[LinearIndex + 1];

		const double InnerDistanceSquared = (InOrigin - LinearPoint.Position).SizeSquared();
		const double OuterDistanceSquared = (InOrigin - NextLinearPoint.Position).SizeSquared();
		if (InnerDistanceSquared <= RadiusSquared && OuterDistanceSquared >= RadiusSquared)
		{
			StartingLinearIndex = LinearIndex;

			const double InnerDistance = FMath::Sqrt(InnerDistanceSquared);
			const double OuterDistance = FMath::Sqrt(OuterDistanceSquared);
			const double InterpParam = FMath::Clamp((InRadius - InnerDistance) / (OuterDistance - InnerDistance), 0.0f, 1.0f);
			
			return FMath::Lerp(LinearPoint.SplineParam, NextLinearPoint.SplineParam, InterpParam);
		}
	}

	StartingLinearIndex = 0;
	return TransformedSpline.ReparamTable.Points.Last().OutVal;
}

void FAnimNode_SplineIK::GatherBoneReferences(const FReferenceSkeleton& RefSkeleton)
{
	CachedBoneReferences.Reset();

	int32 StartIndex = RefSkeleton.FindBoneIndex(StartBone.BoneName);
	int32 EndIndex = RefSkeleton.FindBoneIndex(EndBone.BoneName);

	if (StartIndex != INDEX_NONE && EndIndex != INDEX_NONE)
	{
		// walk up hierarchy towards root from end to start
		int32 BoneIndex = EndIndex;
		while (BoneIndex != StartIndex)
		{
			// we hit the root, so clear the cached bones - we have an invalid chain
			if (BoneIndex == INDEX_NONE)
			{
				CachedBoneReferences.Reset();
				break;
			}

			FName BoneName = RefSkeleton.GetBoneName(BoneIndex);
			CachedBoneReferences.EmplaceAt(0, BoneName, BoneIndex);

			BoneIndex = RefSkeleton.GetParentIndex(BoneIndex);
		}

		if (CachedBoneReferences.Num())
		{
			FName BoneName = RefSkeleton.GetBoneName(StartIndex);
			CachedBoneReferences.EmplaceAt(0, BoneName, StartIndex);

			// reallocate transform array to match bones
			if (bAutoCalculateSpline)
			{
				ControlPoints.SetNum(CachedBoneReferences.Num());
			}
			else
			{
				ControlPoints.SetNum(FMath::Max(2, PointCount));
			}
		}
	}

	BuildBoneSpline(RefSkeleton);
}

void FAnimNode_SplineIK::BuildBoneSpline(const FReferenceSkeleton& RefSkeleton)
{
	if (CachedBoneReferences.Num() > 0)
	{
		TArray<FTransform> ComponentSpaceTransforms;
		FAnimationRuntime::FillUpComponentSpaceTransforms(RefSkeleton, RefSkeleton.GetRefBonePose(), ComponentSpaceTransforms);

		// Build cached bone info
		CachedBoneLengths.Reset();
		CachedOffsetRotations.Reset();
		for (int32 BoneIndex = 0; BoneIndex < CachedBoneReferences.Num(); BoneIndex++)
		{
			double BoneLength = 0.0f;
			FQuat BoneOffsetRotation = FQuat::Identity;

			if (BoneIndex > 0)
			{
				FSplineIKCachedBoneData& BoneData = CachedBoneReferences[BoneIndex];
				const FTransform& Transform = ComponentSpaceTransforms[BoneData.RefSkeletonIndex];

				const FSplineIKCachedBoneData& ParentBoneData = CachedBoneReferences[BoneIndex - 1];
				const FTransform& ParentTransform = ComponentSpaceTransforms[ParentBoneData.RefSkeletonIndex];
				const FVector BoneDir = Transform.GetLocation() - ParentTransform.GetLocation();
				BoneLength = BoneDir.Size();

				// Calculate a quaternion that gets us from our current rotation to the desired one.
				FVector TransformedAxis = Transform.GetRotation().RotateVector(FMatrix::Identity.GetUnitAxis((EAxis::Type)BoneAxis)).GetSafeNormal();
				BoneOffsetRotation = FQuat::FindBetweenNormals(BoneDir.GetSafeNormal(), TransformedAxis);
			}
			
			CachedBoneLengths.Add(static_cast<float>(BoneLength));
			CachedOffsetRotations.Add(BoneOffsetRotation);
		}

		// Setup curve params in component space
		BoneSpline.Position.Reset();
		BoneSpline.Rotation.Reset();
		BoneSpline.Scale.Reset();

		const int32 ClampedPointCount = FMath::Max(2, PointCount);
		if (bAutoCalculateSpline || ClampedPointCount == CachedBoneReferences.Num())
		{
			// We are auto-calculating, so use each bone as a control point
			for (int32 BoneIndex = 0; BoneIndex < CachedBoneReferences.Num(); BoneIndex++)
			{
				FSplineIKCachedBoneData& BoneData = CachedBoneReferences[BoneIndex];
				const float CurveAlpha = (float)BoneIndex;

				const FTransform& Transform = ComponentSpaceTransforms[BoneData.RefSkeletonIndex];
				BoneSpline.Position.Points.Emplace(CurveAlpha, Transform.GetLocation(), FVector::ZeroVector, FVector::ZeroVector, CIM_CurveAuto);
				BoneSpline.Rotation.Points.Emplace(CurveAlpha, Transform.GetRotation(), FQuat::Identity, FQuat::Identity, CIM_Linear);
				BoneSpline.Scale.Points.Emplace(CurveAlpha, Transform.GetScale3D(), FVector::ZeroVector, FVector::ZeroVector, CIM_CurveAuto);
			}
		}
		else
		{
			// We are not auto-calculating, so we need to build an approximation to the curve. First we build a curve using our transformed curve
			// as a temp storage area, then we evaluate the curve at appropriate points to approximate the bone chain with a new cubic.
			TransformedSpline.Position.Reset();
			TransformedSpline.Rotation.Reset();
			TransformedSpline.Scale.Reset();

			// Build the linear spline
			float TotalBoneCount = (float)(CachedBoneReferences.Num() - 1);
			for (int32 BoneIndex = 0; BoneIndex < CachedBoneReferences.Num(); BoneIndex++)
			{
				const FSplineIKCachedBoneData& BoneData = CachedBoneReferences[BoneIndex];
				const float CurveAlpha = (float)BoneIndex / TotalBoneCount;

				const FTransform& Transform = ComponentSpaceTransforms[BoneData.RefSkeletonIndex];
				TransformedSpline.Position.Points.Emplace(CurveAlpha, Transform.GetLocation(), FVector::ZeroVector, FVector::ZeroVector, CIM_CurveAuto);
				TransformedSpline.Rotation.Points.Emplace(CurveAlpha, Transform.GetRotation(), FQuat::Identity, FQuat::Identity, CIM_Linear);
				TransformedSpline.Scale.Points.Emplace(CurveAlpha, Transform.GetScale3D(), FVector::ZeroVector, FVector::ZeroVector, CIM_CurveAuto);
			}

			TransformedSpline.UpdateSpline();

			// now build the approximation

			float TotalPointCount = (float)(ClampedPointCount - 1);
			for (int32 PointIndex = 0; PointIndex < ClampedPointCount; ++PointIndex)
			{
				const float TransformedCurveAlpha = (float)PointIndex / TotalPointCount;
				const float CurveAlpha = (float)PointIndex;

				BoneSpline.Position.Points.Emplace(CurveAlpha, TransformedSpline.Position.Eval(TransformedCurveAlpha), FVector::ZeroVector, FVector::ZeroVector, CIM_CurveAuto);
				BoneSpline.Rotation.Points.Emplace(CurveAlpha, TransformedSpline.Rotation.Eval(TransformedCurveAlpha), FQuat::Identity, FQuat::Identity, CIM_Linear);
				BoneSpline.Scale.Points.Emplace(CurveAlpha, TransformedSpline.Scale.Eval(TransformedCurveAlpha), FVector::ZeroVector, FVector::ZeroVector, CIM_CurveAuto);
			}

			// clear the transformed spline so we dont end up using it
			TransformedSpline.Position.Reset();
			TransformedSpline.Rotation.Reset();
			TransformedSpline.Scale.Reset();
		}

		BoneSpline.UpdateSpline();

		OriginalSplineLength = BoneSpline.GetSplineLength();

		FSplinePositionLinearApproximation::Build(BoneSpline, LinearApproximation);
	}
}

float FAnimNode_SplineIK::GetTwist(float InAlpha, float TotalSplineAlpha)
{
	TwistBlend.SetAlpha(InAlpha / TotalSplineAlpha);
	return TwistBlend.GetBlendedValue();
}

