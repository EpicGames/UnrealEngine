// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_SpringBone.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/WorldSettings.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_SpringBone)

/////////////////////////////////////////////////////
// FAnimNode_SpringBone

FAnimNode_SpringBone::FAnimNode_SpringBone()
	: MaxDisplacement(0.0)
	, SpringStiffness(50.0)
	, SpringDamping(4.0)
	, ErrorResetThresh(256.0)
	, BoneLocation(FVector::ZeroVector)
	, BoneVelocity(FVector::ZeroVector)
	, OwnerVelocity(FVector::ZeroVector)
	, RemainingTime(0.f)
#if WITH_EDITORONLY_DATA
	, bNoZSpring_DEPRECATED(false)
#endif
	, bLimitDisplacement(false)
	, bTranslateX(true)
	, bTranslateY(true)
	, bTranslateZ(true)
	, bRotateX(false)
	, bRotateY(false)
	, bRotateZ(false)
	, bHadValidStrength(false)
	, bOverrideOwnerVelocity(false)
	, OwnerVelocityOverride(FVector::ZeroVector)
{
}

void FAnimNode_SpringBone::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_SkeletalControlBase::Initialize_AnyThread(Context);

	RemainingTime = 0.0f;
}

void FAnimNode_SpringBone::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	FAnimNode_SkeletalControlBase::CacheBones_AnyThread(Context);
}

void FAnimNode_SpringBone::UpdateInternal(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(UpdateInternal)
	FAnimNode_SkeletalControlBase::UpdateInternal(Context);

	RemainingTime += Context.GetDeltaTime();

	// Fixed step simulation at 120hz
	FixedTimeStep = (1.f / 120.f) * TimeDilation;
}

void FAnimNode_SpringBone::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	const float ActualBiasedAlpha = AlphaScaleBias.ApplyTo(Alpha);

	//MDW_TODO Add more output info?
	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("(Alpha: %.1f%% RemainingTime: %.3f)"), ActualBiasedAlpha*100.f, RemainingTime);

	DebugData.AddDebugItem(DebugLine);
	ComponentPose.GatherDebugData(DebugData);
}

FORCEINLINE void CopyToVectorByFlags(FVector& DestVec, const FVector& SrcVec, bool bX, bool bY, bool bZ)
{
	if (bX)
	{
		DestVec.X = SrcVec.X;
	}
	if (bY)
	{
		DestVec.Y = SrcVec.Y;
	}
	if (bZ)
	{
		DestVec.Z = SrcVec.Z;
	}
}

void FAnimNode_SpringBone::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(SpringBone, !IsInGameThread());

	check(OutBoneTransforms.Num() == 0);

	const bool bNoOffset = !bTranslateX && !bTranslateY && !bTranslateZ;
	if (bNoOffset)
	{
		return;
	}

	// Location of our bone in world space
	const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();

	const FCompactPoseBoneIndex SpringBoneIndex = SpringBone.GetCompactPoseIndex(BoneContainer);
	const FTransform& SpaceBase = Output.Pose.GetComponentSpaceTransform(SpringBoneIndex);
	FTransform BoneTransformInWorldSpace = SpaceBase * Output.AnimInstanceProxy->GetComponentTransform();

	FVector const TargetPos = BoneTransformInWorldSpace.GetLocation();

	// Init values first time
	if (RemainingTime == 0.0f)
	{
		BoneLocation = TargetPos;
		BoneVelocity = FVector::ZeroVector;
	}

	if(!FMath::IsNearlyZero(FixedTimeStep, KINDA_SMALL_NUMBER))
	{
		while (RemainingTime > FixedTimeStep)
		{
			// Update location of our base by how much our base moved this frame.
			FVector const BaseTranslation = (OwnerVelocity * FixedTimeStep);
			BoneLocation += BaseTranslation;

			// Reinit values if outside reset threshold
			if (((TargetPos - BoneLocation).SizeSquared() > (ErrorResetThresh*ErrorResetThresh)))
			{
				BoneLocation = TargetPos;
				BoneVelocity = FVector::ZeroVector;
			}

			// Calculate error vector.
			FVector const Error = (TargetPos - BoneLocation);
			FVector const DampingForce = SpringDamping * BoneVelocity;
			FVector const SpringForce = SpringStiffness * Error;

			// Calculate force based on error and vel
			FVector const Acceleration = SpringForce - DampingForce;

			// Integrate velocity
			// Make sure damping with variable frame rate actually dampens velocity. Otherwise Spring will go nuts.
			double const CutOffDampingValue = 1.0 / FixedTimeStep;
			if (SpringDamping > CutOffDampingValue)
			{
				double const SafetyScale = CutOffDampingValue / SpringDamping;
				BoneVelocity += SafetyScale * (Acceleration * FixedTimeStep);
			}
			else
			{
				BoneVelocity += (Acceleration * FixedTimeStep);
			}

			// Clamp velocity to something sane (|dX/dt| <= ErrorResetThresh)
			double const BoneVelocityMagnitude = BoneVelocity.Size();
			if (BoneVelocityMagnitude * FixedTimeStep > ErrorResetThresh)
			{
				BoneVelocity *= (ErrorResetThresh / (BoneVelocityMagnitude * FixedTimeStep));
			}

			// Integrate position
			FVector const OldBoneLocation = BoneLocation;
			FVector const DeltaMove = (BoneVelocity * FixedTimeStep);
			BoneLocation += DeltaMove;

			// Filter out spring translation based on our filter properties
			CopyToVectorByFlags(BoneLocation, TargetPos, !bTranslateX, !bTranslateY, !bTranslateZ);


			// If desired, limit error
			if (bLimitDisplacement)
			{
				FVector CurrentDisp = BoneLocation - TargetPos;
				// Too far away - project back onto sphere around target.
				if (CurrentDisp.SizeSquared() > FMath::Square(MaxDisplacement))
				{
					FVector DispDir = CurrentDisp.GetSafeNormal();
					BoneLocation = TargetPos + (MaxDisplacement * DispDir);
				}
			}

			// Update velocity to reflect post processing done to bone location.
			BoneVelocity = (BoneLocation - OldBoneLocation) / FixedTimeStep;

			check(!BoneLocation.ContainsNaN());
			check(!BoneVelocity.ContainsNaN());

			RemainingTime -= FixedTimeStep;
		}
		LocalBoneTransform = Output.AnimInstanceProxy->GetComponentTransform().InverseTransformPosition(BoneLocation);
	}
	else
	{
		BoneLocation = Output.AnimInstanceProxy->GetComponentTransform().TransformPosition(LocalBoneTransform);
	}
	// Now convert back into component space and output - rotation is unchanged.
	FTransform OutBoneTM = SpaceBase;
	OutBoneTM.SetLocation(LocalBoneTransform);

	const bool bUseRotation = bRotateX || bRotateY || bRotateZ;
	if (bUseRotation)
	{
		FCompactPoseBoneIndex ParentBoneIndex = Output.Pose.GetPose().GetParentBoneIndex(SpringBoneIndex);
		const FTransform& ParentSpaceBase = Output.Pose.GetComponentSpaceTransform(ParentBoneIndex);

		FVector ParentToTarget = (TargetPos - ParentSpaceBase.GetLocation()).GetSafeNormal();
		FVector ParentToCurrent = (BoneLocation - ParentSpaceBase.GetLocation()).GetSafeNormal();

		FQuat AdditionalRotation = FQuat::FindBetweenNormals(ParentToTarget, ParentToCurrent);

		// Filter rotation based on our filter properties
		FVector EularRot = AdditionalRotation.Euler();
		CopyToVectorByFlags(EularRot, FVector::ZeroVector, !bRotateX, !bRotateY, !bRotateZ);

		OutBoneTM.SetRotation(FQuat::MakeFromEuler(EularRot) * OutBoneTM.GetRotation());
	}

	// Output new transform for current bone.
	OutBoneTransforms.Add(FBoneTransform(SpringBoneIndex, OutBoneTM));

	TRACE_ANIM_NODE_VALUE(Output, TEXT("Remaining Time"), RemainingTime);
}


bool FAnimNode_SpringBone::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) 
{
	return (SpringBone.IsValidToEvaluate(RequiredBones));
}

void FAnimNode_SpringBone::InitializeBoneReferences(const FBoneContainer& RequiredBones) 
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	SpringBone.Initialize(RequiredBones);
}

void FAnimNode_SpringBone::PreUpdate(const UAnimInstance* InAnimInstance)
{
	if (const USkeletalMeshComponent* SkelComp = InAnimInstance->GetSkelMeshComponent())
	{
		if (const UWorld* World = SkelComp->GetWorld())
		{
			check(World->GetWorldSettings());
			TimeDilation = World->GetWorldSettings()->GetEffectiveTimeDilation();

			if (bOverrideOwnerVelocity)
			{
				OwnerVelocity = OwnerVelocityOverride;
			}
			else
			{
				AActor* SkelOwner = SkelComp->GetOwner();
				if (SkelComp->GetAttachParent() != nullptr && (SkelOwner == nullptr))
				{
					SkelOwner = SkelComp->GetAttachParent()->GetOwner();
					OwnerVelocity = SkelOwner->GetVelocity();
				}
				else
				{
					OwnerVelocity = FVector::ZeroVector;
				}
			}

		}
	}
}

