// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_Trail.h"
#include "Animation/AnimInstanceProxy.h"
#include "AngularLimit.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_Trail)

/////////////////////////////////////////////////////
// FAnimNode_Trail

DECLARE_CYCLE_STAT(TEXT("Trail Eval"), STAT_Trail_Eval, STATGROUP_Anim);

FAnimNode_Trail::FAnimNode_Trail()
	: ChainLength(2)
	, ChainBoneAxis(EAxis::X)
	, bInvertChainBoneAxis(false)
	, bLimitStretch(false)
	, bLimitRotation(false)
	, bUsePlanarLimit(false)
	, bActorSpaceFakeVel(false)
	, bReorientParentToChild(true)
	, bHadValidStrength(false)
#if WITH_EDITORONLY_DATA
	, bEnableDebug(false)
	, bShowBaseMotion(true)
	, bShowTrailLocation(false)
	, bShowLimit(true)
	, bEditorDebugEnabled(false)
	, DebugLifeTime(0.f)
	, TrailRelaxation_DEPRECATED(10.f)
#endif// #if WITH_EDITORONLY_DATA
	, MaxDeltaTime(0.f)
	, RelaxationSpeedScale(1.f)
	, StretchLimit(0)
	, FakeVelocity(FVector::ZeroVector)
#if WITH_EDITORONLY_DATA
	, TrailBoneRotationBlendAlpha_DEPRECATED(1.f)
#endif // WITH_EDITORONLY_DATA
	, LastBoneRotationAnimAlphaBlend(0.f)
{
	FRichCurve* TrailRelaxRichCurve = TrailRelaxationSpeed.GetRichCurve();
	TrailRelaxRichCurve->AddKey(0.f, 10.f);
	TrailRelaxRichCurve->AddKey(1.f, 5.f);
}

void FAnimNode_Trail::UpdateInternal(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(UpdateInternal)
	FAnimNode_SkeletalControlBase::UpdateInternal(Context);

	ThisTimstep += Context.GetDeltaTime();

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Active Bone"), TrailBone.BoneName);
}

void FAnimNode_Trail::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	DebugLine += "(";
	AddDebugNodeData(DebugLine);
	DebugLine += FString::Printf(TEXT(" Active: %s)"), *TrailBone.BoneName.ToString());
	DebugData.AddDebugItem(DebugLine);

	ComponentPose.GatherDebugData(DebugData);
}


void FAnimNode_Trail::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	SCOPE_CYCLE_COUNTER(STAT_Trail_Eval);

	check(OutBoneTransforms.Num() == 0);
	const float TimeStep = (MaxDeltaTime > 0.f)? FMath::Clamp(ThisTimstep, 0.f, MaxDeltaTime) : ThisTimstep;
	ThisTimstep = 0.f;

	if( ChainBoneIndices.Num() <= 0 )
	{
		return;
	}

	checkSlow (ChainBoneIndices.Num() == ChainLength);
	checkSlow (PerJointTrailData.Num() == ChainLength);
	// The incoming BoneIndex is the 'end' of the spline chain. We need to find the 'start' by walking SplineLength bones up hierarchy.
	// Fail if we walk past the root bone.
	const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();
	const FTransform& ComponentTransform = Output.AnimInstanceProxy->GetComponentTransform();
	FTransform BaseTransform;
 	if (BaseJoint.IsValidToEvaluate(BoneContainer))
 	{
		FCompactPoseBoneIndex BasePoseIndex = BoneContainer.MakeCompactPoseIndex(FMeshPoseBoneIndex(BaseJoint.BoneIndex));
		FTransform BaseBoneTransform = Output.Pose.GetComponentSpaceTransform(BasePoseIndex);
 		BaseTransform = BaseBoneTransform * ComponentTransform;
 	}
	else
	{
		BaseTransform = ComponentTransform;
	}

	OutBoneTransforms.AddZeroed(ChainLength);

	// this should be checked outside
	checkSlow (TrailBone.IsValidToEvaluate(BoneContainer));

	// If we have >0 this frame, but didn't last time, record positions of all the bones.
	// Also do this if number has changed or array is zero.
	//@todo I don't think this will work anymore. if Alpha is too small, it won't call evaluate anyway
	// so this has to change. AFAICT, this will get called only FIRST TIME
	bool bHasValidStrength = (Alpha > 0.f);
	if(bHasValidStrength && !bHadValidStrength)
	{
		for(int32 i=0; i<ChainBoneIndices.Num(); i++)
		{
			if (BoneContainer.Contains(IntCastChecked<FBoneIndexType>(ChainBoneIndices[i])))
			{
				FCompactPoseBoneIndex ChildIndex = BoneContainer.MakeCompactPoseIndex(FMeshPoseBoneIndex(ChainBoneIndices[i]));
				const FTransform& ChainTransform = Output.Pose.GetComponentSpaceTransform(ChildIndex);
				TrailBoneLocations[i] = ChainTransform.GetTranslation();
			}
			else
			{
				TrailBoneLocations[i] = FVector::ZeroVector;
			}
		}
		OldBaseTransform = BaseTransform;
	}
	bHadValidStrength = bHasValidStrength;

	// transform between last frame and now.
	FTransform OldToNewTM = OldBaseTransform.GetRelativeTransform(BaseTransform);

	// Add fake velocity if present to all but root bone
	if(!FakeVelocity.IsZero())
	{
		FVector FakeMovement = -FakeVelocity * TimeStep;

		if (bActorSpaceFakeVel)
		{
			FTransform BoneToWorld(Output.AnimInstanceProxy->GetActorTransform());
			BoneToWorld.RemoveScaling();
			FakeMovement = BoneToWorld.TransformVector(FakeMovement);
		}

		FakeMovement = BaseTransform.InverseTransformVector(FakeMovement);
		// Then add to each bone
		for(int32 i=1; i<TrailBoneLocations.Num(); i++)
		{
			TrailBoneLocations[i] += FakeMovement;
		}
	}

	// Root bone of trail is not modified.
	FCompactPoseBoneIndex RootIndex = BoneContainer.MakeCompactPoseIndex(FMeshPoseBoneIndex(ChainBoneIndices[0])); 
	const FTransform& ChainTransform = Output.Pose.GetComponentSpaceTransform(RootIndex);
	OutBoneTransforms[0] = FBoneTransform(RootIndex, ChainTransform);
	TrailBoneLocations[0] = ChainTransform.GetTranslation();

	TArray<FTransform> DebugPlaneTransforms;
#if WITH_EDITORONLY_DATA
	if (bUsePlanarLimit)
	{
		DebugPlaneTransforms.AddDefaulted(PlanarLimits.Num());
	}
#endif // WITH_EDITORONLY_DATA

	checkSlow(RotationLimits.Num() == ChainLength);
	checkSlow(RotationOffsets.Num() == ChainLength);

	// first solve trail locations
	for (int32 i = 1; i < ChainBoneIndices.Num(); i++)
	{
		// Parent bone position in component space.
		FCompactPoseBoneIndex ParentIndex = BoneContainer.MakeCompactPoseIndex(FMeshPoseBoneIndex(ChainBoneIndices[i - 1]));
		FVector ParentPos = TrailBoneLocations[i - 1];
		FVector ParentAnimPos = Output.Pose.GetComponentSpaceTransform(ParentIndex).GetTranslation();

		// Child bone position in component space.
		FCompactPoseBoneIndex ChildIndex = BoneContainer.MakeCompactPoseIndex(FMeshPoseBoneIndex(ChainBoneIndices[i]));
		FVector ChildPos = OldToNewTM.TransformPosition(TrailBoneLocations[i]); // move from 'last frames component' frame to 'this frames component' frame
		FVector ChildAnimPos = Output.Pose.GetComponentSpaceTransform(ChildIndex).GetTranslation();

		// Desired parent->child offset.
		FVector TargetDelta = (ChildAnimPos - ParentAnimPos);

		// Desired child position.
		FVector ChildTarget = ParentPos + TargetDelta;

		// Find vector from child to target
		FVector Error = (ChildTarget - ChildPos);
		// Calculate how much to push the child towards its target
		const float SpeedScale = RelaxationSpeedScaleInputProcessor.ApplyTo(RelaxationSpeedScale, TimeStep);
		const float Correction = FMath::Clamp<float>(TimeStep * SpeedScale * PerJointTrailData[i].TrailRelaxationSpeedPerSecond, 0.f, 1.f);

		// Scale correction vector and apply to get new world-space child position.
		TrailBoneLocations[i] = ChildPos + (Error * Correction);

		// Limit stretch first
		// If desired, prevent bones stretching too far.
		if (bLimitStretch)
		{
			double RefPoseLength = TargetDelta.Size();
			FVector CurrentDelta = TrailBoneLocations[i] - TrailBoneLocations[i - 1];
			double CurrentLength = CurrentDelta.Size();

			// If we are too far - cut it back (just project towards parent particle).
			if ((CurrentLength - RefPoseLength > StretchLimit) && CurrentLength > UE_DOUBLE_SMALL_NUMBER)
			{
				FVector CurrentDir = CurrentDelta / CurrentLength;
				TrailBoneLocations[i] = TrailBoneLocations[i - 1] + (CurrentDir * (RefPoseLength + StretchLimit));
			}
		}
		
		// set planar limit if used
		if (bUsePlanarLimit)
		{
			for (int32 Index = 0; Index<PlanarLimits.Num(); ++Index)
			{
				const FAnimPhysPlanarLimit& PlanarLimit = PlanarLimits[Index];
				FTransform LimitPlaneTransform = PlanarLimit.PlaneTransform;

				if (PlanarLimit.DrivingBone.IsValidToEvaluate(BoneContainer))
				{
					FCompactPoseBoneIndex DrivingBoneIndex = PlanarLimit.DrivingBone.GetCompactPoseIndex(BoneContainer);

					FTransform DrivingBoneTransform = Output.Pose.GetComponentSpaceTransform(DrivingBoneIndex);
					LimitPlaneTransform *= DrivingBoneTransform;
				}

				FPlane LimitPlane(LimitPlaneTransform.GetLocation(), LimitPlaneTransform.GetUnitAxis(EAxis::Z));
#if WITH_EDITORONLY_DATA				
				DebugPlaneTransforms[Index] = LimitPlaneTransform;
#endif // #if WITH_EDITORONLY_DATA
				const double DistanceFromPlane = LimitPlane.PlaneDot(TrailBoneLocations[i]);
				if (DistanceFromPlane < 0.0)
				{
					TrailBoneLocations[i] -= DistanceFromPlane*FVector(LimitPlane.X, LimitPlane.Y, LimitPlane.Z);
				}
			}
		}

		// Modify child matrix
		OutBoneTransforms[i] = FBoneTransform(ChildIndex, Output.Pose.GetComponentSpaceTransform(ChildIndex));
		OutBoneTransforms[i].Transform.SetTranslation(TrailBoneLocations[i]);

		// reorient parent to child 
		if (bReorientParentToChild)
		{
			FVector CurrentBoneDir = OutBoneTransforms[i - 1].Transform.TransformVector(GetAlignVector(ChainBoneAxis, bInvertChainBoneAxis));
			CurrentBoneDir = CurrentBoneDir.GetSafeNormal(UE_DOUBLE_SMALL_NUMBER);

			// Calculate vector from parent to child.
			FVector DeltaTranslation = OutBoneTransforms[i].Transform.GetTranslation() - OutBoneTransforms[i - 1].Transform.GetTranslation();
			FVector NewBoneDir = FVector(DeltaTranslation).GetSafeNormal(UE_DOUBLE_SMALL_NUMBER);

			// Calculate a quaternion that gets us from our current rotation to the desired one.
			FQuat DeltaLookQuat = FQuat::FindBetweenNormals(CurrentBoneDir, NewBoneDir);
			FQuat ParentRotation = OutBoneTransforms[i - 1].Transform.GetRotation();
			FQuat NewRotation = DeltaLookQuat * ParentRotation;
			if (bLimitRotation)
			{
				// right now we're setting rotation of parent
				// if we want to limit rotation, try limit parent rotation
				FQuat GrandParentRotation = FQuat::Identity;
				if (i == 1)
				{
					const FCompactPoseBoneIndex GrandParentIndex = BoneContainer.GetParentBoneIndex(ParentIndex);
					if (GrandParentIndex != INDEX_NONE)
					{
						GrandParentRotation = Output.Pose.GetComponentSpaceTransform(GrandParentIndex).GetRotation();
					}
				}
				else
				{
					// get local
					GrandParentRotation = OutBoneTransforms[i - 2].Transform.GetRotation();
				}

				// we're fixing up parent local rotation here
				FQuat NewLocalRotation = GrandParentRotation.Inverse() * NewRotation;
				const FQuat& RefRotation = BoneContainer.GetRefPoseTransform(ParentIndex).GetRotation();
				const FRotationLimit& RotationLimit = RotationLimits[i - 1];
				// we limit to ref rotaiton
				if (AnimationCore::ConstrainAngularRangeUsingEuler(NewLocalRotation, RefRotation, RotationLimit.LimitMin + RotationOffsets[i - 1], RotationLimit.LimitMax + RotationOffsets[i - 1]))
				{
					// if we changed rotaiton, let's find new tranlstion
					NewRotation = GrandParentRotation * NewLocalRotation;
					FVector NewTransltion = NewRotation.Vector() * DeltaTranslation.Size();
					// we don't want to go to target, this creates very poppy motion. 
					// @todo: to do this better, I feel we need alpha to blend into external limit and blend back to it
					FVector AdjustedLocation = FMath::Lerp(DeltaTranslation, NewTransltion, Correction) + OutBoneTransforms[i - 1].Transform.GetTranslation();
					OutBoneTransforms[i].Transform.SetTranslation(AdjustedLocation);
					// update new trail location, so that next chain will use this info
					TrailBoneLocations[i] = AdjustedLocation;
				}
			}

			// clamp rotation, but translation is still there - should fix translation
			OutBoneTransforms[i - 1].Transform.SetRotation(NewRotation);
		}
	}

	// For the last bone in the chain, use the rotation from the bone above it.
	FQuat LeafRotation = FQuat::FastLerp(OutBoneTransforms[ChainLength - 2].Transform.GetRotation(), OutBoneTransforms[ChainLength - 1].Transform.GetRotation(), LastBoneRotationAnimAlphaBlend);
	LeafRotation.Normalize();
	OutBoneTransforms[ChainLength - 1].Transform.SetRotation(LeafRotation);

#if WITH_EDITORONLY_DATA
	if (bEnableDebug || bEditorDebugEnabled)
	{
		if (bShowBaseMotion)
		{
			// draw new velocity on new base transform
			FVector PreviousLoc = OldBaseTransform.GetLocation();
			FVector NewLoc = BaseTransform.GetLocation();
			Output.AnimInstanceProxy->AnimDrawDebugDirectionalArrow(PreviousLoc, NewLoc, 5.f, FColor::Red, false, DebugLifeTime);
		}

		if (bShowTrailLocation)
		{
			const int32 TrailNum = TrailBoneLocations.Num();
			if (TrailDebugColors.Num() != TrailNum)
			{
				TrailDebugColors.Reset();
				TrailDebugColors.AddUninitialized(TrailNum);

				for (int32 Index = 0; Index < TrailNum; ++Index)
				{
					TrailDebugColors[Index] = FColor::MakeRandomColor();
				}
			}
			// draw trail positions
			for (int32 Index = 0; Index < TrailNum - 1; ++Index)
			{
				FVector PreviousLoc = ComponentTransform.TransformPosition(TrailBoneLocations[Index]);
				FVector NewLoc = ComponentTransform.TransformPosition(TrailBoneLocations[Index + 1]);
				Output.AnimInstanceProxy->AnimDrawDebugLine(PreviousLoc, NewLoc, TrailDebugColors[Index], false, DebugLifeTime);
			}
		}

		// draw limits
		if (bShowLimit)
		{
			if (bUsePlanarLimit)
			{
				const int32 PlaneLimitNum = DebugPlaneTransforms.Num();
				if (PlaneDebugColors.Num() != PlaneLimitNum)
				{
					PlaneDebugColors.Reset();
					PlaneDebugColors.AddUninitialized(PlaneLimitNum);

					for (int32 Index = 0; Index < PlaneLimitNum; ++Index)
					{
						PlaneDebugColors[Index] = FColor::MakeRandomColor();
					}
				}

				// draw plane info
				for (int32 Index = 0; Index < PlaneLimitNum; ++Index)
				{
					const FTransform& PlaneTransform =  DebugPlaneTransforms[Index];
					FTransform WorldPlaneTransform = PlaneTransform * ComponentTransform;
					Output.AnimInstanceProxy->AnimDrawDebugPlane(WorldPlaneTransform, 40.f, PlaneDebugColors[Index], false, DebugLifeTime, 0.5);
					Output.AnimInstanceProxy->AnimDrawDebugDirectionalArrow(WorldPlaneTransform.GetLocation(),
						WorldPlaneTransform.GetLocation() + WorldPlaneTransform.GetRotation().RotateVector(FVector(0, 0, 40)), 10.f, PlaneDebugColors[Index], false, DebugLifeTime, 0.5f);
				}
				
			}
		}
	}
#endif //#if WITH_EDITORONLY_DATA
	// Update OldBaseTransform
	OldBaseTransform = BaseTransform;
}

bool FAnimNode_Trail::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) 
{
	// if bones are valid
	if (TrailBone.IsValidToEvaluate(RequiredBones))
	{
		for (auto& ChainIndex : ChainBoneIndices)
		{
			if (ChainIndex == INDEX_NONE)
			{
				// unfortunately there is no easy way to communicate this back to user other than spamming here because this gets called every frame
				// originally tried in AnimGraphNode, but that doesn't know hierarchy so I can't verify it there. Maybe should try with USkeleton asset there. @todo
				return false;
			}
			else if (RequiredBones.Contains(IntCastChecked<FBoneIndexType>(ChainIndex)) == false)
			{
				return false;
			}
		}
	}

	return (ChainBoneIndices.Num() > 0);
}

void FAnimNode_Trail::InitializeBoneReferences(const FBoneContainer& RequiredBones) 
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	TrailBone.Initialize(RequiredBones);
	BaseJoint.Initialize(RequiredBones);

	// initialize chain bone indices
	ChainBoneIndices.Reset();
	if (ChainLength > 1 && TrailBone.IsValidToEvaluate(RequiredBones))
	{
		ChainBoneIndices.AddZeroed(ChainLength);

		int32 WalkBoneIndex = TrailBone.BoneIndex;
		ChainBoneIndices[ChainLength - 1] = WalkBoneIndex;

		for(int32 i = 1; i < ChainLength; i++)
		{
			//Insert indices at the start of array, so that parents are before children in the array.
			int32 TransformIndex = ChainLength - (i + 1);

			// if reached to root or invalid, invalidate the data
			if(WalkBoneIndex == INDEX_NONE || WalkBoneIndex == 0)
			{
				ChainBoneIndices[TransformIndex] = INDEX_NONE;
			}
			else
			{
				// Get parent bone.
				WalkBoneIndex = RequiredBones.GetParentBoneIndex(WalkBoneIndex);
				ChainBoneIndices[TransformIndex] = WalkBoneIndex;
			}
		}
	}

	for (FAnimPhysPlanarLimit& PlanarLimit : PlanarLimits)
	{
		PlanarLimit.DrivingBone.Initialize(RequiredBones);
	}
}

FVector FAnimNode_Trail::GetAlignVector(EAxis::Type AxisOption, bool bInvert)
{
	FVector AxisDir;

	if (AxisOption == EAxis::X)
	{
		AxisDir = FVector(1, 0, 0);
	}
	else if (AxisOption == EAxis::Y)
	{
		AxisDir = FVector(0, 1, 0);
	}
	else
	{
		AxisDir = FVector(0, 0, 1);
	}

	if (bInvert)
	{
		AxisDir *= -1.f;
	}

	return AxisDir;
}

void FAnimNode_Trail::PostLoad()
{
#if WITH_EDITORONLY_DATA
	if (TrailRelaxation_DEPRECATED != 10.f)
	{
		FRichCurve* TrailRelaxRichCurve = TrailRelaxationSpeed.GetRichCurve();
		TrailRelaxRichCurve->Reset();
		TrailRelaxRichCurve->AddKey(0.f, TrailRelaxation_DEPRECATED);
		TrailRelaxRichCurve->AddKey(1.f, TrailRelaxation_DEPRECATED);
		// since we don't know if it's same as default or not, we have to keep default
		// if default, the default constructor will take care of it. If not, we'll reset
		TrailRelaxation_DEPRECATED = 10.f;
	}
	EnsureChainSize();
#endif
}

void FAnimNode_Trail::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_SkeletalControlBase::Initialize_AnyThread(Context);

	// allocated all memory here in initialize
	PerJointTrailData.Reset();
	TrailBoneLocations.Reset();
	if(ChainLength > 1)
	{
		PerJointTrailData.AddZeroed(ChainLength);
		TrailBoneLocations.AddZeroed(ChainLength);

		float Interval = (ChainLength > 1)? (1.f/(ChainLength-1)) : 0.f;
		const FRichCurve* TrailRelaxRichCurve = TrailRelaxationSpeed.GetRichCurveConst();
		check(TrailRelaxRichCurve);
		for(int32 Idx=0; Idx<ChainLength; ++Idx)
		{
			PerJointTrailData[Idx].TrailRelaxationSpeedPerSecond = TrailRelaxRichCurve->Eval(Interval * Idx);
		}
	}

	RelaxationSpeedScaleInputProcessor.Reinitialize();
}

#if WITH_EDITOR
void FAnimNode_Trail::EnsureChainSize()
{
	if (RotationLimits.Num() != ChainLength)
	{
		const int32 CurNum = RotationLimits.Num();
		if (CurNum >= ChainLength)
		{
			RotationLimits.SetNum(ChainLength);
			RotationOffsets.SetNum(ChainLength);
		}
		else
		{
			RotationLimits.AddDefaulted(ChainLength - CurNum);
			RotationOffsets.AddZeroed(ChainLength - CurNum);
		}
	}
}

#endif // WITH_EDITOR

