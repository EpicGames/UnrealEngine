// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_TwistCorrectiveNode.h"
#include "AnimationRuntime.h"
#include "Animation/AnimInstanceProxy.h"
#include "Components/SkeletalMeshComponent.h"
#include "UObject/UE5MainStreamObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_TwistCorrectiveNode)

/////////////////////////////////////////////////////
// FAnimNode_TwistCorrectiveNode

FAnimNode_TwistCorrectiveNode::FAnimNode_TwistCorrectiveNode()
	: RangeMax(90.0f)
	, RemappedMin(0.0f)
	, RemappedMax(1.0f)
{
}

void FAnimNode_TwistCorrectiveNode::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	ComponentPose.GatherDebugData(DebugData);
}

FVector FAnimNode_TwistCorrectiveNode::GetReferenceAxis(FCSPose<FCompactPose>& MeshBases, const FReferenceBoneFrame& Reference) const
{
	const FBoneContainer& BoneContainer = MeshBases.GetPose().GetBoneContainer();
	const FTransform ReferencetBoneTransform = MeshBases.GetComponentSpaceTransform(Reference.Bone.GetCompactPoseIndex(BoneContainer));

	return Reference.Axis.GetTransformedAxis(ReferencetBoneTransform);
}

void FAnimNode_TwistCorrectiveNode::EvaluateComponentSpaceInternal(FComponentSpacePoseContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateComponentSpaceInternal)
	// Get the Local space transform and the ref pose transform to see how the transform for the source bone has changed
	const FBoneContainer& BoneContainer = Context.Pose.GetPose().GetBoneContainer();

	// Get the Local space transform and the ref pose transform to see how the transform for the source bone has changed
	const FVector BaseDirVector = GetReferenceAxis(Context.Pose, BaseFrame);
	const FVector TwistDirVector = GetReferenceAxis(Context.Pose, TwistFrame);

	const FTransform ReferencetBoneTransform = Context.Pose.GetComponentSpaceTransform(BaseFrame.Bone.GetCompactPoseIndex(BoneContainer));
	const float CurAngle = GetAngle(BaseDirVector, TwistDirVector, ReferencetBoneTransform);

	// mapping part, [InitialDotValue, MaxDotValue] -> [RemappedMin, RemappedMax]
	// @todo handle RangeMax == ReferenceAngle situation
	const float FinalMappedValue = (FMath::Clamp(CurAngle, ReferenceAngle, RangeMaxInRadian) - ReferenceAngle) * (RemappedMax - RemappedMin) / (RangeMaxInRadian - ReferenceAngle);

	//	set Curve Value
	Context.Curve.Set(CurveName, FinalMappedValue*Alpha);
}

bool FAnimNode_TwistCorrectiveNode::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones)
{
	return BaseFrame.Bone.IsValidToEvaluate(RequiredBones) && BaseFrame.Axis.IsValid() &&
		TwistFrame.Bone.IsValidToEvaluate(RequiredBones) && TwistFrame.Axis.IsValid();
}

void FAnimNode_TwistCorrectiveNode::InitializeBoneReferences(const FBoneContainer& RequiredBones)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	BaseFrame.Bone.Initialize(RequiredBones);
	TwistFrame.Bone.Initialize(RequiredBones);

	BaseFrame.Axis.Initialize();
	TwistFrame.Axis.Initialize();

	RangeMaxInRadian = FMath::DegreesToRadians(RangeMax);
}

void FAnimNode_TwistCorrectiveNode::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_SkeletalControlBase::Initialize_AnyThread(Context);
}

void FAnimNode_TwistCorrectiveNode::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	FAnimNode_SkeletalControlBase::CacheBones_AnyThread(Context);
	TArray<FTransform> SpaceBases;

	const FTransform ComponentToWorld = Context.AnimInstanceProxy->GetSkelMeshComponent()->GetComponentToWorld();
	const FBoneContainer& BoneContainer = Context.AnimInstanceProxy->GetRequiredBones();
	const FTransform BaseFrameTransform = FAnimationRuntime::GetComponentSpaceRefPose(BaseFrame.Bone.GetCompactPoseIndex(BoneContainer), BoneContainer) * ComponentToWorld;
	const FTransform TwistFrameTransform = FAnimationRuntime::GetComponentSpaceRefPose(TwistFrame.Bone.GetCompactPoseIndex(BoneContainer), BoneContainer) * ComponentToWorld;

	// get 
	const FVector BaseAxis = BaseFrame.Axis.GetTransformedAxis(BaseFrameTransform);
	const FVector TwistAxis = TwistFrame.Axis.GetTransformedAxis(TwistFrameTransform);

	FCSPose<FCompactPose> MeshRefPose;

	ReferenceAngle = GetAngle(BaseAxis, TwistAxis, BaseFrameTransform);
}

float FAnimNode_TwistCorrectiveNode::GetAngle(const FVector& Base, const FVector& Twist, const FTransform& ReferencetBoneTransform) const
{
	// this angle claculate should happen from plane normal axis
	// that defines direction of angle calculation
	// firsrt get base frame component space bone transform
	// now place a plane in that space
	FVector TwistPlaneNormal = TwistPlaneNormalAxis.GetTransformedAxis(ReferencetBoneTransform);

	// find out if it's same direction. If not, we're clamped
	float BaseAngle = 0.0, TwistAngle = 0.0;

	// if facing same direction we care
	float BaseDotProduct = static_cast<float>(FVector::DotProduct(TwistPlaneNormal, Base));
	if (BaseDotProduct > 0.0)
	{
		// http://www.vitutor.com/geometry/distance/line_plane.html
		BaseAngle = FMath::Asin(BaseDotProduct);
	}

	float TwistDotProduct = static_cast<float>(FVector::DotProduct(TwistPlaneNormal, Twist));
	if (TwistDotProduct > 0.0)
	{
		// http://www.vitutor.com/geometry/distance/line_plane.html
		TwistAngle = FMath::Asin(TwistDotProduct);
	}

	return TwistAngle - BaseAngle;
}

bool FAnimNode_TwistCorrectiveNode::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	return false;
}

void FAnimNode_TwistCorrectiveNode::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if(Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::AnimationRemoveSmartNames)
	{
		CurveName = Curve_DEPRECATED.Name;
	}
#endif
}

/////////////////////////////////////////////////////////


