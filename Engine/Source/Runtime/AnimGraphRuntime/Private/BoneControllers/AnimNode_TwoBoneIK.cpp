// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_TwoBoneIK.h"
#include "Engine/Engine.h"
#include "AnimationRuntime.h"
#include "Components/SkeletalMeshComponent.h"
#include "Materials/Material.h"
#include "TwoBoneIK.h"
#include "AnimationCoreLibrary.h"
#include "Animation/AnimInstanceProxy.h"
#include "PrimitiveDrawingUtils.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MaterialShared.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_TwoBoneIK)

DECLARE_CYCLE_STAT(TEXT("TwoBoneIK Eval"), STAT_TwoBoneIK_Eval, STATGROUP_Anim);


/////////////////////////////////////////////////////
// FAnimNode_TwoBoneIK

FAnimNode_TwoBoneIK::FAnimNode_TwoBoneIK()
	: StartStretchRatio(1.f)
	, MaxStretchScale(1.2f)
#if WITH_EDITORONLY_DATA
	, StretchLimits_DEPRECATED(FVector2D::ZeroVector)
	, bNoTwist_DEPRECATED(false)
#endif
	, EffectorLocation(FVector::ZeroVector)
	, CachedUpperLimbIndex(INDEX_NONE)
	, JointTargetLocation(FVector::ZeroVector)
	, CachedLowerLimbIndex(INDEX_NONE)
	, EffectorLocationSpace(BCS_ComponentSpace)
	, JointTargetLocationSpace(BCS_ComponentSpace)
	, bAllowStretching(false)
	, bTakeRotationFromEffectorSpace(false)
	, bMaintainEffectorRelRot(false)
	, bAllowTwist(true)
{
}

void FAnimNode_TwoBoneIK::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	DebugLine += "(";
	AddDebugNodeData(DebugLine);
	DebugLine += FString::Printf(TEXT(" IKBone: %s)"), *IKBone.BoneName.ToString());
	DebugData.AddDebugItem(DebugLine);

	ComponentPose.GatherDebugData(DebugData);
}

FTransform FAnimNode_TwoBoneIK::GetTargetTransform(const FTransform& InComponentTransform, FCSPose<FCompactPose>& MeshBases, FBoneSocketTarget& InTarget, EBoneControlSpace Space, const FVector& InOffset) 
{
	FTransform OutTransform;
	if (Space == BCS_BoneSpace)
	{
		OutTransform = InTarget.GetTargetTransform(InOffset, MeshBases, InComponentTransform);
	}
	else
	{
		// parent bone space still goes through this way
		// if your target is socket, it will try find parents of joint that socket belongs to
		OutTransform.SetLocation(InOffset);
		FAnimationRuntime::ConvertBoneSpaceTransformToCS(InComponentTransform, MeshBases, OutTransform, InTarget.GetCompactPoseBoneIndex(), Space);
	}

	return OutTransform;
}

void FAnimNode_TwoBoneIK::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	SCOPE_CYCLE_COUNTER(STAT_TwoBoneIK_Eval);

	check(OutBoneTransforms.Num() == 0);

	const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();

	// Get indices of the lower and upper limb bones and check validity.
	bool bInvalidLimb = false;

	FCompactPoseBoneIndex IKBoneCompactPoseIndex = IKBone.GetCompactPoseIndex(BoneContainer);

	const bool bInBoneSpace = (EffectorLocationSpace == BCS_ParentBoneSpace) || (EffectorLocationSpace == BCS_BoneSpace);

	// Get Local Space transforms for our bones. We do this first in case they already are local.
	// As right after we get them in component space. (And that does the auto conversion).
	// We might save one transform by doing local first...
	const FTransform EndBoneLocalTransform = Output.Pose.GetLocalSpaceTransform(IKBoneCompactPoseIndex);
	const FTransform LowerLimbLocalTransform = Output.Pose.GetLocalSpaceTransform(CachedLowerLimbIndex);
	const FTransform UpperLimbLocalTransform = Output.Pose.GetLocalSpaceTransform(CachedUpperLimbIndex);

	// Now get those in component space...
	FTransform LowerLimbCSTransform = Output.Pose.GetComponentSpaceTransform(CachedLowerLimbIndex);
	FTransform UpperLimbCSTransform = Output.Pose.GetComponentSpaceTransform(CachedUpperLimbIndex);
	FTransform EndBoneCSTransform = Output.Pose.GetComponentSpaceTransform(IKBoneCompactPoseIndex);

	// Get current position of root of limb.
	// All position are in Component space.
	const FVector RootPos = UpperLimbCSTransform.GetTranslation();
	const FVector InitialJointPos = LowerLimbCSTransform.GetTranslation();
	const FVector InitialEndPos = EndBoneCSTransform.GetTranslation();

	// Transform EffectorLocation from EffectorLocationSpace to ComponentSpace.
	FTransform EffectorTransform = GetTargetTransform(Output.AnimInstanceProxy->GetComponentTransform(), Output.Pose, EffectorTarget, EffectorLocationSpace, EffectorLocation);

	// Get joint target (used for defining plane that joint should be in).
	FTransform JointTargetTransform = GetTargetTransform(Output.AnimInstanceProxy->GetComponentTransform(), Output.Pose, JointTarget, JointTargetLocationSpace, JointTargetLocation);

	FVector	JointTargetPos = JointTargetTransform.GetTranslation();

	// This is our reach goal.
	FVector DesiredPos = EffectorTransform.GetTranslation();

	// IK solver
	UpperLimbCSTransform.SetLocation(RootPos);
	LowerLimbCSTransform.SetLocation(InitialJointPos);
	EndBoneCSTransform.SetLocation(InitialEndPos);

	AnimationCore::SolveTwoBoneIK(UpperLimbCSTransform, LowerLimbCSTransform, EndBoneCSTransform, JointTargetPos, DesiredPos, bAllowStretching, StartStretchRatio, MaxStretchScale);

#if WITH_EDITOR
	CachedJointTargetPos = JointTargetPos;
	CachedJoints[0] = UpperLimbCSTransform.GetLocation();
	CachedJoints[1] = LowerLimbCSTransform.GetLocation();
	CachedJoints[2] = EndBoneCSTransform.GetLocation();
#endif // WITH_EDITOR

	// if no twist, we clear twist from each limb
	if (!bAllowTwist)
	{
		auto RemoveTwist = [this](const FTransform& InParentTransform, FTransform& InOutTransform, const FTransform& OriginalLocalTransform, const FVector& InAlignVector) 
		{
			FTransform LocalTransform = InOutTransform.GetRelativeTransform(InParentTransform);
			FQuat LocalRotation = LocalTransform.GetRotation();
			FQuat NewTwist, NewSwing;
			LocalRotation.ToSwingTwist(InAlignVector, NewSwing, NewTwist);
			NewSwing.Normalize();

			// get new twist from old local
			LocalRotation = OriginalLocalTransform.GetRotation();
			FQuat OldTwist, OldSwing;
			LocalRotation.ToSwingTwist(InAlignVector, OldSwing, OldTwist);
			OldTwist.Normalize();

			InOutTransform.SetRotation(InParentTransform.GetRotation() * NewSwing * OldTwist);
			InOutTransform.NormalizeRotation();
		};

		const FCompactPoseBoneIndex UpperLimbParentIndex = BoneContainer.GetParentBoneIndex(CachedUpperLimbIndex);
		FVector AlignDir = TwistAxis.GetTransformedAxis(FTransform::Identity);
		if (UpperLimbParentIndex != INDEX_NONE)
		{
			FTransform UpperLimbParentTransform = Output.Pose.GetComponentSpaceTransform(UpperLimbParentIndex);
			RemoveTwist(UpperLimbParentTransform, UpperLimbCSTransform, UpperLimbLocalTransform, AlignDir);
		}
			
		RemoveTwist(UpperLimbCSTransform, LowerLimbCSTransform, LowerLimbLocalTransform, AlignDir);
	}
	
	// Update transform for upper bone.
	{
		// Order important. First bone is upper limb.
		OutBoneTransforms.Add( FBoneTransform(CachedUpperLimbIndex, UpperLimbCSTransform) );
	}

	// Update transform for lower bone.
	{
		// Order important. Second bone is lower limb.
		OutBoneTransforms.Add( FBoneTransform(CachedLowerLimbIndex, LowerLimbCSTransform) );
	}

	// Update transform for end bone.
	{
		// only allow bTakeRotationFromEffectorSpace during bone space
		if (bInBoneSpace && bTakeRotationFromEffectorSpace)
		{
			EndBoneCSTransform.SetRotation(EffectorTransform.GetRotation());
		}
		else if (bMaintainEffectorRelRot)
		{
			EndBoneCSTransform = EndBoneLocalTransform * LowerLimbCSTransform;
		}
		// Order important. Third bone is End Bone.
		OutBoneTransforms.Add(FBoneTransform(IKBoneCompactPoseIndex, EndBoneCSTransform));
	}

	// Make sure we have correct number of bones
	check(OutBoneTransforms.Num() == 3);

	TRACE_ANIM_NODE_VALUE(Output, TEXT("IK Bone"), IKBone.BoneName);
}

bool FAnimNode_TwoBoneIK::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) 
{
	if (!IKBone.IsValidToEvaluate(RequiredBones))
	{
		return false;
	}
	
	if (CachedUpperLimbIndex == INDEX_NONE || CachedLowerLimbIndex == INDEX_NONE)
	{
		return false;
	}

	// check bone space here
	if (EffectorLocationSpace == BCS_ParentBoneSpace || EffectorLocationSpace == BCS_BoneSpace)
	{
		if (!EffectorTarget.IsValidToEvaluate(RequiredBones))
		{
			return false;
		}
	}

	if (JointTargetLocationSpace == BCS_ParentBoneSpace || JointTargetLocationSpace == BCS_BoneSpace)
	{
		if (!JointTarget.IsValidToEvaluate(RequiredBones))
		{
			return false;
		}
	}

	return true;
}

void FAnimNode_TwoBoneIK::InitializeBoneReferences(const FBoneContainer& RequiredBones) 
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	IKBone.Initialize(RequiredBones);

	EffectorTarget.InitializeBoneReferences(RequiredBones);
	JointTarget.InitializeBoneReferences(RequiredBones);

	FCompactPoseBoneIndex IKBoneCompactPoseIndex = IKBone.GetCompactPoseIndex(RequiredBones);
	CachedLowerLimbIndex = FCompactPoseBoneIndex(INDEX_NONE);
	CachedUpperLimbIndex = FCompactPoseBoneIndex(INDEX_NONE);
	if (IKBoneCompactPoseIndex != INDEX_NONE)
	{
		CachedLowerLimbIndex = RequiredBones.GetParentBoneIndex(IKBoneCompactPoseIndex);
		if (CachedLowerLimbIndex != INDEX_NONE)
		{
			CachedUpperLimbIndex = RequiredBones.GetParentBoneIndex(CachedLowerLimbIndex);
		}
	}

}

void FAnimNode_TwoBoneIK::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	Super::Initialize_AnyThread(Context);
	EffectorTarget.Initialize(Context.AnimInstanceProxy);
	JointTarget.Initialize(Context.AnimInstanceProxy);
}

#if WITH_EDITOR
// can't use World Draw functions because this is called from Render of viewport, AFTER ticking component, 
// which means LineBatcher already has ticked, so it won't render anymore
// to use World Draw functions, we have to call this from tick of actor
void FAnimNode_TwoBoneIK::ConditionalDebugDraw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* MeshComp) const
{
	FTransform LocalToWorld = MeshComp->GetComponentToWorld();
	FVector WorldPosition[3];
	WorldPosition[0] = LocalToWorld.TransformPosition(CachedJoints[0]);
	WorldPosition[1] = LocalToWorld.TransformPosition(CachedJoints[1]);
	WorldPosition[2] = LocalToWorld.TransformPosition(CachedJoints[2]);
	const FVector JointTargetInWorld = LocalToWorld.TransformPosition(CachedJointTargetPos);

	DrawTriangle(PDI, WorldPosition[0], WorldPosition[1], WorldPosition[2], GEngine->DebugEditorMaterial->GetRenderProxy(), SDPG_World);
	PDI->DrawLine(WorldPosition[0], JointTargetInWorld, FLinearColor::Red, SDPG_Foreground);
	PDI->DrawLine(WorldPosition[1], JointTargetInWorld, FLinearColor::Red, SDPG_Foreground);
	PDI->DrawLine(WorldPosition[2], JointTargetInWorld, FLinearColor::Red, SDPG_Foreground);
}
#endif // WITH_EDITOR
