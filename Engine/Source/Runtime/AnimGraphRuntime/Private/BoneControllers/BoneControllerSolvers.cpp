// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/BoneControllerSolvers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BoneControllerSolvers)

FTransform FIKFootPelvisPullDownSolver::Solve(FTransform PelvisTransform, TArrayView<const float> FKFootDistancesToPelvis, TArrayView<const FVector> IKFootLocations, float DeltaTime)
{
	const FVector InitialPelvisLocation = PelvisTransform.GetLocation();
	
	const int32 IKFootLocationsCount = IKFootLocations.Num();
	const int32 FKFootDistancesCount = FKFootDistancesToPelvis.Num();
	check(IKFootLocationsCount > 0);
	check(FKFootDistancesCount > 0);
	check(FKFootDistancesCount == IKFootLocationsCount);

	FVector AdjustedPelvisLocation = InitialPelvisLocation;
	FVector DeltaAdjustment = FVector::ZeroVector;

	const double PerFootWeight = 1.0 / static_cast<double>(IKFootLocationsCount);
	const double AdjustmentDistMaxSquared = FMath::Pow(PelvisAdjustmentMaxDistance, 2.0);

	// Pull pelvis closer to feet iteratively
	for (int32 Iter = 0; Iter < PelvisAdjustmentMaxIter; ++Iter)
	{
		const FVector PreAdjustmentLocation = AdjustedPelvisLocation;
		AdjustedPelvisLocation = FVector::ZeroVector;

		// Apply pelvis adjustment contributions from all IK/FK foot chains
		for (int32 Index = 0; Index < IKFootLocationsCount; ++Index)
		{
			const FVector IdealLocation = IKFootLocations[Index] + (PreAdjustmentLocation - IKFootLocations[Index]).GetSafeNormal() * FKFootDistancesToPelvis[Index];
			AdjustedPelvisLocation += (IdealLocation * PerFootWeight);
		}

		const FVector PrevDeltaAdjustment = DeltaAdjustment;
		DeltaAdjustment = AdjustedPelvisLocation - InitialPelvisLocation;
		const double DeltaAdjustmentDist = FVector::Dist(PrevDeltaAdjustment, DeltaAdjustment);

		// Keep track of how much delta adjustment is being applied per iteration
		if (DeltaAdjustmentDist <= PelvisAdjustmentErrorTolerance)
		{
			break;
		}
	}

	// Apply spring between initial and adjusted spring location to smooth out change over time
	//PelvisAdjustmentInterp
	PelvisAdjustmentInterp.Update(DeltaAdjustment, DeltaTime);

	// Apply an alpha with the initial pelvis location, to retain some of the original motion
	AdjustedPelvisLocation = InitialPelvisLocation + FMath::Lerp(FVector::ZeroVector, PelvisAdjustmentInterp.GetPosition(), PelvisAdjustmentInterpAlpha);

	// Guarantee that we don't over-adjust the pelvis beyond the specified distance tolerance
	if (FVector::DistSquared(AdjustedPelvisLocation, InitialPelvisLocation) >= AdjustmentDistMaxSquared)
	{
		DeltaAdjustment = AdjustedPelvisLocation - InitialPelvisLocation;
		AdjustedPelvisLocation = InitialPelvisLocation + DeltaAdjustment.GetSafeNormal() * PelvisAdjustmentMaxDistance;
	}
	
	PelvisTransform.SetLocation(AdjustedPelvisLocation);
	return PelvisTransform;
}
