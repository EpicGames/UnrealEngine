// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/BoneControllerTypes.h"
#include "Animation/AnimInstanceProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BoneControllerTypes)

FVector FWarpingVectorValue::AsComponentSpaceDirection(const FAnimInstanceProxy* AnimInstanceProxy, const FTransform& IKFootRootTransform) const
{
	switch (Mode)
	{
	case EWarpingVectorMode::ComponentSpaceVector:
		return Value.GetSafeNormal();
	case EWarpingVectorMode::ActorSpaceVector:
	{
		const FVector WorldSpaceDirection = AnimInstanceProxy->GetActorTransform().TransformVectorNoScale(Value);
		const FVector ComponentSpaceDirection = AnimInstanceProxy->GetComponentTransform().InverseTransformVectorNoScale(WorldSpaceDirection);
		return ComponentSpaceDirection.GetSafeNormal();
	}
	case EWarpingVectorMode::WorldSpaceVector:
		return AnimInstanceProxy->GetComponentTransform().InverseTransformVectorNoScale(Value).GetSafeNormal();
	case EWarpingVectorMode::IKFootRootLocalSpaceVector:
		return IKFootRootTransform.TransformVectorNoScale(Value.GetSafeNormal());
	}
#if WITH_EDITORONLY_DATA
	const UEnum* TypeEnum = FindObject<UEnum>(nullptr, TEXT("/Script/AnimGraphRuntime.EWarpingVectorMode"));
	check(!!TypeEnum);
	ensureMsgf(false, TEXT("AsComponentSpaceDirection specified an unhandled EWarpingVectorMode: %s"), *(TypeEnum->GetNameStringByIndex(static_cast<int32>(Mode))));
#endif
	return Value;
}
