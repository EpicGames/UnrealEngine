// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonAnimationLibrary.h"
#include "AnimationCoreLibrary.h"

#define LOCTEXT_NAMESPACE "CommonAnimationLibrary"

//////////////////////////////////////////////////////////////////////////

float CommonAnimationLibrary::ScalarEasing(float Value, const FRuntimeFloatCurve& CustomCurve, EEasingFuncType EasingType, bool bFlip, float Weight)
{
	const float Original = Value;
	float Result = Value;

	if(bFlip)
	{
		Value = 1.f - Value;
	}

	switch(EasingType)
	{
		case EEasingFuncType::Linear:
		{
			Result = FMath::Clamp<float>(Value, 0.f, 1.f);
			break;
		}
		case EEasingFuncType::Sinusoidal:
		{
			Result = FMath::Clamp<float>((FMath::Sin(Value * UE_PI - UE_HALF_PI) + 1.f) / 2.f, 0.f, 1.f);
			break;
		}
		case EEasingFuncType::Cubic:
		{
			Result = FMath::Clamp<float>(FMath::CubicInterp<float>(0.f, 0.f, 1.f, 0.f, Value), 0.f, 1.f);
			break;
		}
		case EEasingFuncType::QuadraticInOut:
		{
			Result = FMath::Clamp<float>(FMath::InterpEaseInOut<float>(0.f, 1.f, Value, 2), 0.f, 1.f);
			break;
		}
		case EEasingFuncType::CubicInOut:
		{
			Result = FMath::Clamp<float>(FMath::InterpEaseInOut<float>(0.f, 1.f, Value, 3), 0.f, 1.f);
			break;
		}
		case EEasingFuncType::HermiteCubic:
		{
			Result = FMath::Clamp<float>(FMath::SmoothStep(0.0f, 1.0f, Value), 0.0f, 1.0f);
			break;
		}
		case EEasingFuncType::QuarticInOut:
		{
			Result = FMath::Clamp<float>(FMath::InterpEaseInOut<float>(0.f, 1.f, Value, 4), 0.f, 1.f);
			break;
		}
		case EEasingFuncType::QuinticInOut:
		{
			Result = FMath::Clamp<float>(FMath::InterpEaseInOut<float>(0.f, 1.f, Value, 5), 0.f, 1.f);
			break;
		}
		case EEasingFuncType::CircularIn:
		{
			Result = FMath::Clamp<float>(FMath::InterpCircularIn<float>(0.0f, 1.0f, Value), 0.0f, 1.0f);
			break;
		}
		case EEasingFuncType::CircularOut:
		{
			Result = FMath::Clamp<float>(FMath::InterpCircularOut<float>(0.0f, 1.0f, Value), 0.0f, 1.0f);
			break;
		}
		case EEasingFuncType::CircularInOut:
		{
			Result = FMath::Clamp<float>(FMath::InterpCircularInOut<float>(0.0f, 1.0f, Value), 0.0f, 1.0f);
			break;
		}
		case EEasingFuncType::ExpIn:
		{
			Result = FMath::Clamp<float>(FMath::InterpExpoIn<float>(0.0f, 1.0f, Value), 0.0f, 1.0f);
			break;
		}
		case EEasingFuncType::ExpOut:
		{
			Result = FMath::Clamp<float>(FMath::InterpExpoOut<float>(0.0f, 1.0f, Value), 0.0f, 1.0f);
			break;
		}
		case EEasingFuncType::ExpInOut:
		{
			Result = FMath::Clamp<float>(FMath::InterpExpoInOut<float>(0.0f, 1.0f, Value), 0.0f, 1.0f);
			break;
		}
		case EEasingFuncType::CustomCurve:
		{
			if (CustomCurve.GetRichCurveConst() != nullptr)
			{
				Result = CustomCurve.GetRichCurveConst()->Eval(Value, Value);
			}
			break;
		}
	}

	if(bFlip)
	{
		Result = 1.f - Result;
	}

	return FMath::Lerp<float>(Original, Result, Weight);
}

FVector CommonAnimationLibrary::RetargetSingleLocation(
	FVector Location,
	const FTransform& Source,
	const FTransform& Target,
	const FRuntimeFloatCurve& CustomCurve,
	EEasingFuncType EasingType,
	bool bFlipEasing,
	float EasingWeight,
	FVector Axis,
	float SourceMinimum,
	float SourceMaximum,
	float TargetMinimum,
	float TargetMaximum
)
{
	const float SourceDelta = SourceMaximum - SourceMinimum;
	const float TargetDelta = TargetMaximum - TargetMinimum;
	if (FMath::IsNearlyEqual(SourceDelta, 0.f) || FMath::IsNearlyEqual(TargetDelta, 0.f))
	{
		return Location;
	}

	const float AxisLengthSquared =static_cast<float>(Axis.SizeSquared());
	if (FMath::IsNearlyEqual(AxisLengthSquared, 0.f))
	{
		Axis = FVector(1.f, 0.f, 0.f);
	}
	else if (!FMath::IsNearlyEqual(AxisLengthSquared, 1.f))
	{
		// normalize only in this case
		Axis = Axis * (1.f / sqrt(AxisLengthSquared));
	}

	Location = Source.InverseTransformPosition(Location);

	const float Delta = static_cast<float>(FVector::DotProduct(Location, Axis));
	float OutBias = (Delta - SourceMinimum) / SourceDelta;
	if (EasingType != EEasingFuncType::Linear)
	{
		OutBias = ScalarEasing(OutBias, CustomCurve, EasingType, bFlipEasing, EasingWeight);
	}
	const float TargetBias = FMath::Lerp<float>(TargetMinimum, TargetMaximum, OutBias);

	Location = Location + Axis * (TargetBias - Delta);

	return Target.TransformPosition(Location);
}

FQuat CommonAnimationLibrary::RetargetSingleRotation(
	const FQuat& RotationIn,
	const FTransform& Source,
	const FTransform& Target,
	const FRuntimeFloatCurve& CustomCurve,
	EEasingFuncType EasingType,
	bool bFlipEasing,
	float EasingWeight,
	ERotationComponent RotationComponent,
	FVector TwistAxis,
	bool bUseAbsoluteAngle,
	float SourceMinimum,
	float SourceMaximum,
	float TargetMinimum,
	float TargetMaximum
)
{
	FQuat Rotation = RotationIn;

	const float SourceDelta = SourceMaximum - SourceMinimum;
	const float TargetDelta = TargetMaximum - TargetMinimum;
	if (FMath::IsNearlyEqual(SourceDelta, 0.f) || FMath::IsNearlyEqual(TargetDelta, 0.f))
	{
		return Rotation;
	}

	Rotation = Source.InverseTransformRotation(Rotation);

	float Angle = 0.f;
	FQuat Swing, Twist;
	FVector Euler;

	switch (RotationComponent)
	{
		case ERotationComponent::EulerX:
		{
			Euler = Rotation.Euler();
			Angle = static_cast<float>(Euler.X);
			break;
		}
		case ERotationComponent::EulerY:
		{
			Euler = Rotation.Euler();
			Angle = static_cast<float>(Euler.Y);
			break;
		}
		case ERotationComponent::EulerZ:
		{
			Euler = Rotation.Euler();
			Angle = static_cast<float>(Euler.Z);
			break;
		}
		case ERotationComponent::QuaternionAngle:
		{
			Angle = static_cast<float>(FMath::RadiansToDegrees(Rotation.GetAngle()));
			break;
		}
		case ERotationComponent::SwingAngle:
		case ERotationComponent::TwistAngle:
		{
			const float AxisLengthSquared = static_cast<float>(TwistAxis.SizeSquared());
			if (FMath::IsNearlyEqual(AxisLengthSquared, 0.f))
			{
				TwistAxis = FVector(1.f, 0.f, 0.f);
			}
			else
			{
				TwistAxis.Normalize();
			}

			Rotation.ToSwingTwist(TwistAxis, Swing, Twist);
			if (RotationComponent == ERotationComponent::SwingAngle)
			{
				Angle = FMath::RadiansToDegrees(static_cast<float>(Swing.GetAngle()));
			}
			else
			{
				Angle = FMath::RadiansToDegrees(static_cast<float>(Twist.GetAngle()));
			}
			break;
		}
	}

	const float AngleSign = Angle < 0.f ? -1.f : 1.f;
	if (bUseAbsoluteAngle)
	{
		Angle = FMath::Abs<float>(Angle);
	}

	float OutBias = (Angle - SourceMinimum) / SourceDelta;
	const float TempBias = OutBias;
	if (EasingType != EEasingFuncType::Linear)
	{
		OutBias = ScalarEasing(OutBias, CustomCurve, EasingType, bFlipEasing, EasingWeight);
	}

	float TargetAngle = FMath::Lerp<float>(TargetMinimum, TargetMaximum, OutBias);
	if (bUseAbsoluteAngle)
	{
		TargetAngle *= AngleSign;
	}

	switch (RotationComponent)
	{
		case ERotationComponent::EulerX:
		{
			Euler.X = TargetAngle;
			Rotation = FQuat::MakeFromEuler(Euler);
			break;
		}
		case ERotationComponent::EulerY:
		{
			Euler.Y = TargetAngle;
			Rotation = FQuat::MakeFromEuler(Euler);
			break;
		}
		case ERotationComponent::EulerZ:
		{
			Euler.Z = TargetAngle;
			Rotation = FQuat::MakeFromEuler(Euler);
			break;
		}
		case ERotationComponent::QuaternionAngle:
		{
			Rotation = FQuat(Rotation.GetRotationAxis(), FMath::DegreesToRadians(TargetAngle));
			break;
		}
		case ERotationComponent::SwingAngle:
		{
			Rotation = Twist * FQuat(Swing.GetRotationAxis(), FMath::DegreesToRadians(TargetAngle));
			break;
		}
		case ERotationComponent::TwistAngle:
		{
			Rotation = FQuat(Twist.GetRotationAxis(), FMath::DegreesToRadians(TargetAngle)) * Swing;
			break;
		}
	}

	Rotation.Normalize();
	return Target.TransformRotation(Rotation);
}

#undef LOCTEXT_NAMESPACE

