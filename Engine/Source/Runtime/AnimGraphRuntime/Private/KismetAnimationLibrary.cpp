// Copyright Epic Games, Inc. All Rights Reserved.

#include "KismetAnimationLibrary.h"
#include "CommonAnimationLibrary.h"
#include "AnimationCoreLibrary.h"
#include "Blueprint/BlueprintSupport.h"
#include "Components/SkeletalMeshComponent.h"
#include "TwoBoneIK.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(KismetAnimationLibrary)

#define LOCTEXT_NAMESPACE "UKismetAnimationLibrary"

//////////////////////////////////////////////////////////////////////////
// UKismetAnimationLibrary

const FName AnimationLibraryWarning = FName("Animation Library");

UKismetAnimationLibrary::UKismetAnimationLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FBlueprintSupport::RegisterBlueprintWarning(
		FBlueprintWarningDeclaration(
			AnimationLibraryWarning,
			LOCTEXT("AnimationLibraryWarning", "Animation Library Warning")
		)
	);
}

void UKismetAnimationLibrary::K2_TwoBoneIK(const FVector& RootPos, const FVector& JointPos, const FVector& EndPos, const FVector& JointTarget, const FVector& Effector, FVector& OutJointPos, FVector& OutEndPos, bool bAllowStretching, float StartStretchRatio, float MaxStretchScale)
{
	AnimationCore::SolveTwoBoneIK(RootPos, JointPos, EndPos, JointTarget, Effector, OutJointPos, OutEndPos, bAllowStretching, StartStretchRatio, MaxStretchScale);
}

FTransform UKismetAnimationLibrary::K2_LookAt(const FTransform& CurrentTransform, const FVector& TargetPosition, FVector AimVector, bool bUseUpVector, FVector UpVector, float ClampConeInDegree)
{
	if (AimVector.IsNearlyZero())
	{
		// aim vector should be normalized
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("AimVector should not be zero. Please specify which direction.")), ELogVerbosity::Warning, AnimationLibraryWarning);
		return FTransform::Identity;
	}

	if (bUseUpVector && UpVector.IsNearlyZero())
	{
		// upvector has to be normalized
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("LookUpVector should not be zero. Please specify which direction.")), ELogVerbosity::Warning, AnimationLibraryWarning);
		bUseUpVector = false;
	}

	if (ClampConeInDegree < 0.f || ClampConeInDegree > 180.f)
	{
		// ClampCone is out of range, it will be clamped to (0.f, 180.f)
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("ClampConeInDegree should range from (0, 180). ")), ELogVerbosity::Warning, AnimationLibraryWarning);
	}

	FQuat DiffRotation = AnimationCore::SolveAim(CurrentTransform, TargetPosition, AimVector.GetSafeNormal(), bUseUpVector, UpVector.GetSafeNormal(), ClampConeInDegree);
	FTransform NewTransform = CurrentTransform;
	NewTransform.SetRotation(DiffRotation);
	return NewTransform;
}

float UKismetAnimationLibrary::K2_DistanceBetweenTwoSocketsAndMapRange(const USkeletalMeshComponent* Component, const FName SocketOrBoneNameA, ERelativeTransformSpace SocketSpaceA, const FName SocketOrBoneNameB, ERelativeTransformSpace SocketSpaceB, bool bRemapRange, float InRangeMin, float InRangeMax, float OutRangeMin, float OutRangeMax)
{
	if (Component && SocketOrBoneNameA != NAME_None && SocketOrBoneNameB != NAME_None)
	{
		FTransform SocketTransformA = Component->GetSocketTransform(SocketOrBoneNameA, SocketSpaceA);
		FTransform SocketTransformB = Component->GetSocketTransform(SocketOrBoneNameB, SocketSpaceB);

		const float Distance = static_cast<float>((SocketTransformB.GetLocation() - SocketTransformA.GetLocation()).Size());

		if (bRemapRange)
		{
			return FMath::GetMappedRangeValueClamped(FVector2f(InRangeMin, InRangeMax), FVector2f(OutRangeMin, OutRangeMax), Distance);
		}
		else
		{
			return Distance;
		}
	}

	return 0.f;
}

FVector UKismetAnimationLibrary::K2_DirectionBetweenSockets(const USkeletalMeshComponent* Component, const FName SocketOrBoneNameFrom, const FName SocketOrBoneNameTo)
{
	if (Component && SocketOrBoneNameFrom != NAME_None && SocketOrBoneNameTo != NAME_None)
	{
		FTransform SocketTransformFrom = Component->GetSocketTransform(SocketOrBoneNameFrom, RTS_World);
		FTransform SocketTransformTo = Component->GetSocketTransform(SocketOrBoneNameTo, RTS_World);

		return (SocketTransformTo.GetLocation() - SocketTransformFrom.GetLocation());
	}

	return FVector(0.f);
}

FVector UKismetAnimationLibrary::K2_MakePerlinNoiseVectorAndRemap(float X, float Y, float Z, float RangeOutMinX, float RangeOutMaxX, float RangeOutMinY, float RangeOutMaxY, float RangeOutMinZ, float RangeOutMaxZ)
{
	FVector OutVector;
	OutVector.X = K2_MakePerlinNoiseAndRemap(X, RangeOutMinX, RangeOutMaxX);
	OutVector.Y = K2_MakePerlinNoiseAndRemap(Y, RangeOutMinY, RangeOutMaxY);
	OutVector.Z = K2_MakePerlinNoiseAndRemap(Z, RangeOutMinZ, RangeOutMaxZ);
	return OutVector;
}

float UKismetAnimationLibrary::K2_MakePerlinNoiseAndRemap(float Value, float RangeOutMin, float RangeOutMax)
{
	// perlin noise output is always from [-1, 1]
	return FMath::GetMappedRangeValueClamped(FVector2f(-1.f, 1.f), FVector2f(RangeOutMin, RangeOutMax), FMath::PerlinNoise1D(Value));
}

float UKismetAnimationLibrary::K2_CalculateVelocityFromPositionHistory(
	float DeltaSeconds,
	FVector Position,
	UPARAM(ref) FPositionHistory& History,
	int32 NumberOfSamples,
	float VelocityMin,
	float VelocityMax
) {
	NumberOfSamples = FMath::Max<uint32>(NumberOfSamples, 2);
	if (DeltaSeconds <= 0.0f)
	{
		return 0.f;
	}

	// if the number of samples changes down clear the history
	if (History.Positions.Num() > NumberOfSamples)
	{
		History.Positions.Reset();
		History.Velocities.Reset();
		History.LastIndex = 0;
	}

	if (History.Positions.Num() != History.Velocities.Num())
	{
		UE_LOGF(LogAnimation, Warning, "Position History has a different number of position and velocity samples.");
		return 0.f;
	}

	// append to the history until it's full and then loop around when filling it 
	// to reuse the memory
	if (History.Positions.Num() == 0)
	{
		History.Positions.Reserve(NumberOfSamples);
		History.Velocities.Reserve(NumberOfSamples);
		History.Positions.Add(Position);
		History.Velocities.Add(0.f);
		History.LastIndex = 0;
		return 0.f;
	}
	else
	{
		const float LengthOfV = static_cast<float>(((Position - History.Positions[History.LastIndex]) / DeltaSeconds).Size());

		if (History.Positions.Num() == NumberOfSamples)
		{
			int32 NextIndex = History.LastIndex + 1;
			if (NextIndex == History.Positions.Num())
			{
				NextIndex = 0;
			}
			History.Positions[NextIndex] = Position;
			History.Velocities[NextIndex] = LengthOfV;
			History.LastIndex = NextIndex;
		}
		else
		{
			History.LastIndex = History.Positions.Num();
			History.Positions.Add(Position);
			History.Velocities.Add(LengthOfV);
		}
	}

	// compute average velocity
	float LengthOfV = 0.0f;
	for (int32 i = 0; i < History.Velocities.Num(); i++)
	{
		LengthOfV += History.Velocities[i];
	}

	// Avoids NaN due to the FMath::Max instruction above.
	LengthOfV /= float(History.Velocities.Num());

	if (VelocityMin < 0.0f || VelocityMax < 0.0f || VelocityMax <= VelocityMin)
	{
		return LengthOfV;
	}

	// Avoids NaN due to the condition above.
	return FMath::Clamp((LengthOfV - VelocityMin) / (VelocityMax - VelocityMin), 0.f, 1.f);
}

float UKismetAnimationLibrary::K2_CalculateVelocityFromSockets(
	float DeltaSeconds,
	USkeletalMeshComponent * Component,
	const FName SocketOrBoneName,
	const FName FrameOfReference,
	ERelativeTransformSpace SocketSpace,
	FVector OffsetInBoneSpace,
	UPARAM(ref) FPositionHistory& History,
	int32 NumberOfSamples,
	float VelocityMin,
	float VelocityMax,
	EEasingFuncType EasingType,
	const FRuntimeFloatCurve& CustomCurve
) {
	if (Component && SocketOrBoneName != NAME_None)
	{
		FTransform SocketTransform = Component->GetSocketTransform(SocketOrBoneName, SocketSpace);
		if (FrameOfReference != NAME_None)
		{
			// make the bone's / socket's transform relative to the frame of reference.
			FTransform FrameOfReferenceTransform = Component->GetSocketTransform(FrameOfReference, SocketSpace);
			SocketTransform = SocketTransform.GetRelativeTransform(FrameOfReferenceTransform);
		}

		FVector Position = SocketTransform.TransformPosition(OffsetInBoneSpace);
		float Velocity = K2_CalculateVelocityFromPositionHistory(DeltaSeconds, Position, History, NumberOfSamples, VelocityMin, VelocityMax);
		return CommonAnimationLibrary::ScalarEasing(Velocity, CustomCurve, EasingType);
	}

	return VelocityMin;
}

struct FK2ProfilingTimer
{
	double LastTime;
	double AccummulatedTime;
};

class FProfilingTimerPerThread : public TThreadSingleton<FProfilingTimerPerThread>
{
public:
	TArray<FK2ProfilingTimer> ProfilingTimers;
};

void UKismetAnimationLibrary::K2_StartProfilingTimer()
{
	FK2ProfilingTimer Timer;
	Timer.LastTime = FPlatformTime::Seconds() * 1000.0;
	Timer.AccummulatedTime = 0.0;
	FProfilingTimerPerThread::Get().ProfilingTimers.Add(Timer);
}

float UKismetAnimationLibrary::K2_EndProfilingTimer(bool bLog, const FString& LogPrefix)
{
	TArray<FK2ProfilingTimer>& ProfilingTimers = FProfilingTimerPerThread::Get().ProfilingTimers;
	
	if (ProfilingTimers.Num() == 0)
	{
		UE_LOGF(LogAnimation, Warning, "Unbalanced use of Start & End Profiling Timer nodes.");
		return 0.f;
	}

	FK2ProfilingTimer Timer = ProfilingTimers.Pop();
	double CurrentTimer = FPlatformTime::Seconds() * 1000.0;
	Timer.AccummulatedTime = CurrentTimer - Timer.LastTime;

	float Delta = (float)Timer.AccummulatedTime;
	if (bLog)
	{
		if (LogPrefix.IsEmpty())
		{
			UE_LOGF(LogAnimation, Warning, "%.03f ms", Delta);
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "[%ls] %.03f ms", *LogPrefix, Delta);
		}
	}

	return Delta;
}

float UKismetAnimationLibrary::CalculateDirection(const FVector& Velocity, const FRotator& BaseRotation)
{
	if (!Velocity.IsNearlyZero())
	{
		const FMatrix RotMatrix = FRotationMatrix(BaseRotation);
		const FVector ForwardVector = RotMatrix.GetScaledAxis(EAxis::X);
		const FVector RightVector = RotMatrix.GetScaledAxis(EAxis::Y);
		const FVector NormalizedVel = Velocity.GetSafeNormal2D();

		// get a cos(alpha) of forward vector vs velocity
		const float ForwardCosAngle = static_cast<float>(FVector::DotProduct(ForwardVector, NormalizedVel));
		// now get the alpha and convert to degree
		float ForwardDeltaDegree = FMath::RadiansToDegrees(FMath::Acos(ForwardCosAngle));

		// depending on where right vector is, flip it
		const float RightCosAngle = static_cast<float>(FVector::DotProduct(RightVector, NormalizedVel));
		if (RightCosAngle < 0.f)
		{
			ForwardDeltaDegree *= -1.f;
		}

		return ForwardDeltaDegree;
	}

	return 0.f;
}

#undef LOCTEXT_NAMESPACE


