// Copyright Epic Games, Inc. All Rights Reserved.

#include "LayeredBoneBlendLibrary.h"
#include "Animation/AnimInstanceProxy.h"
#include "AnimNodes/AnimNode_LayeredBoneBlend.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LayeredBoneBlendLibrary)

FLayeredBoneBlendReference ULayeredBoneBlendLibrary::ConvertToLayeredBoneBlend(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result)
{
	return FAnimNodeReference::ConvertToType<FLayeredBoneBlendReference>(Node, Result);
}

int32 ULayeredBoneBlendLibrary::GetNumPoses(const FLayeredBoneBlendReference& LayeredBoneBlend)
{
	int32 NumPoses = 0;
	
	LayeredBoneBlend.CallAnimNodeFunction<FAnimNode_LayeredBoneBlend>(
		TEXT("GetNumPoses"),
		[&NumPoses](const FAnimNode_LayeredBoneBlend& InLayeredBoneBlend)
		{
			NumPoses = InLayeredBoneBlend.BlendPoses.Num();
		});

	return NumPoses;
}

FLayeredBoneBlendReference ULayeredBoneBlendLibrary::SetBlendMask(const FAnimUpdateContext& UpdateContext, const FLayeredBoneBlendReference& LayeredBoneBlend, int32 PoseIndex, FName BlendMaskName)
{
	LayeredBoneBlend.CallAnimNodeFunction<FAnimNode_LayeredBoneBlend>(
		TEXT("SetBlendMask"),
		[&UpdateContext, PoseIndex, BlendMaskName](FAnimNode_LayeredBoneBlend& InLayeredBoneBlend)
		{
			const FAnimationUpdateContext* AnimationUpdateContext = UpdateContext.GetContext();
			if(AnimationUpdateContext == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "SetBlendMask: Invalid update context provided.");
				return;
			}

			USkeleton* Skeleton = AnimationUpdateContext->AnimInstanceProxy->GetSkeleton();
			if(Skeleton == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "SetBlendMask: Invalid skeleton.");
				return;
			}

			if(InLayeredBoneBlend.BlendMode != ELayeredBoneBlendMode::BlendMask)
			{
				UE_LOGF(LogAnimation, Warning, "SetBlendMask: Node is not set to use blend masks.");
				return;
			}

			if(!InLayeredBoneBlend.BlendPoses.IsValidIndex(PoseIndex))
			{
				UE_LOGF(LogAnimation, Warning, "SetBlendMask: Invalid pose index %d provided (%d present).", PoseIndex, InLayeredBoneBlend.BlendPoses.Num());
				return;
			}
			
			UBlendProfile* BlendMask = Skeleton->GetBlendProfile(BlendMaskName);
			if(BlendMask == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "SetBlendMask: Blend mask '%ls' not found.", *BlendMaskName.ToString());
				return;
			}

			if(BlendMask->Mode != EBlendProfileMode::BlendMask)
			{
				UE_LOGF(LogAnimation, Warning, "SetBlendMask: Blend mask '%ls' does not use BlendMask mode.", *BlendMaskName.ToString());
				return;
			}

			InLayeredBoneBlend.SetBlendMask(PoseIndex, BlendMask);
		});

	return LayeredBoneBlend;
}
