// Copyright Epic Games, Inc. All Rights Reserved.

#include "LinkedAnimGraphLibrary.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimNode_LinkedAnimGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LinkedAnimGraphLibrary)

FLinkedAnimGraphReference ULinkedAnimGraphLibrary::ConvertToLinkedAnimGraph(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result)
{
	return FAnimNodeReference::ConvertToType<FLinkedAnimGraphReference>(Node, Result);
}

bool ULinkedAnimGraphLibrary::HasLinkedAnimInstance(const FLinkedAnimGraphReference& Node)
{
	bool bResult = false;
	
	Node.CallAnimNodeFunction<FAnimNode_LinkedAnimGraph>(
		TEXT("HasLinkedAnimInstance"),
		[&bResult](FAnimNode_LinkedAnimGraph& LinkedAnimGraphNode)
		{
			bResult = LinkedAnimGraphNode.GetTargetInstance<UAnimInstance>() != nullptr;
		});

	return bResult;
}

UAnimInstance* ULinkedAnimGraphLibrary::GetLinkedAnimInstance(const FLinkedAnimGraphReference& Node)
{
	UAnimInstance* Instance = nullptr;
	
	Node.CallAnimNodeFunction<FAnimNode_LinkedAnimGraph>(
		TEXT("GetLinkedAnimInstance"),
		[&Instance](FAnimNode_LinkedAnimGraph& LinkedAnimGraphNode)
		{
			Instance = LinkedAnimGraphNode.GetTargetInstance<UAnimInstance>();
		});

	return Instance;
}
