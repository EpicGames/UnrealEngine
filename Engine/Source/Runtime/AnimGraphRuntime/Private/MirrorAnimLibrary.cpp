// Copyright Epic Games, Inc. All Rights Reserved.

#include "MirrorAnimLibrary.h"
#include "AnimNodes/AnimNode_Mirror.h"
#include UE_INLINE_GENERATED_CPP_BY_NAME(MirrorAnimLibrary)

DEFINE_LOG_CATEGORY_STATIC(LogMirrorNodeLibrary, Verbose, All);

FMirrorAnimNodeReference UMirrorAnimLibrary::ConvertToMirrorNode(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result)
{
	return FAnimNodeReference::ConvertToType<FMirrorAnimNodeReference>(Node, Result);
}

FMirrorAnimNodeReference UMirrorAnimLibrary::SetMirror(const FMirrorAnimNodeReference& MirrorNode, bool bInMirror)
{
	MirrorNode.CallAnimNodeFunction<FAnimNode_MirrorBase>(
		TEXT("SetMirror"),[bInMirror](FAnimNode_MirrorBase& InMirrorBaseNode)
		{
			if (!InMirrorBaseNode.SetMirror(bInMirror))
			{
				UE_LOGF(LogMirrorNodeLibrary, Warning, "Could not set Mirror flag on mirror anim node, value is not dynamic. Set it as Always Dynamic.");
			}
		});
	return MirrorNode;
}

FMirrorAnimNodeReference UMirrorAnimLibrary::SetMirrorTransitionBlendTime(const FMirrorAnimNodeReference& MirrorNode, float InBlendTime)
{
	MirrorNode.CallAnimNodeFunction<FAnimNode_MirrorBase>(
		TEXT("SetBlendTimeOnMirrorStateChange"),[InBlendTime](FAnimNode_MirrorBase& InMirrorBaseNode)
		{
			if (!InMirrorBaseNode.SetBlendTimeOnMirrorStateChange(InBlendTime))
			{
				UE_LOGF(LogMirrorNodeLibrary, Warning, "Could not set Mirror Transition Blend Time on mirror anim node, value is not dynamic. Set it as Always Dynamic.");
			}
		});
	return MirrorNode;
}

bool UMirrorAnimLibrary::GetMirror(const FMirrorAnimNodeReference& MirrorNode)
{
	bool bIsMirrored = false;
	
	MirrorNode.CallAnimNodeFunction<FAnimNode_MirrorBase>(
		TEXT("GetMirror"),[&bIsMirrored](FAnimNode_MirrorBase& InMirrorBaseNode)
		{
			bIsMirrored = InMirrorBaseNode.GetMirror();
		});
	return bIsMirrored;
}

UMirrorDataTable* UMirrorAnimLibrary::GetMirrorDataTable(const FMirrorAnimNodeReference& MirrorNode)
{
	UMirrorDataTable* MirrorDataTable = nullptr;
	
	MirrorNode.CallAnimNodeFunction<FAnimNode_MirrorBase>(
		TEXT("GetMirrorDataTable"),[&MirrorDataTable](FAnimNode_MirrorBase& InMirrorBaseNode)
		{
			MirrorDataTable = InMirrorBaseNode.GetMirrorDataTable();
		});
	return MirrorDataTable;
}

float UMirrorAnimLibrary::GetMirrorTransitionBlendTime(const FMirrorAnimNodeReference& MirrorNode)
{
	float BlendTime = false;
	
	MirrorNode.CallAnimNodeFunction<FAnimNode_MirrorBase>(
		TEXT("GetBlendTimeOnMirrorStateChange"),[&BlendTime](FAnimNode_MirrorBase& InMirrorBaseNode)
		{
			BlendTime = InMirrorBaseNode.GetBlendTimeOnMirrorStateChange();
		});
	return BlendTime;
}