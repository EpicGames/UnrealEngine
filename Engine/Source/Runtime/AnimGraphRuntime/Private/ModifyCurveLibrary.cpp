// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModifyCurveLibrary.h"
#include "AnimNodes/AnimNode_ModifyCurve.h"
#include UE_INLINE_GENERATED_CPP_BY_NAME(ModifyCurveLibrary)

DEFINE_LOG_CATEGORY_STATIC(LogModifyCurveNodeLibrary, Verbose, All);

FModifyCurveAnimNodeReference UModifyCurveAnimLibrary::ConvertToModifyCurveNode(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result)
{
	return FAnimNodeReference::ConvertToType<FModifyCurveAnimNodeReference>(Node, Result);
}

FModifyCurveAnimNodeReference UModifyCurveAnimLibrary::SetCurveMap(const FModifyCurveAnimNodeReference& ModifyCurveNode, const TMap<FName, float> & InCurveMap)
{
	ModifyCurveNode.CallAnimNodeFunction<FAnimNode_ModifyCurve>(TEXT("SetCurveMap"),[&InCurveMap](FAnimNode_ModifyCurve& InModifyCurveNode)
	{
		InModifyCurveNode.CurveMap.Append(InCurveMap);
	});
	
	return ModifyCurveNode;
}

EModifyCurveApplyMode UModifyCurveAnimLibrary::GetApplyMode(const FModifyCurveAnimNodeReference& ModifyCurveNode)
{
	EModifyCurveApplyMode OutMode = EModifyCurveApplyMode::Blend;
	
	ModifyCurveNode.CallAnimNodeFunction<FAnimNode_ModifyCurve>(TEXT("GetApplyMode"),[&OutMode](FAnimNode_ModifyCurve& InModifyCurveNode)
	{
		OutMode = InModifyCurveNode.ApplyMode;
	});
		
	return OutMode;
}

FModifyCurveAnimNodeReference UModifyCurveAnimLibrary::SetApplyMode(const FModifyCurveAnimNodeReference& ModifyCurveNode, EModifyCurveApplyMode InMode)
{
	ModifyCurveNode.CallAnimNodeFunction<FAnimNode_ModifyCurve>(TEXT("SetApplyMode"),[InMode](FAnimNode_ModifyCurve& InModifyCurveNode)
	{
		InModifyCurveNode.ApplyMode = InMode;
	});
	
	return ModifyCurveNode;
}

float UModifyCurveAnimLibrary::GetAlpha(const FModifyCurveAnimNodeReference& ModifyCurveNode)
{
	float OutAlpha = 0.0f;

	ModifyCurveNode.CallAnimNodeFunction<FAnimNode_ModifyCurve>(TEXT("GetAlpha"), [&OutAlpha](const FAnimNode_ModifyCurve& InModifyCurveNode)
	{
		OutAlpha = InModifyCurveNode.Alpha;
	});
	
	return OutAlpha;
}

FModifyCurveAnimNodeReference UModifyCurveAnimLibrary::SetAlpha(const FModifyCurveAnimNodeReference& ModifyCurveNode, float InAlpha)
{
	ModifyCurveNode.CallAnimNodeFunction<FAnimNode_ModifyCurve>(TEXT("SetAlpha"), [InAlpha](FAnimNode_ModifyCurve& InModifyCurveNode)
	{
		InModifyCurveNode.Alpha = InAlpha;
	});
	
	return ModifyCurveNode;
}
