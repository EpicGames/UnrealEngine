// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayMontageCallbackProxy.h"
#include "Animation/AnimMontage.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlayMontageCallbackProxy)

//////////////////////////////////////////////////////////////////////////
// UPlayMontageCallbackProxy

UPlayMontageCallbackProxy::UPlayMontageCallbackProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, MontageInstanceID(INDEX_NONE)
	, bInterruptedCalledBeforeBlendingOut(false)
{
}

UPlayMontageCallbackProxy* UPlayMontageCallbackProxy::CreateProxyObjectForPlayMontage(
	class USkeletalMeshComponent* InSkeletalMeshComponent,
	class UAnimMontage* MontageToPlay,
	float PlayRate,
	float StartingPosition,
	FName StartingSection,
	bool bShouldStopAllMontages)
{
	UPlayMontageCallbackProxy* Proxy = NewObject<UPlayMontageCallbackProxy>();
	Proxy->SetFlags(RF_StrongRefOnFrame);
	Proxy->PlayMontage(InSkeletalMeshComponent, MontageToPlay, PlayRate, StartingPosition, StartingSection, bShouldStopAllMontages);
	return Proxy;
}


bool UPlayMontageCallbackProxy::PlayMontage(class USkeletalMeshComponent* InSkeletalMeshComponent, 
	class UAnimMontage* MontageToPlay, 
	float PlayRate, 
	float StartingPosition, 
	FName StartingSection,
	bool bShouldStopAllMontages)
{
	bool bPlayedSuccessfully = false;
	if (InSkeletalMeshComponent)
	{
		if (UAnimInstance* AnimInstance = InSkeletalMeshComponent->GetAnimInstance())
		{
			const float MontageLength = AnimInstance->Montage_Play(MontageToPlay, PlayRate, EMontagePlayReturnType::MontageLength, StartingPosition, bShouldStopAllMontages);
			bPlayedSuccessfully = (MontageLength > 0.f);

			if (bPlayedSuccessfully)
			{
				AnimInstancePtr = AnimInstance;
				if (FAnimMontageInstance* MontageInstance = AnimInstance->GetActiveInstanceForMontage(MontageToPlay))
				{
					MontageInstanceID = MontageInstance->GetInstanceID();
				}

				if (StartingSection != NAME_None)
				{
					AnimInstance->Montage_JumpToSection(StartingSection, MontageToPlay);
				}

				BlendingOutDelegate.BindUObject(this, &UPlayMontageCallbackProxy::OnMontageBlendingOut);
				AnimInstance->Montage_SetBlendingOutDelegate(BlendingOutDelegate, MontageToPlay);

				MontageEndedDelegate.BindUObject(this, &UPlayMontageCallbackProxy::OnMontageEnded);
				AnimInstance->Montage_SetEndDelegate(MontageEndedDelegate, MontageToPlay);

				AnimInstance->OnPlayMontageNotifyBegin.AddDynamic(this, &UPlayMontageCallbackProxy::OnNotifyBeginReceived);
				AnimInstance->OnPlayMontageNotifyEnd.AddDynamic(this, &UPlayMontageCallbackProxy::OnNotifyEndReceived);
			}
		}
	}

	if (!bPlayedSuccessfully)
	{
		OnInterrupted.Broadcast(NAME_None);
	}

	return bPlayedSuccessfully;
}

bool UPlayMontageCallbackProxy::IsNotifyValid(FName NotifyName, const FBranchingPointNotifyPayload& BranchingPointNotifyPayload) const
{
	return ((MontageInstanceID != INDEX_NONE) && (BranchingPointNotifyPayload.MontageInstanceID == MontageInstanceID));
}


void UPlayMontageCallbackProxy::OnNotifyBeginReceived(FName NotifyName, const FBranchingPointNotifyPayload& BranchingPointNotifyPayload)
{
	if (IsNotifyValid(NotifyName, BranchingPointNotifyPayload))
	{
		OnNotifyBegin.Broadcast(NotifyName);
	}
}


void UPlayMontageCallbackProxy::OnNotifyEndReceived(FName NotifyName, const FBranchingPointNotifyPayload& BranchingPointNotifyPayload)
{
	if (IsNotifyValid(NotifyName, BranchingPointNotifyPayload))
	{
		OnNotifyEnd.Broadcast(NotifyName);
	}
}


void UPlayMontageCallbackProxy::OnMontageBlendingOut(UAnimMontage* Montage, bool bInterrupted)
{
	if (bInterrupted)
	{
		OnInterrupted.Broadcast(NAME_None);
		bInterruptedCalledBeforeBlendingOut = true;
	}
	else
	{
		OnBlendOut.Broadcast(NAME_None);
	}
}

void UPlayMontageCallbackProxy::OnMontageEnded(UAnimMontage* Montage, bool bInterrupted)
{
	if (!bInterrupted)
	{
		OnCompleted.Broadcast(NAME_None);
	}
	else if (!bInterruptedCalledBeforeBlendingOut)
	{
		OnInterrupted.Broadcast(NAME_None);
	}

	UnbindDelegates();
}

void UPlayMontageCallbackProxy::UnbindDelegates()
{
	if (UAnimInstance* AnimInstance = AnimInstancePtr.Get())
	{
		AnimInstance->OnPlayMontageNotifyBegin.RemoveDynamic(this, &UPlayMontageCallbackProxy::OnNotifyBeginReceived);
		AnimInstance->OnPlayMontageNotifyEnd.RemoveDynamic(this, &UPlayMontageCallbackProxy::OnNotifyEndReceived);
	}
}

void UPlayMontageCallbackProxy::BeginDestroy()
{
	UnbindDelegates();

	Super::BeginDestroy();
}
