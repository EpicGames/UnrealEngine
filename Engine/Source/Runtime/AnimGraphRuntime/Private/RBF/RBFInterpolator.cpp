// Copyright Epic Games, Inc. All Rights Reserved.

#include "RBF/RBFInterpolator.h"
#include "AutoRTFM.h"

// Just to be sure, also added this in Eigen.Build.cs
#ifndef EIGEN_MPL2_ONLY
#define EIGEN_MPL2_ONLY
#endif

#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(push)
#pragma warning(disable:6294) // Ill-defined for-loop:  initial condition does not satisfy test.  Loop body not executed.
#endif
PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
#include <Eigen/LU>
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END
#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(pop)
#endif

bool FRBFInterpolatorBase::SetUpperKernel(
	const TArrayView<float>& UpperKernel, 
	int32 Size
)
{
	using namespace Eigen;

	bool bSuccess = false;

	// The matrix logic below is self-contained and can run in the open without affecting
	// transactional correctness.
	// `Coeffs` is the only externally-visible alteration to program state.
	// Allocating and zeroing Coeffs before we enter the open ensures that it can be rolled
	// back safely if the transaction is aborted; we don't need a RecordOpenWrite below.
	Coeffs.Init(0.0f, Size * Size);

	UE_AUTORTFM_OPEN
	{
		MatrixXf FullKernel;

		// Construct the full kernel from the upper half calculated in the templated
		// portion.
		FullKernel = MatrixXf::Identity(Size, Size);

		for (int32 c = 0, i = 0; i < Size; i++)	
		{
			for (int32 j = i; j < Size; j++)
			{
				FullKernel(i, j) = FullKernel(j, i) = UpperKernel[c++];
			}
		}

		// Usually the RBF formulation is computed by solving for:
		// 
		//   A * w = T
		//
		// Where A is the target kernel is a symmetric matrix containing the distance 
		// between each node, w is the weights we want, and T is the target vector whose
		// values we want to interpolate.
		//
		// However, in our case, we consider the activation of each node's output value
		// to be a part of a N-dimensional vector, whose size is the same as the node count,
		// and therefore, collectively, same as the target kernel's dimensions. 
		// Each row is all zeros except for the activation value of 1.0 for a node at that
		// node's index, effectively forming an identity matrix.
		// 
		// This allows us to reformulate the problem as:
		//
		//   A * w = I
		//
		// Or in other words:  
		//
		//   w = A^-1
		//
		// Eigen will now happily pick LU factorization with partial pivoting for 
		// the inverse, which is blazingly fast.

		Map<MatrixXf> Result(Coeffs.GetData(), Size, Size);

		// If the matrix is non-invertible, return now and bail out.
		float Det = FullKernel.determinant();
		if (!FMath::IsNearlyZero(Det))
		{
			Result = FullKernel.inverse();
			bSuccess = true;
		}
	};

	return bSuccess;
}
