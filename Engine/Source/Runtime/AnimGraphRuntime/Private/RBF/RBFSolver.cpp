// Copyright Epic Games, Inc. All Rights Reserved.

#include "RBF/RBFSolver.h"

#include "RBF/RBFInterpolator.h"

#include "Containers/Set.h"
#include "Misc/MemStack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RBFSolver)


struct FRBFSolverData
{
	FRBFParams Params;
	TArray<FRBFEntry> EntryTargets;
	TRBFInterpolator<FRBFEntry> Rbf;
};


FRotator FRBFEntry::AsRotator(int32 Index) const
{
	FRotator Result = FRotator::ZeroRotator;

	const int32 BaseIndex = Index * 3;

	if (Values.Num() >= BaseIndex + 3)
	{
		Result.Roll = Values[BaseIndex + 0];
		Result.Pitch = Values[BaseIndex + 1];
		Result.Yaw = Values[BaseIndex + 2];
	}
	return Result;
}

FQuat FRBFEntry::AsQuat(int32 Index) const
{
	return AsRotator(Index).Quaternion();
}

FVector FRBFEntry::AsVector(int32 Index) const
{
	return AsRotator(Index).Vector();
}



void FRBFEntry::AddFromRotator(const FRotator& InRot)
{
	const int32 BaseIndex = Values.AddUninitialized(3);

	Values[BaseIndex + 0] = static_cast<float>(InRot.Roll);
	Values[BaseIndex + 1] = static_cast<float>(InRot.Pitch);
	Values[BaseIndex + 2] = static_cast<float>(InRot.Yaw);
}

void FRBFEntry::AddFromVector(const FVector& InVector)
{
	const int32 BaseIndex = Values.AddUninitialized(3);

	Values[BaseIndex + 0] = static_cast<float>(InVector.X);
	Values[BaseIndex + 1] = static_cast<float>(InVector.Y);
	Values[BaseIndex + 2] = static_cast<float>(InVector.Z);
}

//////////////////////////////////////////////////////////////////////////

FRBFParams::FRBFParams()
	: TargetDimensions(3)
	, SolverType(ERBFSolverType::Additive)
	, Radius(1.f)
	, bAutomaticRadius(false)
	, Function(ERBFFunctionType::Gaussian)
	, DistanceMethod(ERBFDistanceMethod::Euclidean)
	, TwistAxis(EBoneAxis::BA_X)
	, WeightThreshold(KINDA_SMALL_NUMBER)
	, NormalizeMethod(ERBFNormalizeMethod::OnlyNormalizeAboveOne)
	, MedianReference(FVector(0, 0, 0))
	, MedianMin(45.0f)
	, MedianMax(60.0f)
{

}

FVector FRBFParams::GetTwistAxisVector() const
{
	switch (TwistAxis)
	{
	case BA_X:
	default:
		return FVector(1.f, 0.f, 0.f);
	case BA_Y:
		return FVector(0.f, 1.f, 0.f);
	case BA_Z:
		return FVector(0.f, 0.f, 1.f);
	}
}


//////////////////////////////////////////////////////////////////////////

/* Returns the distance between entries, using different metrics, in radians. */
static float GetDistanceBetweenEntries(
	const FRBFEntry& A,
	const FRBFEntry& B,
	ERBFDistanceMethod DistanceMetric,
	const FVector& TwistAxis
)
{
	check(A.GetDimensions() == B.GetDimensions());

	const int32 NumRots = A.GetDimensions() / 3;
	float TotalDistance = 0.0f;

	for (int32 i = 0; i < NumRots; i++)
	{
		float Distance = 0.0f;
		switch (DistanceMetric)
		{
		case ERBFDistanceMethod::Euclidean:
			Distance = static_cast<float>(RBFDistanceMetric::Euclidean(A.AsRotator(i), B.AsRotator(i)));
			break;

		case ERBFDistanceMethod::Quaternion:
			Distance = static_cast<float>(RBFDistanceMetric::ArcLength(A.AsQuat(i), B.AsQuat(i)));
			break;

		case ERBFDistanceMethod::SwingAngle:
		case ERBFDistanceMethod::DefaultMethod:
			Distance += static_cast<float>(RBFDistanceMetric::SwingAngle(A.AsQuat(i), B.AsQuat(i), TwistAxis));
			break;

		case ERBFDistanceMethod::TwistAngle:
			Distance += static_cast<float>(RBFDistanceMetric::TwistAngle(A.AsQuat(i), B.AsQuat(i), TwistAxis));
			break;
		}

		TotalDistance += FMath::Square(Distance);
	}

	return FMath::Sqrt(TotalDistance);
}


float FRBFSolver::FindDistanceBetweenEntries(const FRBFEntry& A, const FRBFEntry& B, const FRBFParams& Params, ERBFDistanceMethod OverrideMethod)
{
	ERBFDistanceMethod DistanceMethod = OverrideMethod == ERBFDistanceMethod::DefaultMethod ? Params.DistanceMethod : OverrideMethod;

	float Distance = GetDistanceBetweenEntries(A, B, DistanceMethod, Params.GetTwistAxisVector());
	return FMath::RadiansToDegrees(Distance);
}


// Sigma controls the falloff width. The larger the value the narrower the falloff
static float GetWeightedValue(
	float Value, 
	float KernelWidth, 
	ERBFFunctionType FalloffFunctionType,
	bool bBackCompFix = false
)
{
	if (ensure(Value >= 0.0f))
	{
		switch (FalloffFunctionType)
		{
		case ERBFFunctionType::Linear:
		case ERBFFunctionType::DefaultFunction:
		default:
			if (bBackCompFix)
			{
				// This is how the old code formulated it. It has no control over the falloff
				// but ignores all values that have a distance greater than 1.0.
				return FMath::Max(1.0f - Value, 0.0f);
			}
			else
			{
				return RBFKernel::Linear(Value, KernelWidth);
			}

		case ERBFFunctionType::Gaussian:
			if (bBackCompFix)
			{
				// This is how the old code formulated it. It has a much wider falloff than the
				// one below it.
				return FMath::Exp(-Value * Value);
			}
			else
			{
				return RBFKernel::Gaussian(Value, KernelWidth);
			}

		case ERBFFunctionType::Exponential:
			if (bBackCompFix)
			{
				// This is how the old code formulated it. It has a much wider falloff than the
				// one below it.
				return 1.f / FMath::Exp(Value);
			}
			else
			{
				return RBFKernel::Exponential(Value, KernelWidth);
			}

		case ERBFFunctionType::Cubic:
			return RBFKernel::Cubic(Value, KernelWidth);

		case ERBFFunctionType::Quintic:
			return RBFKernel::Quintic(Value, KernelWidth);
		}
	}
	else
	{
		return 0.0f;
	}
}

static float GetOptimalKernelWidth(
	const FRBFParams& Params,
	const FVector &TwistAxis,
	const TArrayView<FRBFEntry>& Targets
	)
	{
		float Sum = 0.0f;
		int Count = 0;

		for (const FRBFEntry& A : Targets)
		{
			for (const FRBFEntry& B : Targets)
			{
				if (&A != &B)
				{
					Sum += GetDistanceBetweenEntries(A, B, Params.DistanceMethod, TwistAxis);
				Count++;
				}
			}
		}

	return Sum / float(Count);
}

static auto InterpolativeWeightFunction(
	const FRBFParams& Params,
	const TArrayView<FRBFEntry>& Targets
	)
{
	FVector TwistAxis = Params.GetTwistAxisVector();

	float KernelWidth;
	if (Params.bAutomaticRadius)
	{
		KernelWidth = GetOptimalKernelWidth(Params, TwistAxis, Targets);
	}
	else 
	{
		KernelWidth = FMath::DegreesToRadians(Params.Radius);
	}

	return [KernelWidth, TwistAxis, &Params](const FRBFEntry& A, const FRBFEntry& B) {
		float Distance = GetDistanceBetweenEntries(A, B, Params.DistanceMethod, TwistAxis);
		return GetWeightedValue(Distance, KernelWidth, Params.Function);
	};
}

static bool ValidateInterpolative(
	const FRBFParams& Params,
	const TArray<FRBFTarget>& Targets,
	TArray<int>& InvalidTargets
)
{
	TArray<FRBFEntry> EntryTargets;
	for (const auto& T : Targets)
		EntryTargets.Add(T);

	TArray<TTuple<int, int>> InvalidPairs;
	TSet<int> InvalidTargetSet;

	InvalidTargets.Empty();
	if (TRBFInterpolator<FRBFEntry>::GetIdenticalNodePairs(EntryTargets, InterpolativeWeightFunction(Params, EntryTargets), InvalidPairs))
	{
		// We mark the second of the pair to be invalid. Given how GetInvalidNodePairs iterates over all possible pairs,
		// this should guarantee to catch them all.
		for (const auto& IP : InvalidPairs)
			InvalidTargetSet.Add(IP.Get<1>());

		for (const auto& IT : InvalidTargetSet)
			InvalidTargets.Add(IT);

		// Return things in a nice sorted order, rather than TSet's hash order.
		InvalidTargets.Sort();
	}

	return InvalidTargets.Num() == 0;
}

bool FRBFSolver::ValidateTargets(
	const FRBFParams& Params,
	const TArray<FRBFTarget>& Targets,
	TArray<int>& InvalidTargets
)
{
	switch (Params.SolverType)
	{
	case ERBFSolverType::Additive:
	default:
		// The additive solver does not care
		return true;

	case ERBFSolverType::Interpolative:
		return ValidateInterpolative(Params, Targets, InvalidTargets);
	}
}


static void SolveAdditive(
	const FRBFParams& Params,
	const TArray<FRBFTarget>& Targets,
	const FRBFEntry& Input,
	float* AllWeights
	)
{
	// Iterate over each pose, adding its contribution
	for (int32 TargetIdx = 0; TargetIdx < Targets.Num(); TargetIdx++)
	{
		const FRBFTarget& Target = Targets[TargetIdx];
		ERBFFunctionType FunctionType = Target.FunctionType == ERBFFunctionType::DefaultFunction ? Params.Function : Target.FunctionType;

		// Find distance
		const float Distance = FRBFSolver::FindDistanceBetweenEntries(Target, Input, Params, Target.DistanceMethod);
		const float Scaling = FRBFSolver::GetRadiusForTarget(Target, Params);
		const float X = Distance / Scaling;

		// Evaluate radial basis function to find weight. We default to sigma = 1.0 and scale instead
		// using the radius value. We use the old formulation for Gauss + 
		float Weight = GetWeightedValue(X, 1.0f, FunctionType, /*BackCompFix=*/ true);

		// Apply custom curve if desired
		if (Target.bApplyCustomCurve)
		{
			Weight = Target.CustomCurve.Eval(Weight, Weight); // default is un-mapped Weight
		}

		// Add to array of all weights. Don't threshold yet, wait for normalization step.
		AllWeights[TargetIdx] = Weight;
	}
}

TSharedPtr<const FRBFSolverData> FRBFSolver::InitSolver(const FRBFParams& Params, const TArray<FRBFTarget>& Targets)
{
	FRBFSolverData* InitData = new FRBFSolverData;

	InitData->Params = Params;

	if (Params.SolverType == ERBFSolverType::Interpolative)
	{
		for (const FRBFTarget& T : Targets)
			InitData->EntryTargets.Add(T);

		InitData->Rbf = TRBFInterpolator<FRBFEntry>(InitData->EntryTargets, InterpolativeWeightFunction(Params, InitData->EntryTargets));
	}

	return TSharedPtr<const FRBFSolverData>(InitData);
}


bool FRBFSolver::IsSolverDataValid(const FRBFSolverData& SolverData, const FRBFParams& Params, const TArray<FRBFTarget>& Targets)
{
	if (SolverData.EntryTargets.Num() != Targets.Num())
		return false;

	for (int32 i = 0; i < Targets.Num(); i++)
	{
		const TArray<float>& ValuesA = Targets[i].Values;
		const TArray<float>& ValuesB = SolverData.EntryTargets[i].Values;

		if (ValuesA.Num() != ValuesB.Num())
		{
			return false;
		}

		for (int j = 0; j < ValuesA.Num(); j++)
		{
			if (!FMath::IsNearlyEqualByULP(ValuesA[j], ValuesB[j]))
			{
				return false;
			}
		}
	}

	return SolverData.Params.SolverType == Params.SolverType &&
		FMath::IsNearlyEqualByULP(SolverData.Params.Radius, Params.Radius) &&
		SolverData.Params.Function == Params.Function &&
		SolverData.Params.DistanceMethod == Params.DistanceMethod &&
		SolverData.Params.TwistAxis == Params.TwistAxis;
}


void FRBFSolver::Solve(
	const FRBFSolverData& SolverData,
	const FRBFParams& Params,
	const TArray<FRBFTarget>& Targets,
	const FRBFEntry& Input,
	TArray<FRBFOutputWeight>& OutputWeights
)
{
	if (!ensure(Params.TargetDimensions == Input.GetDimensions()))
	{
		return;
	}

	TArray<float, TMemStackAllocator<>> AllWeights;
	AllWeights.AddZeroed(Targets.Num());

	switch (SolverData.Params.SolverType)
	{
	case ERBFSolverType::Additive:
	default:
		SolveAdditive(Params, Targets, Input, AllWeights.GetData());
		break;

	case ERBFSolverType::Interpolative:
	{
		SolverData.Rbf.Interpolate(AllWeights, Input);

		// Scale the weight by the scale factor on the target.
		for (int32 i = 0; i < Targets.Num(); i++)
		{
			AllWeights[i] *= Targets[i].ScaleFactor;
		}
		break;
	}
	}

	float TotalWeight = 0.f; // Keep track of total weight generated, to normalize at the end
	for (float Weight : AllWeights)
	{
		TotalWeight += Weight;
	}

	// Only normalize and apply if we got some kind of weight
	if (TotalWeight > KINDA_SMALL_NUMBER)
	{
		float WeightScale = 1.f;
		if (TotalWeight > 1.f)
		{
			WeightScale = 1.f / TotalWeight;
		}
		else
		{
			switch (Params.NormalizeMethod)
			{
				case ERBFNormalizeMethod::OnlyNormalizeAboveOne:
				{
					break;
				}
				case ERBFNormalizeMethod::AlwaysNormalize:
				{
					WeightScale = 1.f / TotalWeight;
					break;
				}
				case ERBFNormalizeMethod::NormalizeWithinMedian:
				{
					if (Params.MedianMax < Params.MedianMin)
					{
						break;
					}

					FRBFEntry MedianEntry;
					while (Input.GetDimensions() > MedianEntry.GetDimensions())
					{
						MedianEntry.AddFromVector(Params.MedianReference);
					}
					
					float MedianDistance = FindDistanceBetweenEntries(Input, MedianEntry, Params);
					if (MedianDistance > Params.MedianMax)
					{
						break;
					}
					if (MedianDistance <= Params.MedianMin)
					{
						WeightScale = 1.f / TotalWeight;
						break;
					}

					float Bias = FMath::Clamp<float>((MedianDistance - Params.MedianMin) / (Params.MedianMax - Params.MedianMin), 0.f, 1.f);
					WeightScale = FMath::Lerp<float>(1.f / TotalWeight, 1.f, Bias);
					break;
				}
			}
		}
		
		/// TotalWeight : (Params.bNormalizeWeightsBelowSumOfOne ? 1.f / TotalWeight : 1.f);
		for (int32 TargetIdx = 0; TargetIdx < Targets.Num(); TargetIdx++)
		{
			float NormalizedWeight = AllWeights[TargetIdx] * WeightScale;

			// If big enough, add to output list
			if (NormalizedWeight > Params.WeightThreshold)
			{
				OutputWeights.Add(FRBFOutputWeight(TargetIdx, NormalizedWeight));
			}
		}
	}
}


bool FRBFSolver::FindTargetNeighbourDistances(const FRBFParams& Params, const TArray<FRBFTarget>& Targets, TArray<float>& NeighbourDists)
{
	const int32 NumTargets = Targets.Num();

	NeighbourDists.Empty();
	NeighbourDists.AddZeroed(NumTargets);

	if (NumTargets > 1)
	{
		// Iterate over targets
		for (int32 TargetIdx = 0; TargetIdx < NumTargets; TargetIdx++)
		{
			float& NearestDist = NeighbourDists[TargetIdx];
			NearestDist = BIG_NUMBER; // init to large value

			for (int32 OtherTargetIdx = 0; OtherTargetIdx < NumTargets; OtherTargetIdx++)
			{
				if (OtherTargetIdx != TargetIdx) // If not ourself..
				{
					// Get distance between poses
					float Dist = FindDistanceBetweenEntries(Targets[TargetIdx], Targets[OtherTargetIdx], Params, Targets[TargetIdx].DistanceMethod);
					NearestDist = FMath::Min(Dist, NearestDist);
				}
			}

			// Avoid zero dist if poses are all on top of each other
			NearestDist = FMath::Max(NearestDist, KINDA_SMALL_NUMBER);
		}

		return true;
	}
	else
	{
		return false;
	}
}

float FRBFSolver::GetRadiusForTarget(const FRBFTarget& Target, const FRBFParams& Params)
{
	float Radius = Params.Radius;
	if (Params.SolverType == ERBFSolverType::Additive)
	{
		Radius *= Target.ScaleFactor;
	}

	return FMath::Max(Radius, KINDA_SMALL_NUMBER);
}


float FRBFSolver::GetOptimalRadiusForTargets(const FRBFParams& Params, const TArray<FRBFTarget>& Targets)
{
	TArray<FRBFEntry, TMemStackAllocator<>> EntryTargets;
	EntryTargets.Reset(Targets.Num());
	for (const auto& T : Targets)
		EntryTargets.Add(T);

	FVector TwistAxis = Params.GetTwistAxisVector();

	float KernelWidth = GetOptimalKernelWidth(Params, TwistAxis, EntryTargets);

	return FMath::RadiansToDegrees(KernelWidth);
}

