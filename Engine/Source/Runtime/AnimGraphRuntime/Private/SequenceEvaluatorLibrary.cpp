// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequenceEvaluatorLibrary.h"

#include "Animation/AnimNode_Inertialization.h"
#include "AnimNodes/AnimNode_SequenceEvaluator.h"
#include "AnimationRuntime.h"
#include "Animation/AnimTrace.h"
#include "Animation/AnimInstanceProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequenceEvaluatorLibrary)

DEFINE_LOG_CATEGORY_STATIC(LogSequenceEvaluatorLibrary, Verbose, All);

FSequenceEvaluatorReference USequenceEvaluatorLibrary::ConvertToSequenceEvaluator(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result)
{
	return FAnimNodeReference::ConvertToType<FSequenceEvaluatorReference>(Node, Result);
}

FSequenceEvaluatorReference USequenceEvaluatorLibrary::SetExplicitTime(const FSequenceEvaluatorReference& SequenceEvaluator, float Time)
{
	SequenceEvaluator.CallAnimNodeFunction<FAnimNode_SequenceEvaluator>(
		TEXT("SetExplicitTime"),
		[Time](FAnimNode_SequenceEvaluator& InSequenceEvaluator)
		{
			if(!InSequenceEvaluator.SetExplicitTime(Time))
			{
				UE_LOGF(LogSequenceEvaluatorLibrary, Warning, "Could not set explicit time on sequence evaluator, value is not dynamic. Set it as Always Dynamic.");
			}
		});

	return SequenceEvaluator;
}

FSequenceEvaluatorReference USequenceEvaluatorLibrary::SetExplicitFrame(const FSequenceEvaluatorReference& SequenceEvaluator, int32 Frame)
{
	SequenceEvaluator.CallAnimNodeFunction<FAnimNode_SequenceEvaluator>(
		TEXT("SetExplicitTime"),
		[Frame](FAnimNode_SequenceEvaluator& InSequenceEvaluator)
		{
			if(!InSequenceEvaluator.SetExplicitFrame(Frame))
			{
				UE_LOGF(LogSequenceEvaluatorLibrary, Warning, "Could not set explicit frame on sequence evaluator, value is not dynamic. Set it as Always Dynamic.");
			}
		});

	return SequenceEvaluator;
}

FSequenceEvaluatorReference USequenceEvaluatorLibrary::AdvanceTime(const FAnimUpdateContext& UpdateContext, const FSequenceEvaluatorReference& SequenceEvaluator, float PlayRate)
{
	SequenceEvaluator.CallAnimNodeFunction<FAnimNode_SequenceEvaluator>(
		TEXT("AdvanceTime"),
		[&UpdateContext, PlayRate](FAnimNode_SequenceEvaluator& InSequenceEvaluator)
		{
			if (const FAnimationUpdateContext* AnimationUpdateContext = UpdateContext.GetContext())
			{
				float ExplicitTime = InSequenceEvaluator.GetExplicitTime();
				FAnimationRuntime::AdvanceTime(InSequenceEvaluator.IsLooping(), AnimationUpdateContext->GetDeltaTime() * PlayRate, ExplicitTime, InSequenceEvaluator.GetCurrentAssetLength());

				if (!InSequenceEvaluator.SetExplicitTime(ExplicitTime))
				{
					UE_LOGF(LogSequenceEvaluatorLibrary, Warning, "Could not advance time on sequence evaluator, ExplicitTime is not dynamic. Set it as Always Dynamic.");
				}
			}
			else
			{
				UE_LOGF(LogSequenceEvaluatorLibrary, Warning, "AdvanceTime called with invalid context");
			}
		});

	return SequenceEvaluator;
}

FSequenceEvaluatorReference USequenceEvaluatorLibrary::SetSequence(const FSequenceEvaluatorReference& SequenceEvaluator, UAnimSequenceBase* Sequence)
{
	SequenceEvaluator.CallAnimNodeFunction<FAnimNode_SequenceEvaluator>(
		TEXT("SetSequence"),
		[Sequence](FAnimNode_SequenceEvaluator& InSequenceEvaluator)
		{
			if(!InSequenceEvaluator.SetSequence(Sequence))
			{
				UE_LOGF(LogSequenceEvaluatorLibrary, Warning, "Could not set sequence on sequence evaluator, value is not dynamic. Set it as Always Dynamic.");
			}
		});

	return SequenceEvaluator;
}

FSequenceEvaluatorReference USequenceEvaluatorLibrary::SetSequenceWithInertialBlending(const FAnimUpdateContext& UpdateContext, const FSequenceEvaluatorReference& SequenceEvaluator, UAnimSequenceBase* Sequence, float BlendTime)
{
	SequenceEvaluator.CallAnimNodeFunction<FAnimNode_SequenceEvaluator>(
		TEXT("SetSequenceWithInterialBlending"),
		[Sequence, &UpdateContext, BlendTime](FAnimNode_SequenceEvaluator& InSequenceEvaluator)
		{
			const UAnimSequenceBase* CurrentSequence = InSequenceEvaluator.GetSequence();
			const bool bAnimSequenceChanged = (CurrentSequence != Sequence);
			
			if(!InSequenceEvaluator.SetSequence(Sequence))
			{
				UE_LOGF(LogSequenceEvaluatorLibrary, Warning, "Could not set sequence on sequence evaluator, value is not dynamic. Set it as Always Dynamic.");
			}

			if(bAnimSequenceChanged && BlendTime > 0.0f)
			{
				if (const FAnimationUpdateContext* AnimationUpdateContext = UpdateContext.GetContext())
				{
					if (UE::Anim::IInertializationRequester* InertializationRequester = AnimationUpdateContext->GetMessage<UE::Anim::IInertializationRequester>())
					{
						FInertializationRequest Request;
						Request.Duration = BlendTime;
#if ANIM_TRACE_ENABLED
						Request.NodeId = AnimationUpdateContext->GetCurrentNodeId();
						Request.AnimInstance = AnimationUpdateContext->AnimInstanceProxy->GetAnimInstanceObject();
#endif

						InertializationRequester->RequestInertialization(Request);
					}
				}
				else
				{
					UE_LOGF(LogSequenceEvaluatorLibrary, Warning, "SetSequenceWithInterialBlending called with invalid context");
				}
			}
		});

	return SequenceEvaluator;
}

float USequenceEvaluatorLibrary::GetAccumulatedTime(const FSequenceEvaluatorReference& SequenceEvaluator)
{
	float OutAccumulatedTime = 0.0f;
	SequenceEvaluator.CallAnimNodeFunction<FAnimNode_SequenceEvaluator>(
		TEXT("GetAccumulatedTime"),
		[&OutAccumulatedTime](FAnimNode_SequenceEvaluator& InSequenceEvaluator)
		{
			OutAccumulatedTime = InSequenceEvaluator.GetAccumulatedTime();
		});

	return OutAccumulatedTime;
}

UAnimSequenceBase* USequenceEvaluatorLibrary::GetSequence(const FSequenceEvaluatorReference& SequenceEvaluator)
{
	UAnimSequenceBase* OutSequence = nullptr;
	SequenceEvaluator.CallAnimNodeFunction<FAnimNode_SequenceEvaluator>(
		TEXT("GetAccumulatedTime"),
		[&OutSequence](FAnimNode_SequenceEvaluator& InSequenceEvaluator)
		{
			OutSequence = InSequenceEvaluator.GetSequence();
		});

	return OutSequence;
}
