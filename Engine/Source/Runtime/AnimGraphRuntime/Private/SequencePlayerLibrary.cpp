// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencePlayerLibrary.h"

#include "Animation/AnimNode_Inertialization.h"
#include "Animation/AnimNode_SequencePlayer.h"
#include "Animation/AnimTrace.h"
#include "Animation/AnimInstanceProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequencePlayerLibrary)

DEFINE_LOG_CATEGORY_STATIC(LogSequencePlayerLibrary, Verbose, All);

FSequencePlayerReference USequencePlayerLibrary::ConvertToSequencePlayer(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result)
{
	return FAnimNodeReference::ConvertToType<FSequencePlayerReference>(Node, Result);
}

FSequencePlayerReference USequencePlayerLibrary::SetAccumulatedTime(const FSequencePlayerReference& SequencePlayer, float Time)
{
	SequencePlayer.CallAnimNodeFunction<FAnimNode_SequencePlayer>(
		TEXT("SetAccumulatedTime"),
		[Time](FAnimNode_SequencePlayer& InSequencePlayer)
		{
			InSequencePlayer.SetAccumulatedTime(Time);
		});

	return SequencePlayer;
}

FSequencePlayerReference USequencePlayerLibrary::SetStartPosition(const FSequencePlayerReference& SequencePlayer, float StartPosition)
{
	SequencePlayer.CallAnimNodeFunction<FAnimNode_SequencePlayer>(
		TEXT("SetStartPosition"),
		[StartPosition](FAnimNode_SequencePlayer& InSequencePlayer)
		{
			if(!InSequencePlayer.SetStartPosition(StartPosition))
			{
				UE_LOGF(LogSequencePlayerLibrary, Warning, "Could not set start position on sequence player, value is not dynamic. Set it as Always Dynamic.");
			}
		});

	return SequencePlayer;
}

FSequencePlayerReference USequencePlayerLibrary::SetPlayRate(const FSequencePlayerReference& SequencePlayer, float PlayRate)
{
	SequencePlayer.CallAnimNodeFunction<FAnimNode_SequencePlayer>(
		TEXT("SetPlayRate"),
		[PlayRate](FAnimNode_SequencePlayer& InSequencePlayer)
		{
			if(!InSequencePlayer.SetPlayRate(PlayRate))
			{
				UE_LOGF(LogSequencePlayerLibrary, Warning, "Could not set play rate on sequence player, value is not dynamic. Set it as Always Dynamic.");
			}
		});

	return SequencePlayer;
}

FSequencePlayerReference USequencePlayerLibrary::SetSequence(const FSequencePlayerReference& SequencePlayer, UAnimSequenceBase* Sequence)
{
	SequencePlayer.CallAnimNodeFunction<FAnimNode_SequencePlayer>(
		TEXT("SetSequence"),
		[Sequence](FAnimNode_SequencePlayer& InSequencePlayer)
		{
			if(!InSequencePlayer.SetSequence(Sequence))
			{
				UE_LOGF(LogSequencePlayerLibrary, Warning, "Could not set sequence on sequence player, value is not dynamic. Set it as Always Dynamic.");
			}
		});

	return SequencePlayer;
}

FSequencePlayerReference USequencePlayerLibrary::SetSequenceWithInertialBlending(const FAnimUpdateContext& UpdateContext, const FSequencePlayerReference& SequencePlayer, UAnimSequenceBase* Sequence, float BlendTime)
{
	SequencePlayer.CallAnimNodeFunction<FAnimNode_SequencePlayer>(
		TEXT("SetSequenceWithInterialBlending"),
		[Sequence, &UpdateContext, BlendTime](FAnimNode_SequencePlayer& InSequencePlayer)
		{
			const UAnimSequenceBase* CurrentSequence = InSequencePlayer.GetSequence();
			const bool bAnimSequenceChanged = (CurrentSequence != Sequence);
			
			if(!InSequencePlayer.SetSequence(Sequence))
			{
				UE_LOGF(LogSequencePlayerLibrary, Warning, "Could not set sequence on sequence player, value is not dynamic. Set it as Always Dynamic.");
			}

			if(bAnimSequenceChanged && BlendTime > 0.0f)
			{
				if (const FAnimationUpdateContext* AnimationUpdateContext = UpdateContext.GetContext())
				{
					if (UE::Anim::IInertializationRequester* InertializationRequester = AnimationUpdateContext->GetMessage<UE::Anim::IInertializationRequester>())
					{
						FInertializationRequest Request;
						Request.Duration = BlendTime;
#if ANIM_TRACE_ENABLED
						Request.NodeId = AnimationUpdateContext->GetCurrentNodeId();
						Request.AnimInstance = AnimationUpdateContext->AnimInstanceProxy->GetAnimInstanceObject();
#endif

						InertializationRequester->RequestInertialization(Request);
					}
				}
				else
				{
					UE_LOGF(LogSequencePlayerLibrary, Warning, "SetSequenceWithInertialBlending called with invalid context");
				}
			}
		});

	return SequencePlayer;
}

FSequencePlayerReference USequencePlayerLibrary::GetSequence(const FSequencePlayerReference& SequencePlayer, UAnimSequenceBase*& SequenceBase)
{
	SequencePlayer.CallAnimNodeFunction<FAnimNode_SequencePlayer>(
		TEXT("GetSequence"),
		[&SequenceBase](FAnimNode_SequencePlayer& InSequencePlayer)
		{
			SequenceBase = InSequencePlayer.GetSequence();
		});

	return SequencePlayer;
}

UAnimSequenceBase* USequencePlayerLibrary::GetSequencePure(const FSequencePlayerReference& SequencePlayer)
{
	UAnimSequenceBase* SequenceBase = nullptr;
	
	SequencePlayer.CallAnimNodeFunction<FAnimNode_SequencePlayer>(
		TEXT("GetSequence"),
		[&SequenceBase](FAnimNode_SequencePlayer& InSequencePlayer)
		{
			SequenceBase = InSequencePlayer.GetSequence();
		});

	return SequenceBase;
}

float USequencePlayerLibrary::GetAccumulatedTime(const FSequencePlayerReference& SequencePlayer)
{
	float AccumulatedTime = 0.f;
	SequencePlayer.CallAnimNodeFunction<FAnimNode_SequencePlayer>(
		TEXT("GetAccumulatedTime"),
		[&AccumulatedTime](FAnimNode_SequencePlayer& InSequencePlayer)
		{
			AccumulatedTime = InSequencePlayer.GetAccumulatedTime();
		});

	return AccumulatedTime;
}

float USequencePlayerLibrary::GetStartPosition(const FSequencePlayerReference& SequencePlayer)
{
	float StartPosition = 0.f;
	SequencePlayer.CallAnimNodeFunction<FAnimNode_SequencePlayer>(
		TEXT("GetStartPosition"),
		[&StartPosition](FAnimNode_SequencePlayer& InSequencePlayer)
		{
			StartPosition = InSequencePlayer.GetStartPosition();
		});

	return StartPosition;
}

float USequencePlayerLibrary::GetPlayRate(const FSequencePlayerReference& SequencePlayer)
{
	float PlayRate = 1.f;
	SequencePlayer.CallAnimNodeFunction<FAnimNode_SequencePlayer>(
		TEXT("GetPlayRate"),
		[&PlayRate](FAnimNode_SequencePlayer& InSequencePlayer)
		{
			PlayRate = InSequencePlayer.GetPlayRate();
		});

	return PlayRate;
}

bool USequencePlayerLibrary::GetLoopAnimation(const FSequencePlayerReference& SequencePlayer)
{
	bool bLoopAnimation = false;
	SequencePlayer.CallAnimNodeFunction<FAnimNode_SequencePlayer>(
		TEXT("GetLoopAnimation"),
		[&bLoopAnimation](FAnimNode_SequencePlayer& InSequencePlayer)
		{
			bLoopAnimation = InSequencePlayer.IsLooping();
		});

	return bLoopAnimation;
}

float USequencePlayerLibrary::ComputePlayRateFromDuration(const FSequencePlayerReference& SequencePlayer, float Duration /* = 1.0f */)
{
	float PlayRate = 1.f;
	if (Duration > 0.f)
	{
	SequencePlayer.CallAnimNodeFunction<FAnimNode_SequencePlayer>(
		TEXT("GetPlayRate"),
		[&PlayRate, Duration](const FAnimNode_SequencePlayer& InSequencePlayer)
		{
			if (const UAnimSequenceBase* Sequence = InSequencePlayer.GetSequence())
			{
				PlayRate = Sequence->GetPlayLength() / Duration;
			}
		});
	}
	return PlayRate;
}
