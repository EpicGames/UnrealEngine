// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalControlLibrary.h"

#include "BoneControllers/AnimNode_SkeletalControlBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalControlLibrary)

DEFINE_LOG_CATEGORY_STATIC(LogSkeletalControlLibrary, Verbose, All);

FSkeletalControlReference USkeletalControlLibrary::ConvertToSkeletalControl(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result)
{
	return FAnimNodeReference::ConvertToType<FSkeletalControlReference>(Node, Result);
}

FSkeletalControlReference USkeletalControlLibrary::SetAlpha(const FSkeletalControlReference& SkeletalControl, float Alpha)
{
	SkeletalControl.CallAnimNodeFunction<FAnimNode_SkeletalControlBase>(
		TEXT("SetAlpha"),
		[Alpha](FAnimNode_SkeletalControlBase& InSkeletalControl)
		{
			InSkeletalControl.SetAlpha(Alpha);
		});

	return SkeletalControl;
}

float USkeletalControlLibrary::GetAlpha(const FSkeletalControlReference& SkeletalControl)
{
	float Alpha = 0.0f;
	
	SkeletalControl.CallAnimNodeFunction<FAnimNode_SkeletalControlBase>(
	TEXT("GetAlpha"),
	[&Alpha](FAnimNode_SkeletalControlBase& InSkeletalControl)
	{
		Alpha = InSkeletalControl.GetAlpha();
	});

	return Alpha;
}
