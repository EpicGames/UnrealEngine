// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	This is a binding/unbinding functions for custom instances, specially for Sequencer tracks
	
	This is complicated because Sequencer Animation Track and ControlRig Tracks could be supported through this interface
	and it encapsulates lots of complications inside. 
	
	You can use one Animation Track - it's because this track doesn't take input from other pose, so this is always source
	You can use multiple ControlRig Tracks - this is because ControlRig could take inputs from other sources
	
	However this is not end of it. The way sequencer works is to allow you to add/remove anytime or any place. 
		
	So this behaves binding/unbinding depending on if you're source (Animation Track) or not (ControlRig). 
	
	If you want to be used by Animation Track, you should derive from ISequencerAnimationSupport and implement proper interfaces. 
	Now, you want to support layering, you'll have to support DoesSupportDifferentSourceAnimInstance to be true, and allow it to be used as source input. 
	
	1. ControlRigLayerInstance : this does support different source anim instance, and use it as a source of animation
	2. AnimSequencerInstance: this does not support different source anim instance, this acts as one.
	
	The code is to support, depending what role you have, you can be bound differently, so that you don't disturb what's currently available

=============================================================================*/ 


#pragma once
#include "Animation/AnimInstance.h"
#include "AnimSequencerInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/World.h"
#include "SequencerAnimationSupport.h"



class FAnimCustomInstanceHelper
{
public:
	/** 
	 * Called to bind a typed UAnimCustomInstance to an existing skeletal mesh component 
	 * @return the current (or newly created) UAnimCustomInstance
	 */
	template<typename InstanceClassType>
	static InstanceClassType* BindToSkeletalMeshComponent(USkeletalMeshComponent* InSkeletalMeshComponent, bool& bOutWasCreated)
	{
		bOutWasCreated = false;
		// make sure to tick and refresh all the time when ticks
		// @TODO: this needs restoring post-binding
		InSkeletalMeshComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
#if WITH_EDITOR
		InSkeletalMeshComponent->SetUpdateAnimationInEditor(true);
		InSkeletalMeshComponent->SetUpdateClothInEditor(true);
#endif
		TArray<USceneComponent*> ChildComponents;
		InSkeletalMeshComponent->GetChildrenComponents(true, ChildComponents);
		for (USceneComponent* ChildComponent : ChildComponents)
		{
			USkeletalMeshComponent* ChildSkelMeshComp = Cast<USkeletalMeshComponent>(ChildComponent);
			if (ChildSkelMeshComp)
			{
				ChildSkelMeshComp->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
#if WITH_EDITOR
				ChildSkelMeshComp->SetUpdateAnimationInEditor(true);
				ChildSkelMeshComp->SetUpdateClothInEditor(true);
#endif
			}
		}
		// we use sequence instance if it's using anim blueprint that matches. Otherwise, we create sequence player
		// this might need more check - i.e. making sure if it's same skeleton and so on, 
		// Ideally we could just call NeedToSpawnAnimScriptInstance call, which is protected now
		const bool bShouldCreateCustomInstance = ShouldCreateCustomInstancePlayer(InSkeletalMeshComponent);
		UAnimInstance* CurrentAnimInstance = InSkeletalMeshComponent->AnimScriptInstance;
		// See if we have SequencerInterface from current instance
		ISequencerAnimationSupport* CurrentSequencerInterface = Cast<ISequencerAnimationSupport>(CurrentAnimInstance);
		const bool bCurrentlySequencerInterface = CurrentSequencerInterface != nullptr;
		const bool bCreateSequencerInterface = InstanceClassType::StaticClass()->ImplementsInterface(USequencerAnimationSupport::StaticClass());
		bool bSupportDifferentSourceAnimInstance = false;

		if (bCreateSequencerInterface)
		{
			InstanceClassType* DefaultObject = InstanceClassType::StaticClass()->template GetDefaultObject<InstanceClassType>();
			ISequencerAnimationSupport* SequencerSupporter = Cast<ISequencerAnimationSupport>(DefaultObject);
			bSupportDifferentSourceAnimInstance = SequencerSupporter && SequencerSupporter->DoesSupportDifferentSourceAnimInstance();
		}

		// if it should use sequence instance and current one doesn't support Sequencer Interface, we fall back to old behavior
		if (bShouldCreateCustomInstance && !bCurrentlySequencerInterface)
		{
			// this has to wrap around with this because we don't want to reinitialize everytime they come here
			// SetAnimationMode will reinitiaize it even if it's same, so make sure we just call SetAnimationMode if not AnimationCustomMode
			if (InSkeletalMeshComponent->GetAnimationMode() != EAnimationMode::AnimationCustomMode)
			{
				InSkeletalMeshComponent->SetAnimationMode(EAnimationMode::AnimationCustomMode);
			}

			if (Cast<InstanceClassType>(InSkeletalMeshComponent->AnimScriptInstance) == nullptr || !InSkeletalMeshComponent->AnimScriptInstance->GetClass()->IsChildOf(InstanceClassType::StaticClass()))

			{
				InstanceClassType* SequencerInstance = NewObject<InstanceClassType>(InSkeletalMeshComponent, InstanceClassType::StaticClass());
				InSkeletalMeshComponent->AnimScriptInstance = SequencerInstance;
				InSkeletalMeshComponent->AnimScriptInstance->InitializeAnimation();
				bOutWasCreated = true;
				return SequencerInstance;
			}
			// if it's the same type it's expecting, returns the one
			else if (InSkeletalMeshComponent->AnimScriptInstance->GetClass()->IsChildOf(InstanceClassType::StaticClass()))
			{
				return Cast<InstanceClassType>(InSkeletalMeshComponent->AnimScriptInstance);
			}
		}
		else 
		{
			// if it's the same type it's expecting, returns the one
			if (CurrentAnimInstance->GetClass()->IsChildOf(InstanceClassType::StaticClass()))
			{
				return Cast<InstanceClassType>(InSkeletalMeshComponent->AnimScriptInstance);
			}
			// if currently it is sequencer interface, check to see if
			else if (bCurrentlySequencerInterface)
			{
				// this is a case where SequencerInstance is created later, currently it has SequencerInteface
				UAnimInstance* CurSourceInstance = CurrentSequencerInterface->GetSourceAnimInstance();
				// if no source, create new one, and assign the new instance if current sequencer interface supports
				if (CurSourceInstance == nullptr)
				{
					// if current doesn't have source instance and if it does support different source animation
					if (CurrentSequencerInterface->DoesSupportDifferentSourceAnimInstance())
					{
						// create new one requested, and set to new source
						InstanceClassType* NewSequencerInstance = NewObject<InstanceClassType>(InSkeletalMeshComponent, InstanceClassType::StaticClass());
						//if it's sequencer inteface, create one, and assign
						NewSequencerInstance->InitializeAnimation();
						CurrentSequencerInterface->SetSourceAnimInstance(NewSequencerInstance);
						bOutWasCreated = true;
						return NewSequencerInstance;
					}
					else
					{
						UE_LOGF(LogAnimation, Warning, "Currently Sequencer doesn't support Source Instance. They're not compatible to work together.");
					}
				}
				// if source is same as what's requested, just return this.
				else if (CurSourceInstance->GetClass()->IsChildOf(InstanceClassType::StaticClass()))
				{
					// nothing to do? 
					// it has already same type
					return Cast<InstanceClassType>(CurSourceInstance);
				}
				// if this doesn't support different source anim instances, but the new class does
				// see if we could switch it up
				else if (bCreateSequencerInterface && bSupportDifferentSourceAnimInstance && !CurrentSequencerInterface->DoesSupportDifferentSourceAnimInstance())
				{
					InstanceClassType* NewInstance = NewObject<InstanceClassType>(InSkeletalMeshComponent, InstanceClassType::StaticClass());
					ISequencerAnimationSupport* NewSequencerInterface = Cast<ISequencerAnimationSupport>(NewInstance);
					check(NewSequencerInterface);
					if (NewSequencerInterface->DoesSupportDifferentSourceAnimInstance())
					{
						InSkeletalMeshComponent->AnimScriptInstance = NewInstance;
						ensureAlways(NewInstance != CurSourceInstance);
						NewSequencerInterface->SetSourceAnimInstance(CurSourceInstance);

						InSkeletalMeshComponent->AnimScriptInstance->InitializeAnimation();
						bOutWasCreated = true;
						return Cast<InstanceClassType>(InSkeletalMeshComponent->AnimScriptInstance);
					}
				}
			}
			// if requested one is support sequencer animation?
			else if (bCreateSequencerInterface && bSupportDifferentSourceAnimInstance)
			{
				InstanceClassType* NewInstance = NewObject<InstanceClassType>(InSkeletalMeshComponent, InstanceClassType::StaticClass());
				ISequencerAnimationSupport* NewSequencerInterface = Cast<ISequencerAnimationSupport>(NewInstance);
				check(NewSequencerInterface);
				if (NewSequencerInterface->DoesSupportDifferentSourceAnimInstance())
				{
					InSkeletalMeshComponent->AnimScriptInstance = NewInstance;
					NewSequencerInterface->SetSourceAnimInstance(CurrentAnimInstance);

					InSkeletalMeshComponent->AnimScriptInstance->InitializeAnimation();
					bOutWasCreated = true;
					return Cast<InstanceClassType>(InSkeletalMeshComponent->AnimScriptInstance);
				}
			}
		}

		return nullptr;
	}

	/** Called to unbind a UAnimCustomInstance to an existing skeletal mesh component */
	template<typename InstanceClassType>
	static void UnbindFromSkeletalMeshComponent(USkeletalMeshComponent* InSkeletalMeshComponent)
	{
#if WITH_EDITOR
		InSkeletalMeshComponent->SetUpdateAnimationInEditor(false);
		InSkeletalMeshComponent->SetUpdateClothInEditor(false);
#endif

		if (InSkeletalMeshComponent->GetAnimationMode() == EAnimationMode::Type::AnimationCustomMode)
		{
			UAnimInstance* AnimInstance = InSkeletalMeshComponent->GetAnimInstance();
			// if same type, we're fine
			InstanceClassType* SequencerInstance = Cast<InstanceClassType>(AnimInstance);
			if (SequencerInstance)
			{
				bool bClearAnimScriptInstance = true;

				if(ISequencerAnimationSupport* SequencerInterface = Cast<ISequencerAnimationSupport>(AnimInstance))
				{
					if (SequencerInterface->DoesSupportDifferentSourceAnimInstance())
					{
						UAnimInstance* SourceAnimInstance = SequencerInterface->GetSourceAnimInstance();
						// if we have source, replace with it
						if (SourceAnimInstance)
						{
							// clear before you remove
							SequencerInterface->SetSourceAnimInstance(nullptr);
							InSkeletalMeshComponent->AnimScriptInstance = SourceAnimInstance;
							bClearAnimScriptInstance = false;
						}
					}
				}
				
				if(bClearAnimScriptInstance)
				{
					InSkeletalMeshComponent->ClearAnimScriptInstance();
				}
			}
			else // if not, we'd like to see if SequencerSupport
			{
				ISequencerAnimationSupport* SequencerInterface = Cast<ISequencerAnimationSupport>(AnimInstance);
				if (SequencerInterface && SequencerInterface->DoesSupportDifferentSourceAnimInstance())
				{
					UAnimInstance* SourceInstance = SequencerInterface->GetSourceAnimInstance();
					SequencerInstance = Cast<InstanceClassType>(SourceInstance);
					
					if (SequencerInstance)
					{
						SequencerInterface->SetSourceAnimInstance(nullptr);
					}
					// this can be animBP, we want to return that
					else if (SourceInstance)
					{
						SequencerInterface->SetSourceAnimInstance(nullptr);
						InSkeletalMeshComponent->AnimScriptInstance = SourceInstance;
					}
				}
			}
		}
		else if (InSkeletalMeshComponent->GetAnimationMode() == EAnimationMode::Type::AnimationBlueprint)
		{
			UAnimInstance* AnimInstance = InSkeletalMeshComponent->GetAnimInstance();
			ISequencerAnimationSupport* SequencerInterface = Cast<ISequencerAnimationSupport>(AnimInstance);
			if (SequencerInterface && SequencerInterface->DoesSupportDifferentSourceAnimInstance())
			{
				UAnimInstance* SourceInstance = SequencerInterface->GetSourceAnimInstance();
				if (SourceInstance)
				{
					SequencerInterface->SetSourceAnimInstance(nullptr);
					InSkeletalMeshComponent->AnimScriptInstance = SourceInstance;
					AnimInstance = SourceInstance;
				}
			}
		
			if (AnimInstance)
			{
				const TArray<UAnimInstance*>& LinkedInstances = const_cast<const USkeletalMeshComponent*>(InSkeletalMeshComponent)->GetLinkedAnimInstances();
				for (UAnimInstance* LinkedInstance : LinkedInstances)
				{
					// Sub anim instances are always forced to do a parallel update 
					LinkedInstance->UpdateAnimation(0.0f, false, UAnimInstance::EUpdateAnimationFlag::ForceParallelUpdate);
				}

				AnimInstance->UpdateAnimation(0.0f, false);
			}

			// Update space bases to reset it back to ref pose
			InSkeletalMeshComponent->RefreshBoneTransforms();
			InSkeletalMeshComponent->RefreshFollowerComponents();
			InSkeletalMeshComponent->UpdateComponentToWorld();
		}

		// if not game world, don't clean this up
		if (InSkeletalMeshComponent->GetWorld() != nullptr && InSkeletalMeshComponent->GetWorld()->IsGameWorld() == false)
		{
			InSkeletalMeshComponent->ClearMotionVector();
		}
	}

private:
	/** Helper function for BindToSkeletalMeshComponent */
	static ANIMGRAPHRUNTIME_API bool ShouldCreateCustomInstancePlayer(const USkeletalMeshComponent* SkeletalMeshComponent);
};
