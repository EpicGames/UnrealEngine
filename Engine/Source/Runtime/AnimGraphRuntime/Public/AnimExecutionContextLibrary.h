// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimNodeReference.h"
#include "Animation/AnimExecutionContext.h"
#include "AnimExecutionContextLibrary.generated.h"

class UAnimInstance;

// Exposes operations to be performed on anim node contexts
UCLASS(MinimalAPI)
class UAnimExecutionContextLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
	
public:
#if WITH_EDITOR
	/** Prototype function for thread-safe anim node calls */
	UFUNCTION(BlueprintInternalUseOnly, meta=(BlueprintThreadSafe))
	void Prototype_ThreadSafeAnimNodeCall(const FAnimExecutionContext& Context, const FAnimNodeReference& Node) {}

	/** Prototype function for thread-safe anim update calls */
	UFUNCTION(BlueprintInternalUseOnly, meta=(BlueprintThreadSafe))
	void Prototype_ThreadSafeAnimUpdateCall(const FAnimUpdateContext& Context, const FAnimNodeReference& Node) {}	
#endif
	
	/** Get the anim instance that hosts this context */
	UFUNCTION(BlueprintPure, Category = "Animation|Utilities", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API UAnimInstance* GetAnimInstance(const FAnimExecutionContext& Context);

	/** Internal compiler use only - Get a reference to an anim node by index */
	UFUNCTION(BlueprintPure, BlueprintInternalUseOnly, meta=(BlueprintThreadSafe, DefaultToSelf = "Instance"))
	static ANIMGRAPHRUNTIME_API FAnimNodeReference GetAnimNodeReference(UAnimInstance* Instance, int32 Index);

	/** Convert to an initialization context */
	UFUNCTION(BlueprintCallable, Category = "Animation|Utilities", meta=(BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static ANIMGRAPHRUNTIME_API FAnimInitializationContext ConvertToInitializationContext(const FAnimExecutionContext& Context, EAnimExecutionContextConversionResult& Result);
	
	/** Convert to an update context */
	UFUNCTION(BlueprintCallable, Category = "Animation|Utilities", meta=(BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static ANIMGRAPHRUNTIME_API FAnimUpdateContext ConvertToUpdateContext(const FAnimExecutionContext& Context, EAnimExecutionContextConversionResult& Result);

	/** Get the current delta time in seconds */
	UFUNCTION(BlueprintPure, Category = "Animation|Utilities", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API float GetDeltaTime(const FAnimUpdateContext& Context);

	/** Get the current weight of this branch of the graph */
	UFUNCTION(BlueprintPure, Category = "Animation|Utilities", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API float GetCurrentWeight(const FAnimUpdateContext& Context);
	
	/** Get whether this branch of the graph is active (i.e. NOT blending out). */
	UFUNCTION(BlueprintPure, Category = "Animation|Utilities", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API bool IsActive(const FAnimExecutionContext& Context);

	/** Convert to a pose context */
	UFUNCTION(BlueprintCallable, Category = "Animation|Utilities", meta=(BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static ANIMGRAPHRUNTIME_API FAnimPoseContext ConvertToPoseContext(const FAnimExecutionContext& Context, EAnimExecutionContextConversionResult& Result);

	/** Convert to a component space pose context */
	UFUNCTION(BlueprintCallable, Category = "Animation|Utilities", meta=(BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static ANIMGRAPHRUNTIME_API FAnimComponentSpacePoseContext ConvertToComponentSpacePoseContext(const FAnimExecutionContext& Context, EAnimExecutionContextConversionResult& Result);
};
