// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimTrace.h"

#if ANIM_TRACE_ENABLED

struct FAnimationBaseContext;
struct FAnimNode_BlendSpacePlayerBase;
struct FAnimNode_BlendSpaceGraphBase;

class UBlendSpace;

struct FAnimGraphRuntimeTrace
{
	/** Helper function to output debug info for blendspace player nodes */
	ANIMGRAPHRUNTIME_API static void OutputBlendSpacePlayer(const FAnimationBaseContext& InContext, const FAnimNode_BlendSpacePlayerBase& InNode);

	/** Helper function to output debug info for blendspace nodes */
	ANIMGRAPHRUNTIME_API static void OutputBlendSpace(const FAnimationBaseContext& InContext, const FAnimNode_BlendSpaceGraphBase& InNode);

	/**
	 * Helper function to output blend space player debug info from a custom node that does not
	 * inherit from FAnimNode_BlendSpacePlayerBase or FAnimNode_BlendSpaceGraphBase.
	 * Uses a caller-supplied node ID so it works correctly when called from Evaluate_AnyThread
	 * where the context's current node ID may not be set.
	 */
	ANIMGRAPHRUNTIME_API static void OutputBlendSpacePlayerValues(
		const FAnimationBaseContext& InContext,
		int32 InNodeId,
		const UBlendSpace* InBlendSpace,
		FVector InPosition,
		FVector InFilteredPosition);
};

#define TRACE_BLENDSPACE_PLAYER(Context, Node) \
	FAnimGraphRuntimeTrace::OutputBlendSpacePlayer(Context, Node);

#define TRACE_BLENDSPACE(Context, Node) \
	FAnimGraphRuntimeTrace::OutputBlendSpace(Context, Node);

#else

#define TRACE_BLENDSPACE_PLAYER(Context, Node)
#define TRACE_BLENDSPACE(Context, Node)

#endif