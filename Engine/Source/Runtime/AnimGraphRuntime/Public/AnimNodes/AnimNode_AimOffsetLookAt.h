// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNodeBase.h"
#include "AnimNodes/AnimNode_BlendSpacePlayer.h"
#include "AnimNode_AimOffsetLookAt.generated.h"

/** 
 * This node uses a source transform of a socket on the skeletal mesh to automatically calculate
 * Yaw and Pitch directions for a referenced aim offset given a point in the world to look at.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_AimOffsetLookAt : public FAnimNode_BlendSpacePlayer
{
	GENERATED_BODY()

	/** Cached local transform of the source socket */
	FTransform SocketLocalTransform;

	/** Cached local transform of the pivot socket  */
	FTransform PivotSocketLocalTransform;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Links)
	FPoseLink BasePose;

	/*
	* Max LOD that this node is allowed to run
	* For example if you have LODThreshold to be 2, it will run until LOD 2 (based on 0 index)
	* when the component LOD becomes 3, it will stop update/evaluate
	* currently transition would be issue and that has to be re-visited
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Performance, meta = (DisplayName = "LOD Threshold"))
	int32 LODThreshold;

	/** Socket or bone to treat as the look at source. This will then be pointed at LookAtLocation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LookAt, meta = (PinHiddenByDefault))
	FName SourceSocketName;

	/** 
	 * Socket or bone to treat as the look at pivot (optional). This will overwrite the translation of the 
	 * source socket transform improve the lookat direction, especially when the target is close 
	 * to the character 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LookAt, meta = (PinHiddenByDefault))
	FName PivotSocketName;

	/** Location, in world space to look at */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LookAt, meta = (PinShownByDefault))
	FVector LookAtLocation;

	/** Direction in the socket transform to consider the 'forward' or look at axis */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LookAt, meta = (PinHiddenByDefault))
	FVector SocketAxis;

	/** Amount of this node to blend into the output pose */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LookAt, meta = (PinShownByDefault))
	float Alpha;

	/** Cached calculated blend input */
	FVector CurrentBlendInput;

	/** Cached reference to the source socket's bone */
	FBoneReference SocketBoneReference;

	/** Cached reference to the pivot socket's bone */
	FBoneReference PivotSocketBoneReference;

	/** Cached flag to indicate whether LOD threshold is enabled */
	bool bIsLODEnabled;

public:
	ANIMGRAPHRUNTIME_API FAnimNode_AimOffsetLookAt();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance) override;
	virtual bool NeedsOnInitializeAnimInstance() const override { return true; }
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void UpdateAssetPlayer(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	virtual int32 GetLODThreshold() const override { return LODThreshold; }
	// End of FAnimNode_Base interface

	// FAnimNode_BlendSpacePlayer interface
	ANIMGRAPHRUNTIME_API virtual FVector GetPosition() const override;

	ANIMGRAPHRUNTIME_API void UpdateFromLookAtTarget(FPoseContext& LocalPoseContext);
};
