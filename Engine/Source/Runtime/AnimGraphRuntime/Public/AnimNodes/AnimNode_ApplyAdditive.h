// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/InputScaleBias.h"
#include "AnimNode_ApplyAdditive.generated.h"

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_ApplyAdditive : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Links)
	FPoseLink Base;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Links)
	FPoseLink Additive;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Alpha, meta=(PinShownByDefault))
	float Alpha;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Alpha)
	FInputScaleBias AlphaScaleBias;

	/* 
	 * Max LOD that this node is allowed to run
	 * For example if you have LODThreshold to be 2, it will run until LOD 2 (based on 0 index)
	 * when the component LOD becomes 3, it will stop update/evaluate
	 * currently transition would be issue and that has to be re-visited
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Performance, meta=(DisplayName="LOD Threshold"))
	int32 LODThreshold;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (DisplayName = "Blend Settings", DisplayAfter = "bAlphaBoolEnabled"))
	FInputAlphaBoolBlend AlphaBoolBlend;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (PinShownByDefault))
	FName AlphaCurveName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha)
	FInputScaleBiasClamp AlphaScaleBiasClamp;

	float ActualAlpha;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (DisplayAfter = "AlphaScaleBias"))
	EAnimAlphaInputType AlphaInputType;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (PinShownByDefault, DisplayName = "bEnabled", DisplayAfter = "AlphaInputType"))
	bool bAlphaBoolEnabled;

public:	
	ANIMGRAPHRUNTIME_API FAnimNode_ApplyAdditive();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	virtual int32 GetLODThreshold() const override { return LODThreshold; }
	// End of FAnimNode_Base interface

};
