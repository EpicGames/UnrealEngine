// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/InputScaleBias.h"
#include "BoneContainer.h"
#include "AnimNode_BlendBoneByChannel.generated.h"

USTRUCT(BlueprintInternalUseOnly)
struct FBlendBoneByChannelEntry
{
	GENERATED_USTRUCT_BODY()

	/** Bone to take Transform from */
	UPROPERTY(EditAnywhere, Category = Blend)
	FBoneReference SourceBone;
	
	/** Bone to apply Transform to */
	UPROPERTY(EditAnywhere, Category = Blend)
	FBoneReference TargetBone;

	/** Copy Translation from Source to Target */
	UPROPERTY(EditAnywhere, Category = Blend)
	bool bBlendTranslation;

	/** Copy Rotation from Source to Target */
	UPROPERTY(EditAnywhere, Category = Blend)
	bool bBlendRotation;

	/** Copy Scale from Source to Target */
	UPROPERTY(EditAnywhere, Category = Blend)
	bool bBlendScale;

	FBlendBoneByChannelEntry()
		: bBlendTranslation(true)
		, bBlendRotation(true)
		, bBlendScale(true)
	{
	}
};

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_BlendBoneByChannel : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Links)
	FPoseLink A;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Links)
	FPoseLink B;

	UPROPERTY(EditAnywhere, Category = Blend, meta = (DisplayAfter = "AlphaScaleBias"))
	TArray<FBlendBoneByChannelEntry> BoneDefinitions;

private:
	// Array of bone entries, that has been validated to be correct at runtime.
	// So we don't have to perform validation checks per frame.
	TArray<FBlendBoneByChannelEntry> ValidBoneEntries;

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinShownByDefault, DisplayAfter = "B"))
	float Alpha;

private:
	float InternalBlendAlpha;

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	FInputScaleBias AlphaScaleBias;

	/** Space to convert transforms into prior to copying channels */
	UPROPERTY(EditAnywhere, Category = Blend)
	TEnumAsByte<EBoneControlSpace> TransformsSpace;

private:
	bool bBIsRelevant;

public:
	FAnimNode_BlendBoneByChannel()
		: Alpha(0.0f)
		, InternalBlendAlpha(0.0f)
		, TransformsSpace(EBoneControlSpace::BCS_BoneSpace)
		, bBIsRelevant(false)
	{
	}

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface
};

