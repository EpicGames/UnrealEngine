// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimNodeBase.h"
#include "AlphaBlend.h"
#include "AnimNode_BlendListBase.generated.h"

class UBlendProfile;
class UCurveFloat;

UENUM()
enum class EBlendListTransitionType : uint8
{
	StandardBlend,
	Inertialization
};

UENUM()
enum class EBlendListChildUpdateMode : uint8
{	
	/** Do not tick inactive children, do not reset on activate */
	Default,

	/** This reinitializes the re-activated child */
	ResetChildOnActivate,

	/** Always tick children even if they are not active */
	AlwaysTickChildren
};

// Blend list node; has many children
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_BlendListBase : public FAnimNode_Base
{
	GENERATED_BODY()

protected:	
	UPROPERTY(EditAnywhere, EditFixedSize, Category=Links)
	TArray<FPoseLink> BlendPose;

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, EditFixedSize, Category=Config, meta=(PinShownByDefault, FoldProperty))
	TArray<float> BlendTime;

	UPROPERTY(EditAnywhere, Category=Config, meta=(FoldProperty))
	EBlendListTransitionType TransitionType = EBlendListTransitionType::StandardBlend;

	UPROPERTY(EditAnywhere, Category=BlendType, meta=(FoldProperty))
	EAlphaBlendOption BlendType = UE::Anim::DefaultBlendOption;
	
protected:
	UE_DEPRECATED(5.6, "Use ChildUpateMode instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use the ChildUpateMode instead"))
	bool bResetChildOnActivation_DEPRECATED = false;
	
	UPROPERTY(EditAnywhere, Category = Option, meta=(FoldProperty))
	EBlendListChildUpdateMode ChildUpateMode = EBlendListChildUpdateMode::Default;

private:
	UPROPERTY(EditAnywhere, Category=BlendType, meta=(FoldProperty))
	TObjectPtr<UCurveFloat> CustomBlendCurve = nullptr;

	UPROPERTY(EditAnywhere, Category=BlendType, meta=(UseAsBlendProfile=true, FoldProperty))
	TObjectPtr<UBlendProfile> BlendProfile = nullptr;
#endif // #if WITH_EDITORONLY_DATA

protected:
	// Struct for tracking blends for each pose
	struct FBlendData
	{
		FAlphaBlend Blend;
		float Weight;
		float RemainingTime;
		float StartAlpha;
	};

	TArray<FBlendData> PerBlendData;

	// Per-bone blending data, allocated when using blend profiles
	TArray<FBlendSampleData> PerBoneSampleData;

	int32 LastActiveChildIndex = 0;

	// The blend profile used for the current blend
	// Note its possible that the blend profile changes based on the active child
	UBlendProfile* CurrentBlendProfile = nullptr;
	
public:	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	// this is a requirement for clang to compile without warnings.
	FAnimNode_BlendListBase() = default;
	~FAnimNode_BlendListBase() = default;
	FAnimNode_BlendListBase(const FAnimNode_BlendListBase&) = default;
	FAnimNode_BlendListBase(FAnimNode_BlendListBase&&) = default;
	FAnimNode_BlendListBase& operator=(const FAnimNode_BlendListBase&) = default;
	FAnimNode_BlendListBase& operator=(FAnimNode_BlendListBase&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	friend class UAnimGraphNode_BlendListBase;

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

#if WITH_EDITOR
	virtual void AddPose()
	{
		BlendTime.Add(0.1f);
		BlendPose.AddDefaulted();
	}

	virtual void RemovePose(int32 PoseIndex)
	{
		BlendTime.RemoveAt(PoseIndex);
		BlendPose.RemoveAt(PoseIndex);
	}
#endif

public:
	// Get the array of blend times to apply to our input poses
	ANIMGRAPHRUNTIME_API const TArray<float>& GetBlendTimes() const;

	// Get the type of transition that this blend list will make
	ANIMGRAPHRUNTIME_API EBlendListTransitionType GetTransitionType() const;

	// Get the blend type we will use when blending
	ANIMGRAPHRUNTIME_API EAlphaBlendOption GetBlendType() const;
	
	/** Get the child update mode. */
	ANIMGRAPHRUNTIME_API EBlendListChildUpdateMode GetChildUpdateMode() const;

	// Get the custom blend curve to apply when blending, if any
	ANIMGRAPHRUNTIME_API UCurveFloat* GetCustomBlendCurve() const;

	// Get the blend profile to use when blending, if any
	// Note that its possible for the blend profile to change based on the active child
	ANIMGRAPHRUNTIME_API virtual UBlendProfile* GetBlendProfile() const;
	
protected:
	virtual int32 GetActiveChildIndex() { return 0; }
	virtual FString GetNodeName(FNodeDebugData& DebugData) { return DebugData.GetNodeName(this); }

	ANIMGRAPHRUNTIME_API void Initialize();
	void InitializePerBoneData();
	void SetCurrentBlendProfile(UBlendProfile* NewBlendProfile);	

	friend class UBlendListBaseLibrary;
};
