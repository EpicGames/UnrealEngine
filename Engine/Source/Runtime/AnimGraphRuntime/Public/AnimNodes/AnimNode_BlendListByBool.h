// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimNodes/AnimNode_BlendListBase.h"
#include "AnimNode_BlendListByBool.generated.h"

// This node is effectively a 'branch', picking one of two input poses based on an input Boolean value
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_BlendListByBool : public FAnimNode_BlendListBase
{
	GENERATED_BODY()
private:
#if WITH_EDITORONLY_DATA
	// Used in conjunction with bUseSeperateBlendProfileForFalse
	UPROPERTY(EditAnywhere, Category = BlendType, meta = (UseAsBlendProfile = true, FoldProperty, EditCondition="bUseSeperateBlendProfileForFalse", DisplayAfter="bUseSeperateBlendProfileForFalse"))
	TObjectPtr<UBlendProfile> BlendProfileForFalse = nullptr;

	/* Specify whether to use a different blend profile for the 'false' branch than the true branch.
	*
	* If bUseSeperateBlendProfileForFalse is false (default), then the 'BlendProfile' is used when ActiveValue is both true or false
	* If bUseSeperateBlendProfileForFalse is true, then the 'BlendProfileForFalse' value is used when the ActiveValue is false, but 'BlendProfile' is used when ActiveValue is true
	*/ 
	UPROPERTY(EditAnywhere, Category = BlendType, meta = (FoldProperty, DisplayAfter="BlendProfile"))
	bool bUseSeperateBlendProfileForFalse = false;

	// Which input should be connected to the output?
	UPROPERTY(EditAnywhere, Category=Runtime, meta=(PinShownByDefault, FoldProperty))
	bool bActiveValue = false;
#endif
	
public:	
	FAnimNode_BlendListByBool() = default;

	// Get which input should be connected to the output
	ANIMGRAPHRUNTIME_API bool GetActiveValue() const;

	ANIMGRAPHRUNTIME_API bool GetUseSeperateBlendProfiles() const;

	ANIMGRAPHRUNTIME_API UBlendProfile* GetBlendProfileForFalse() const;
	
protected:
	ANIMGRAPHRUNTIME_API virtual int32 GetActiveChildIndex() override;
	virtual FString GetNodeName(FNodeDebugData& DebugData) override { return DebugData.GetNodeName(this); }	

	ANIMGRAPHRUNTIME_API virtual UBlendProfile* GetBlendProfile() const override;
};
