// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimNodes/AnimNode_BlendListBase.h"
#include "AnimNode_BlendListByEnum.generated.h"

// Blend List by Enum, it changes based on enum input that enters
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_BlendListByEnum : public FAnimNode_BlendListBase
{
	GENERATED_BODY()
	
private:
#if WITH_EDITORONLY_DATA
	// Mapping from enum value to BlendPose index; there will be one entry per entry in the enum; entries out of range always map to pose index 0
	UPROPERTY(meta=(FoldProperty))
	TArray<int32> EnumToPoseIndex;
	
	// The currently selected pose (as an enum value)
	UPROPERTY(EditAnywhere, Category=Runtime, meta=(PinShownByDefault, FoldProperty))
	mutable uint8 ActiveEnumValue = 0;
#endif
	
public:	
	FAnimNode_BlendListByEnum() = default;

#if WITH_EDITORONLY_DATA
	// Set the mapping from enum value to BlendPose index. Called during compilation.
	ANIMGRAPHRUNTIME_API void SetEnumToPoseIndex(const TArray<int32>& InEnumToPoseIndex);
#endif
	
	// Get the mapping from enum value to BlendPose index; there will be one entry per entry in the enum; entries out of range always map to pose index 0
	ANIMGRAPHRUNTIME_API const TArray<int32>& GetEnumToPoseIndex() const;
	
	// Get the currently selected pose (as an enum value)
	ANIMGRAPHRUNTIME_API uint8 GetActiveEnumValue() const;
	
protected:
	ANIMGRAPHRUNTIME_API virtual int32 GetActiveChildIndex() override;
	virtual FString GetNodeName(FNodeDebugData& DebugData) override { return DebugData.GetNodeName(this); }
};
