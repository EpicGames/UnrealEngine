// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimNodes/AnimNode_BlendListBase.h"
#include "AnimNode_BlendListByInt.generated.h"

// Blend list node; has many children
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_BlendListByInt : public FAnimNode_BlendListBase
{
	GENERATED_BODY()
private:
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere,  Category=Runtime, meta=(PinShownByDefault, FoldProperty))
	int32 ActiveChildIndex = 0;
#endif
	
public:
	FAnimNode_BlendListByInt() = default;

	// Get the currently active child index
	ANIMGRAPHRUNTIME_API virtual int32 GetActiveChildIndex() override;
	
protected:
	virtual FString GetNodeName(FNodeDebugData& DebugData) override { return DebugData.GetNodeName(this); }
};
