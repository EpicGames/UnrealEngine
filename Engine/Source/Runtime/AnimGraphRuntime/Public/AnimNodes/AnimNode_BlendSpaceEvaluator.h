// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimNodes/AnimNode_BlendSpacePlayer.h"
#include "AnimNode_BlendSpaceEvaluator.generated.h"

// Evaluates a BlendSpace at a specific using a specific time input rather than advancing time
// internally. Typically the playback position of the animation for this node will represent
// something other than time, like jump height. Note that events output from the sequences playing
// and being blended together should not be used. In addition, synchronization of animations
// will potentially be discontinuous if the blend weights are updated, as the leader/follower changes.
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_BlendSpaceEvaluator : public FAnimNode_BlendSpacePlayer
{
	GENERATED_USTRUCT_BODY()

public:
	/**
	 * Normalized time between [0,1]. The actual length of a blendspace is dynamic based on the coordinate, 
	 * so it is exposed as a normalized value. Note that treating this as a "time" value that increases (and wraps)
	 * will not result in the same output as you would get from using a BlendSpace player. The output events
	 * may not be as expected, and synchronization will sometimes be discontinuous if the leader/follower 
	 * animations change as a result of changing the blend weights (even if that is done smoothly).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings, meta=(PinShownByDefault))
	float NormalizedTime;

	/** 
	 * If true, teleport to normalized time, does NOT advance time (does not trigger notifies, does not 
	 * extract Root Motion, etc.). If false, will advance time (will trigger notifies, extract root motion 
	 * if applicable, etc). 
	 */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	bool bTeleportToNormalizedTime = true;

public:	
	ANIMGRAPHRUNTIME_API FAnimNode_BlendSpaceEvaluator();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void UpdateAssetPlayer(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// FAnimNode_BlendSpacePlayer interface
	ANIMGRAPHRUNTIME_API virtual float GetPlayRate() const override;
	virtual bool ShouldTeleportToTime() const override { return bTeleportToNormalizedTime; }
	virtual bool IsEvaluator() const override { return true; }
	// End of FAnimNode_BlendSpacePlayer

};
