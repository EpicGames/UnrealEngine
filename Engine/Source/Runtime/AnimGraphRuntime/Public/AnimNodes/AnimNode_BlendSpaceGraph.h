// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNode_BlendSpaceGraphBase.h"
#include "AnimNode_BlendSpaceGraph.generated.h"

// Allows multiple animations to be blended between based on input parameters
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_BlendSpaceGraph : public FAnimNode_BlendSpaceGraphBase
{
	GENERATED_BODY()

	// @return the sync group that this blendspace uses
	FName GetGroupName() const { return GroupName; }
};
