// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimNodeBase.h"
#include "AnimNode_BlendSpaceGraphBase.generated.h"

class UBlendSpace;

// Allows multiple animations to be blended between based on input parameters
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_BlendSpaceGraphBase : public FAnimNode_Base
{
	GENERATED_BODY()

	friend class UAnimGraphNode_BlendSpaceGraphBase;

	// @return the blendspace that this node uses
	const UBlendSpace* GetBlendSpace() const { return BlendSpace; }

	// @return the current sample coordinates that this node is using to sample the blendspace
	FVector GetPosition() const { return FVector(X, Y, 0); }

	// @return the current sample coordinates after going through the filtering
	FVector GetFilteredPosition() const { return BlendFilter.GetFilterLastOutput(); }

#if WITH_EDITORONLY_DATA
	// Set the node to preview a supplied sample value
	ANIMGRAPHRUNTIME_API void SetPreviewPosition(FVector InVector);
#endif

	// Forces the Position to the specified value
	ANIMGRAPHRUNTIME_API void SnapToPosition(const FVector& NewPosition);

protected:
	// The X coordinate to sample in the blendspace
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Coordinates, meta = (PinShownByDefault, AllowPrivateAccess))
	float X = 0.0f;

	// The Y coordinate to sample in the blendspace
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Coordinates, meta = (PinShownByDefault, AllowPrivateAccess))
	float Y = 0.0f;

	// The group name that we synchronize with (NAME_None if it is not part of any group). Note that
	// this is the name of the group used to sync the output of this node - it will not force
	// syncing of animations contained by it. Animations inside this Blend Space have their own
	// options for syncing.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sync, meta = (NeverAsPin, AllowPrivateAccess))
	FName GroupName = NAME_None;

	// The role this Blend Space can assume within the group (ignored if GroupName is not set). Note
	// that this is the role of the output of this node, not of animations contained by it.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sync, meta = (NeverAsPin, AllowPrivateAccess))
	TEnumAsByte<EAnimGroupRole::Type> GroupRole = EAnimGroupRole::CanBeLeader;

	// The internal blendspace asset to play
	UPROPERTY()
	TObjectPtr<const UBlendSpace> BlendSpace = nullptr;

	// Pose links for each sample in the blendspace
	UPROPERTY()
	TArray<FPoseLink> SamplePoseLinks;

protected:
	// FIR filter applied to inputs
	FBlendFilter BlendFilter;

	// Cache of sampled data, updated each frame
	TArray<FBlendSampleData> BlendSampleDataCache;

	/** Previous position in the triangulation/segmentation */
	int32 CachedTriangulationIndex = -1;

#if WITH_EDITORONLY_DATA
	// Preview blend params - set in editor only
	FVector PreviewPosition = FVector::ZeroVector;

	// Whether to use the preview blend params
	bool bUsePreviewPosition = false;
#endif

	// Internal update handler, skipping evaluation of exposed inputs
	ANIMGRAPHRUNTIME_API void UpdateInternal(const FAnimationUpdateContext& Context);

protected:
	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface
};
