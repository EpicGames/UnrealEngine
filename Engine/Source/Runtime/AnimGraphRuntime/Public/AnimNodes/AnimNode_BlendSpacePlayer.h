// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimNode_AssetPlayerBase.h"
#include "AnimNode_BlendSpacePlayer.generated.h"

class UBlendSpace;

//@TODO: Comment
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_BlendSpacePlayerBase : public FAnimNode_AssetPlayerBase
{
	GENERATED_BODY()

protected:
	// Filter used to dampen coordinate changes
	FBlendFilter BlendFilter;

	// Cache of samples used to determine blend weights
	TArray<FBlendSampleData> BlendSampleDataCache;

	/** Previous position in the triangulation/segmentation */
	int32 CachedTriangulationIndex = -1;

	UPROPERTY(Transient)
	TObjectPtr<UBlendSpace> PreviousBlendSpace = nullptr;

public:	

	// FAnimNode_AssetPlayerBase interface
	ANIMGRAPHRUNTIME_API virtual float GetCurrentAssetTime() const override;
	ANIMGRAPHRUNTIME_API virtual float GetCurrentAssetTimePlayRateAdjusted() const override;
	ANIMGRAPHRUNTIME_API virtual float GetCurrentAssetLength() const override;
	ANIMGRAPHRUNTIME_API virtual UAnimationAsset* GetAnimAsset() const override;
	// End of FAnimNode_AssetPlayerBase interface

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void UpdateAssetPlayer(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// Get the amount of time from the end
	ANIMGRAPHRUNTIME_API float GetTimeFromEnd(float CurrentTime) const;

	// @return the current sample coordinates after going through the filtering
	FVector GetFilteredPosition() const { return BlendFilter.GetFilterLastOutput(); }

	// Forces the Position to the specified value
	ANIMGRAPHRUNTIME_API void SnapToPosition(const FVector& NewPosition);

public:

	// Get the blendspace asset to play
	virtual UBlendSpace* GetBlendSpace() const PURE_VIRTUAL(FAnimNode_BlendSpacePlayerBase::GetBlendSpace, return nullptr;);

	// Get the coordinates that are currently being sampled by the blendspace
	virtual FVector GetPosition() const PURE_VIRTUAL(FAnimNode_BlendSpacePlayerBase::GetPosition, return FVector::Zero(););

	// The start position in [0, 1] to use when initializing. When looping, play will still jump back to the beginning when reaching the end.
	virtual float GetStartPosition() const PURE_VIRTUAL(FAnimNode_BlendSpacePlayerBase::GetStartPosition, return 0.0f;);

	// Get the play rate multiplier. Can be negative, which will cause the animation to play in reverse.
	virtual float GetPlayRate() const PURE_VIRTUAL(FAnimNode_BlendSpacePlayerBase::GetPlayRate, return 1.0f;);

	// Get whether we should reset the current play time when the blend space changes
	virtual bool ShouldResetPlayTimeWhenBlendSpaceChanges() const PURE_VIRTUAL(FAnimNode_BlendSpacePlayerBase::ShouldResetPlayTimeWhenBlendSpaceChanges, return true;);

	// Set whether we should reset the current play time when the blend space changes
	virtual bool SetResetPlayTimeWhenBlendSpaceChanges(bool bReset) PURE_VIRTUAL(FAnimNode_BlendSpacePlayerBase::SetResetPlayTimeWhenBlendSpaceChanges, return false;);

	// An evaluator will be setting the play rate to zero and setting the time explicitly. ShouldTeleportToTime indicates whether we should jump to that time, or move to it playing out root motion and events etc.
	virtual bool ShouldTeleportToTime() const { return false; }

	// Indicates if we are an evaluator - i.e. will be setting the time explicitly rather than letting it play out
	virtual bool IsEvaluator() const { return false; }

	// Set the blendspace asset to play
	virtual bool SetBlendSpace(UBlendSpace* InBlendSpace) PURE_VIRTUAL(FAnimNode_BlendSpacePlayerBase::SetBlendSpace, return false;);

	// Set the coordinates that are currently being sampled by the blendspace
	virtual bool SetPosition(FVector InPosition) PURE_VIRTUAL(FAnimNode_BlendSpacePlayerBase::SetPosition, return false;);

	// Set the play rate multiplier. Can be negative, which will cause the animation to play in reverse.
	virtual bool SetPlayRate(float InPlayRate) PURE_VIRTUAL(FAnimNode_BlendSpacePlayerBase::SetPlayRate, return false;);

	// Set if the animation should loop back to the start when it reaches the end?
	virtual bool SetLoop(bool bInLoop) PURE_VIRTUAL(FAnimNode_BlendSpacePlayerBase::SetLoop, return false;);

protected:
	ANIMGRAPHRUNTIME_API void UpdateInternal(const FAnimationUpdateContext& Context);

private:
	ANIMGRAPHRUNTIME_API void Reinitialize(bool bResetTime = true);

	ANIMGRAPHRUNTIME_API const FBlendSampleData* GetHighestWeightedSample() const;
};

template<>
struct TStructOpsTypeTraits<FAnimNode_BlendSpacePlayerBase> : public TStructOpsTypeTraitsBase2<FAnimNode_BlendSpacePlayerBase>
{
	enum
	{
		WithPureVirtual = true,
	};
};


//@TODO: Comment
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_BlendSpacePlayer : public FAnimNode_BlendSpacePlayerBase
{
	GENERATED_BODY()

	friend class UAnimGraphNode_BlendSpaceBase;
	friend class UAnimGraphNode_BlendSpacePlayer;
	friend class UAnimGraphNode_BlendSpaceEvaluator;
	friend class UAnimGraphNode_RotationOffsetBlendSpace;
	friend class UAnimGraphNode_AimOffsetLookAt;

private:

#if WITH_EDITORONLY_DATA
	// The group name that we synchronize with (NAME_None if it is not part of any group). Note that
	// this is the name of the group used to sync the output of this node - it will not force
	// syncing of animations contained by it.
	UPROPERTY(EditAnywhere, Category = Sync, meta = (FoldProperty))
	FName GroupName = NAME_None;

	// The role this node can assume within the group (ignored if GroupName is not set). Note
	// that this is the role of the output of this node, not of animations contained by it.
	UPROPERTY(EditAnywhere, Category = Sync, meta = (FoldProperty))
	TEnumAsByte<EAnimGroupRole::Type> GroupRole = EAnimGroupRole::CanBeLeader;

	// When enabled, acting as the leader, and using marker-based sync, this asset player will not sync to the previous leader's sync position when joining a sync group and before becoming the leader but instead force everyone else to match its position.
	UPROPERTY(EditAnywhere, Category = Sync, meta = (FoldProperty, EditCondition = "GroupRole != EAnimGroupRole::TransitionFollower && GroupRole != EAnimGroupRole::AlwaysFollower", EditConditionHides))
	bool bOverridePositionWhenJoiningSyncGroupAsLeader = false;
	
	// How this node will synchronize with other animations. Note that this determines how the output
	// of this node is used for synchronization, not of animations contained by it.
	UPROPERTY(EditAnywhere, Category = Sync, meta = (FoldProperty))
	EAnimSyncMethod Method = EAnimSyncMethod::DoNotSync;

	// If true, "Relevant anim" nodes that look for the highest weighted animation in a state will ignore this node
	UPROPERTY(EditAnywhere, Category = Relevancy, meta = (FoldProperty, PinHiddenByDefault))
	bool bIgnoreForRelevancyTest = false;

	// The X coordinate to sample in the blendspace
	UPROPERTY(EditAnywhere, Category = Coordinates, meta = (PinShownByDefault, FoldProperty))
	float X = 0.0f;

	// The Y coordinate to sample in the blendspace
	UPROPERTY(EditAnywhere, Category = Coordinates, meta = (PinShownByDefault, FoldProperty))
	float Y = 0.0f;

	// The play rate multiplier. Can be negative, which will cause the animation to play in reverse.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (DefaultValue = "1.0", PinHiddenByDefault, FoldProperty))
	float PlayRate = 1.0f;

	// Should the animation loop back to the start when it reaches the end?
	UPROPERTY(EditAnywhere, Category = Settings, meta = (DefaultValue = "true", PinHiddenByDefault, FoldProperty))
	bool bLoop = true;

	// Whether we should reset the current play time when the blend space changes
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, FoldProperty))
	bool bResetPlayTimeWhenBlendSpaceChanges = true;

	// The start position in [0, 1] to use when initializing. When looping, play will still jump back to the beginning when reaching the end.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (DefaultValue = "0.f", PinHiddenByDefault, FoldProperty))
	float StartPosition = 0.0f;
#endif

	// The blendspace asset to play
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	TObjectPtr<UBlendSpace> BlendSpace = nullptr;

public:

	// FAnimNode_AssetPlayerBase interface
	ANIMGRAPHRUNTIME_API virtual FName GetGroupName() const override;
	ANIMGRAPHRUNTIME_API virtual EAnimGroupRole::Type GetGroupRole() const override;
	ANIMGRAPHRUNTIME_API virtual bool GetOverridePositionWhenJoiningSyncGroupAsLeader() const override;
	ANIMGRAPHRUNTIME_API virtual EAnimSyncMethod GetGroupMethod() const override;
	ANIMGRAPHRUNTIME_API virtual bool GetIgnoreForRelevancyTest() const override;
	ANIMGRAPHRUNTIME_API virtual bool IsLooping() const override;
	ANIMGRAPHRUNTIME_API virtual bool SetGroupName(FName InGroupName) override;
	ANIMGRAPHRUNTIME_API virtual bool SetGroupRole(EAnimGroupRole::Type InRole) override;
	ANIMGRAPHRUNTIME_API virtual bool SetOverridePositionWhenJoiningSyncGroupAsLeader(bool InOverridePositionWhenJoiningSyncGroupAsLeader) override;;
	ANIMGRAPHRUNTIME_API virtual bool SetGroupMethod(EAnimSyncMethod InMethod) override;
	ANIMGRAPHRUNTIME_API virtual bool SetIgnoreForRelevancyTest(bool bInIgnoreForRelevancyTest) override;
	// End of FAnimNode_AssetPlayerBase interface

	// FAnimNode_BlendSpacePlayerBase interface
	ANIMGRAPHRUNTIME_API virtual UBlendSpace* GetBlendSpace() const override;
	ANIMGRAPHRUNTIME_API virtual FVector GetPosition() const override;
	ANIMGRAPHRUNTIME_API virtual float GetStartPosition() const override;
	ANIMGRAPHRUNTIME_API virtual float GetPlayRate() const override;
	ANIMGRAPHRUNTIME_API virtual bool ShouldResetPlayTimeWhenBlendSpaceChanges() const override;
	ANIMGRAPHRUNTIME_API virtual bool SetResetPlayTimeWhenBlendSpaceChanges(bool bReset) override;
	ANIMGRAPHRUNTIME_API virtual bool SetBlendSpace(UBlendSpace* InBlendSpace) override;
	ANIMGRAPHRUNTIME_API virtual bool SetPosition(FVector InPosition) override;
	ANIMGRAPHRUNTIME_API virtual bool SetPlayRate(float InPlayRate) override;
	ANIMGRAPHRUNTIME_API virtual bool SetLoop(bool bInLoop) override;
	// End of FAnimNode_BlendSpacePlayerBase interface
};

//@TODO: Comment
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_BlendSpacePlayer_Standalone : public FAnimNode_BlendSpacePlayerBase
{
	GENERATED_BODY()

private:

	// The group name that we synchronize with (NAME_None if it is not part of any group). Note that
	// this is the name of the group used to sync the output of this node - it will not force
	// syncing of animations contained by it. Animations inside this Blend Space have their own
	// options for syncing.
	UPROPERTY(EditAnywhere, Category = Sync)
	FName GroupName = NAME_None;

	// The role this Blend Space can assume within the group (ignored if GroupName is not set). Note
	// that this is the role of the output of this node, not of animations contained by it.
	UPROPERTY(EditAnywhere, Category = Sync)
	TEnumAsByte<EAnimGroupRole::Type> GroupRole = EAnimGroupRole::CanBeLeader;

	// When enabled, acting as the leader, and using marker-based sync, this asset player will not sync to the previous leader's sync position when joining a sync group and before becoming the leader but instead force everyone else to match its position.
	UPROPERTY(EditAnywhere, Category = Sync, meta = (EditCondition = "GroupRole != EAnimGroupRole::TransitionFollower && GroupRole != EAnimGroupRole::AlwaysFollower", EditConditionHides))
	bool bOverridePositionWhenJoiningSyncGroupAsLeader = false;
	
	// How this asset will synchronize with other assets. Note that this determines how the output
	// of this node is used for synchronization, not of animations contained by it.
	UPROPERTY(EditAnywhere, Category = Sync)
	EAnimSyncMethod Method = EAnimSyncMethod::DoNotSync;

	// If true, "Relevant anim" nodes that look for the highest weighted animation in a state will ignore this node
	UPROPERTY(EditAnywhere, Category = Relevancy, meta = (PinHiddenByDefault))
	bool bIgnoreForRelevancyTest = false;

	// The X coordinate to sample in the blendspace
	UPROPERTY(EditAnywhere, Category = Coordinates, meta = (PinShownByDefault))
	float X = 0.0f;

	// The Y coordinate to sample in the blendspace
	UPROPERTY(EditAnywhere, Category = Coordinates, meta = (PinShownByDefault))
	float Y = 0.0f;

	// The play rate multiplier. Can be negative, which will cause the animation to play in reverse.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (DefaultValue = "1.0", PinHiddenByDefault))
	float PlayRate = 1.0f;

	// Should the animation loop back to the start when it reaches the end?
	UPROPERTY(EditAnywhere, Category = Settings, meta = (DefaultValue = "true", PinHiddenByDefault))
	bool bLoop = true;

	// Whether we should reset the current play time when the blend space changes
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	bool bResetPlayTimeWhenBlendSpaceChanges = true;

	// The start position in [0, 1] to use when initializing. When looping, play will still jump back to the beginning when reaching the end.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (DefaultValue = "0.f", PinHiddenByDefault))
	float StartPosition = 0.0f;

	// The blendspace asset to play
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	TObjectPtr<UBlendSpace> BlendSpace = nullptr;

public:

	// FAnimNode_AssetPlayerBase interface
	virtual FName GetGroupName() const override { return GroupName; }
	virtual EAnimGroupRole::Type GetGroupRole() const override { return GroupRole; }
	virtual EAnimSyncMethod GetGroupMethod() const override { return Method; }
	virtual bool GetOverridePositionWhenJoiningSyncGroupAsLeader() const override { return bOverridePositionWhenJoiningSyncGroupAsLeader; };
	virtual bool GetIgnoreForRelevancyTest() const override { return bIgnoreForRelevancyTest; }
	virtual bool IsLooping() const override { return bLoop; }
	virtual bool SetGroupName(FName InGroupName) override { GroupName = InGroupName; return true; }
	virtual bool SetGroupRole(EAnimGroupRole::Type InRole) override { GroupRole = InRole; return true; }
	virtual bool SetGroupMethod(EAnimSyncMethod InMethod) override { Method = InMethod; return true; }
	virtual bool SetOverridePositionWhenJoiningSyncGroupAsLeader(bool InOverridePositionWhenJoiningSyncGroupAsLeader) override { bOverridePositionWhenJoiningSyncGroupAsLeader = InOverridePositionWhenJoiningSyncGroupAsLeader; return true; };
	virtual bool SetIgnoreForRelevancyTest(bool bInIgnoreForRelevancyTest) override { bIgnoreForRelevancyTest = bInIgnoreForRelevancyTest; return true; }
	// End of FAnimNode_AssetPlayerBase interface

	// FAnimNode_BlendSpacePlayerBase interface
	virtual UBlendSpace* GetBlendSpace() const override { return BlendSpace; }
	virtual FVector GetPosition() const override { return FVector(X, Y, 0.0); }
	virtual float GetStartPosition() const override { return StartPosition; }
	virtual float GetPlayRate() const override { return PlayRate; }
	virtual bool ShouldResetPlayTimeWhenBlendSpaceChanges() const override { return bResetPlayTimeWhenBlendSpaceChanges; }
	virtual bool SetResetPlayTimeWhenBlendSpaceChanges(bool bReset) override { bResetPlayTimeWhenBlendSpaceChanges = bReset; return true; }
	virtual bool SetBlendSpace(UBlendSpace* InBlendSpace) override { BlendSpace = InBlendSpace; return true; }
	virtual bool SetPosition(FVector InPosition) override { X = static_cast<float>(InPosition[0]); Y = static_cast<float>(InPosition[1]); return true; }
	virtual bool SetPlayRate(float InPlayRate) override { PlayRate = InPlayRate; return true; }
	virtual bool SetLoop(bool bInLoop) override { bLoop = bInLoop; return true; }
	// End of FAnimNode_BlendSpacePlayerBase interface
};
