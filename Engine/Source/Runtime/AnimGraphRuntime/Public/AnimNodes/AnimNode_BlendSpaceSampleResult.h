// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNode_Root.h"
#include "AnimNode_BlendSpaceSampleResult.generated.h"

// Root node of a blend space sample (sink node).
// We dont use AnimNode_Root to let us distinguish these nodes in the property list at link time.
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_BlendSpaceSampleResult : public FAnimNode_Root
{
	GENERATED_BODY()
};
