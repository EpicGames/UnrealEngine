// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimNodeFunctionRef.h"
#include "AnimNode_CallFunction.generated.h"

// When to call the function during the execution of the animation graph
UENUM()
enum class EAnimFunctionCallSite
{
	// Called when the node is initialized - i.e. it becomes weighted/relevant in the graph (before child nodes are initialized)
	OnInitialize,
	
	// Called when the node is updated (before child nodes are updated)
	OnUpdate,

	// Called when the node is updated for the first time with a valid weight
	OnBecomeRelevant,
	
	// Called when the node is evaluated (before child nodes are evaluated)
    OnEvaluate,

	// Called when the node is initialized - i.e. it becomes weighted/relevant in the graph (after child nodes are initialized)
    OnInitializePostRecursion UMETA(DisplayName = "On Initialize (Post Recursion)"),
	
	// Called when the node is updated (after child nodes are updated)
	OnUpdatePostRecursion UMETA(DisplayName = "On Update (Post Recursion)"),

	// Called when the node is updated for the first time with a valid weight (after child nodes are updated)
    OnBecomeRelevantPostRecursion UMETA(DisplayName = "On Become Relevant (Post Recursion)"),

	// Called when the node is evaluated (after child nodes are evaluated)
	OnEvaluatePostRecursion UMETA(DisplayName = "On Evaluate (Post Recursion)"),

	// Called when the node is updated, was at full weight and beings to blend out. Called before child nodes are
	// updated
	OnStartedBlendingOut,

	// Called when the node is updated, was at zero weight and beings to blend in. Called before child nodes are updated
    OnStartedBlendingIn,

	// Called when the node is updated, was at non-zero weight and finishes blending out. Called before child nodes are
	// updated (note that this is necessarily not called within the graph update but from outside)
	// @TODO: Not currently supported, needs subsystem support!
    OnFinishedBlendingOut UMETA(Hidden),

	// Called when the node is updated, was at non-zero weight and becomes full weight. Called before child nodes are
    // updated
    OnFinishedBlendingIn,
};

/** Calls specified user-defined events/functions during anim graph execution */
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_CallFunction : public FAnimNode_Base
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Links)
	FPoseLink Source;

#if WITH_EDITORONLY_DATA
	// Function to call
	UPROPERTY(meta=(FoldProperty))
	FAnimNodeFunctionRef Function;
#endif
	
	// Counter used to determine relevancy
	FGraphTraversalCounter Counter;

	// Weight to determine blend-related call sites
	float CurrentWeight = 0.0f;
	
	//  When to call the function during the execution of the animation graph
	UPROPERTY(EditAnywhere, Category="Function")
	EAnimFunctionCallSite CallSite = EAnimFunctionCallSite::OnUpdate;
		
	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance) override;
	virtual bool NeedsOnInitializeAnimInstance() const override { return true; }
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	// End of FAnimNode_Base interface

	// Calls the function we hold if the callsite matches the one we have set
	ANIMGRAPHRUNTIME_API void CallFunctionFromCallSite(EAnimFunctionCallSite InCallSite, const FAnimationBaseContext& InContext) const;

	// Get the function held on this node
	ANIMGRAPHRUNTIME_API const FAnimNodeFunctionRef& GetFunction() const;
};
