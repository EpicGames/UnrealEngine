// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNodeBase.h"
#include "AnimNode_CopyPoseFromMesh.generated.h"

class USkeletalMeshComponent;
struct FAnimInstanceProxy;

/**
 *	Simple controller to copy a bone's transform to another one.
 */

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_CopyPoseFromMesh : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()

	/*  This is used by default if it's valid */
	UPROPERTY(BlueprintReadWrite, transient, Category=Copy, meta=(PinShownByDefault))
	TWeakObjectPtr<USkeletalMeshComponent> SourceMeshComponent;

	/* If SourceMeshComponent is not valid, and if this is true, it will look for attahced parent as a source */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Copy, meta = (NeverAsPin))
	uint8 bUseAttachedParent : 1;

	/* Copy curves also from SouceMeshComponent. This will copy the curves if this instance also contains curve attributes */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Copy, meta = (NeverAsPin))
	uint8 bCopyCurves : 1;
  
	/* Copy custom attributes (animation attributes) from SourceMeshComponent */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, DisplayName = "Copy Custom Attributes(Animation Attributes)",Category = Copy, meta = (NeverAsPin))
	bool bCopyCustomAttributes;

	/* Use root space transform to copy to the target pose. By default, it copies their relative transform (bone space)*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Copy, meta = (NeverAsPin))
	uint8 bUseMeshPose : 1;

	/* If you want to specify copy root, use this - this will ensure copy only below of this joint (inclusively) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Copy, meta = (NeverAsPin))
	FName RootBoneToCopy;

	ANIMGRAPHRUNTIME_API FAnimNode_CopyPoseFromMesh();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	virtual bool HasPreUpdate() const override { return true; }
	ANIMGRAPHRUNTIME_API virtual void PreUpdate(const UAnimInstance* InAnimInstance) override;
	// End of FAnimNode_Base interface

private:
	// this is source mesh references, so that we could compare and see if it has changed
	TWeakObjectPtr<USkeletalMeshComponent>	CurrentlyUsedSourceMeshComponent;
	TWeakObjectPtr<USkeletalMesh>			CurrentlyUsedSourceMesh;
	TWeakObjectPtr<USkeletalMesh>			CurrentlyUsedMesh;

	// target mesh 
	TWeakObjectPtr<USkeletalMesh>			CurrentlyUsedTargetMesh;
	// cache of target space bases to source space bases
	TMap<int32, int32> BoneMapToSource;
	TMap<int32, int32> SourceBoneToTarget;

	// Cached transforms, copied on the game thread
	TArray<FTransform> SourceMeshTransformArray;

	// Cached transforms, copied on the game thread
	TBitArray<> SourceMeshValidMeshPoseTransforms;

	// Cached curves, copied on the game thread
	FBlendedHeapCurve SourceCurves;

	// Cached attributes, copied on the game thread
	UE::Anim::FMeshAttributeContainer SourceCustomAttributes;

	// reinitialize mesh component 
	ANIMGRAPHRUNTIME_API void ReinitializeMeshComponent(USkeletalMeshComponent* NewSkeletalMeshComponent, USkeletalMeshComponent* TargetMeshComponent);
	ANIMGRAPHRUNTIME_API void RefreshMeshComponent(USkeletalMeshComponent* TargetMeshComponent);
};
