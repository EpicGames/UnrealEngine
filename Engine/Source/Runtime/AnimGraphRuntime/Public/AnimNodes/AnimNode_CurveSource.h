// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptInterface.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/CurveSourceInterface.h"

#include "AnimNode_CurveSource.generated.h"

class UAnimInstance;

/** Supply curves from some external source (e.g. audio) */
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_CurveSource : public FAnimNode_Base
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Links)
	FPoseLink SourcePose;

	/** 
	 * The binding of the curve source we want to bind to.
	 * We will bind to an object that implements ICurveSourceInterface. First we check 
	 * the actor that owns this (if any), then we check each of its components to see if we should
	 * bind to the source that matches this name.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CurveSource, meta = (PinHiddenByDefault))
	FName SourceBinding;

	/** How much we wan to blend the curve in by */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CurveSource, meta = (PinShownByDefault))
	float Alpha;

	/** Our bound source */
	UPROPERTY(Transient)
	TScriptInterface<ICurveSourceInterface> CurveSource;

	ANIMGRAPHRUNTIME_API FAnimNode_CurveSource();

	// FAnimNode_Base interface
	virtual bool HasPreUpdate() const override { return true; }
	ANIMGRAPHRUNTIME_API virtual void PreUpdate(const UAnimInstance* InAnimInstance) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface
};
