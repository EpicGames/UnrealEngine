// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimData/BoneMaskFilter.h"
#include "AnimNode_LayeredBoneBlend.generated.h"

UENUM()
enum class ELayeredBoneBlendMode : uint8
{
	BranchFilter,
	BlendMask,
};

// Layered blend (per bone); has dynamic number of blendposes that can blend per different bone sets
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_LayeredBoneBlend : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()

public:
	/** The source pose */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Links)
	FPoseLink BasePose;

	/** Each layer's blended pose */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, editfixedsize, Category=Links, meta=(BlueprintCompilerGeneratedDefaults))
	TArray<FPoseLink> BlendPoses;

	/** Whether to use branch filters or a blend mask to specify an input pose per-bone influence */
	UPROPERTY(EditAnywhere, Category = Config)
	ELayeredBoneBlendMode BlendMode;

	/** 
	 * The blend masks to use for our layer inputs. Allows the use of per-bone alphas.
	 * Blend masks are used when BlendMode is BlendMask.
	 */
	UPROPERTY(EditAnywhere, editfixedsize, Category=Config, meta=(UseAsBlendMask=true))
	TArray<TObjectPtr<UBlendProfile>> BlendMasks;

	/** 
	 * Configuration for the parts of the skeleton to blend for each layer. Allows
	 * certain parts of the tree to be blended out or omitted from the pose.
	 * LayerSetup is used when BlendMode is BranchFilter.
	 */
	UPROPERTY(EditAnywhere, editfixedsize, Category=Config)
	TArray<FInputBlendPose> LayerSetup;

	/** The weights of each layer */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, editfixedsize, Category=Runtime, meta=(BlueprintCompilerGeneratedDefaults, PinShownByDefault))
	TArray<float> BlendWeights;

protected:
	// transient data to handle weight and target weight
	// this array changes based on required bones
	TArray<FPerBoneBlendWeight> DesiredBoneBlendWeights;
	TArray<FPerBoneBlendWeight> CurrentBoneBlendWeights;

	// Per-bone weights for the skeleton. Serialized as these are only relative to the skeleton, but can potentially
	// be regenerated at runtime if the GUIDs dont match
	UPROPERTY()
	TArray<FPerBoneBlendWeight>	PerBoneBlendWeights;

	// Per-curve source pose index
	TBaseBlendedCurve<FDefaultAllocator, UE::Anim::FCurveElementIndexed> CurvePoseSourceIndices;

	// Guids for skeleton used to determine whether the PerBoneBlendWeights need rebuilding
	UPROPERTY()
	FGuid SkeletonGuid;

	// Guid for virtual bones used to determine whether the PerBoneBlendWeights need rebuilding
	UPROPERTY()
	FGuid VirtualBoneGuid;

	// Serial number of the required bones container
	uint16 RequiredBonesSerialNumber;

public:
	/*
	 * Max LOD that this node is allowed to run
	 * For example if you have LODThreshold to be 2, it will run until LOD 2 (based on 0 index)
	 * when the component LOD becomes 3, it will stop update/evaluate
	 * currently transition would be issue and that has to be re-visited
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Performance, meta = (DisplayName = "LOD Threshold"))
	int32 LODThreshold;

	/** Whether to blend bone rotations in mesh space or in local space */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Config)
	bool bMeshSpaceRotationBlend;

	/** Whether to blend bone rotations in root space or in mesh space */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Config, Meta = (Editcondition = bMeshSpaceRotationBlend))
	bool bRootSpaceRotationBlend;

	/** Whether to blend bone scales in mesh space or in local space */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Config)
	bool bMeshSpaceScaleBlend;
	
	/** How to blend the layers together */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Config)
	TEnumAsByte<enum ECurveBlendOption::Type> CurveBlendOption;

	/** Whether to incorporate the per-bone blend weight of the root bone when lending root motion */
	UPROPERTY(EditAnywhere, Category = Config)
	bool bBlendRootMotionBasedOnRootBone;

	/** Whether to update the base pose first or after all the blend poses */
	UPROPERTY(Experimental, EditAnywhere, BlueprintReadWrite, Category = Config, meta = (PinHiddenByDefault))
	bool bUpdateBasePoseFirst = false;

	bool bHasRelevantPoses;

	FAnimNode_LayeredBoneBlend()
		: BlendMode(ELayeredBoneBlendMode::BranchFilter)
		, RequiredBonesSerialNumber(0)
		, LODThreshold(INDEX_NONE)
		, bMeshSpaceRotationBlend(false)
		, bRootSpaceRotationBlend(false)
		, bMeshSpaceScaleBlend(false)
		, CurveBlendOption(ECurveBlendOption::Override)
		, bBlendRootMotionBasedOnRootBone(true)
		, bHasRelevantPoses(false)
	{
	}

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	virtual int32 GetLODThreshold() const override { return LODThreshold; }
	// End of FAnimNode_Base interface

	void AddPose()
	{
		BlendWeights.Add(1.f);
		BlendPoses.AddDefaulted();
		SyncBlendMasksAndLayers();
	}

	void RemovePose(int32 PoseIndex)
	{
		BlendWeights.RemoveAt(PoseIndex);
		BlendPoses.RemoveAt(PoseIndex);

		if (BlendMasks.IsValidIndex(PoseIndex)) 
		{ 
			BlendMasks.RemoveAt(PoseIndex);
		}

		if (LayerSetup.IsValidIndex(PoseIndex)) 
		{ 
			LayerSetup.RemoveAt(PoseIndex); 
		}
		// just in case, we've seen problems.
		SyncBlendMasksAndLayers();
	}

	// Set the blend mask for the specified input pose
	ANIMGRAPHRUNTIME_API void SetBlendMask(int32 InPoseIndex, UBlendProfile* InBlendMask);
	
	// Invalidate the cached per-bone blend weights from the skeleton
	void InvalidatePerBoneBlendWeights() { RequiredBonesSerialNumber = 0; SkeletonGuid = FGuid(); VirtualBoneGuid = FGuid(); }
	
	// Invalidates the cached bone data so it is recalculated the next time this node is updated
	void InvalidateCachedBoneData() { RequiredBonesSerialNumber = 0; }

private:

	// just for the graph constructor to call.
	void AddFirstPose()
	{
		check(BlendWeights.IsEmpty());
		BlendWeights.Add(1.f);
		BlendPoses.AddDefaulted();
		check(BlendMode == ELayeredBoneBlendMode::BranchFilter);
		LayerSetup.AddDefaulted();
	}

	// Synchronize the number of BlendMasks or Layers with the number of BlendPoses.
	ANIMGRAPHRUNTIME_API void SyncBlendMasksAndLayers();
	
	// Rebuild cache per bone blend weights from the skeleton
	ANIMGRAPHRUNTIME_API void RebuildPerBoneBlendWeights(const USkeleton* InSkeleton);

	// Check whether per-bone blend weights are valid according to the skeleton (GUID check)
	ANIMGRAPHRUNTIME_API bool ArePerBoneBlendWeightsValid(const USkeleton* InSkeleton) const;

	// Update cached data if required
	ANIMGRAPHRUNTIME_API void UpdateCachedBoneData(const FBoneContainer& RequiredBones, const USkeleton* Skeleton);

	friend class UAnimGraphNode_LayeredBoneBlend;
};
