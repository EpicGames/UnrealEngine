// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNodeBase.h"
#include "AnimNode_MakeDynamicAdditive.generated.h"

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_MakeDynamicAdditive : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()

	// Reference pose for additive delta calculation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Links)
	FPoseLink Base;

	// Pose to make additive
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Links)
	FPoseLink Additive;

	// Do additive delta calculation in mesh space
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings)
	bool bMeshSpaceAdditive;

public:	
	ANIMGRAPHRUNTIME_API FAnimNode_MakeDynamicAdditive();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

};
