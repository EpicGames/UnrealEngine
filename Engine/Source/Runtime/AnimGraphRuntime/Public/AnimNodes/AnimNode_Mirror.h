// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/MirrorDataTable.h"
#include "AlphaBlend.h"
#include "AnimNode_Mirror.generated.h"

UENUM()
enum class EMirrorBlendType : uint8
{
	// Use a standard crossfade blend. If Reset Child on Mirror State Change is enabled, this will result in a pop.
	StandardBlend,
	// Use an inertial blend. This requires having a matching inertialization or dead blend node upstream in the graph.
	Inertialization
};

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_MirrorBase : public FAnimNode_Base
{
	GENERATED_BODY()
public:
	ANIMGRAPHRUNTIME_API FAnimNode_MirrorBase(); 
	ANIMGRAPHRUNTIME_API FAnimNode_MirrorBase(const FAnimNode_MirrorBase&); 
	ANIMGRAPHRUNTIME_API ~FAnimNode_MirrorBase(); 

	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;

	// Get the MirrorDataTable
	ANIMGRAPHRUNTIME_API virtual UMirrorDataTable* GetMirrorDataTable() const;

	ANIMGRAPHRUNTIME_API virtual bool SetMirrorDataTable(UMirrorDataTable* MirrorTable);

	// Get mirror blend type. Default to inertialization for backwards compatibility.
	ANIMGRAPHRUNTIME_API virtual EMirrorBlendType GetBlendType() const;

	// Get Mirror State
	ANIMGRAPHRUNTIME_API virtual bool GetMirror() const;
	// How long to blend using inertialization when switching  mirrored state
	ANIMGRAPHRUNTIME_API virtual float GetBlendTimeOnMirrorStateChange() const;

	// Should bones mirror
	ANIMGRAPHRUNTIME_API virtual bool GetBoneMirroring() const;

	// Should the curves mirror
	ANIMGRAPHRUNTIME_API virtual bool GetCurveMirroring() const;

	// Should attributes mirror (based on the bone mirroring data in the mirror data table) 
	ANIMGRAPHRUNTIME_API virtual bool GetAttributeMirroring() const;

	// Whether to reset (reinitialize) the child (source) pose when the mirror state changes
	ANIMGRAPHRUNTIME_API virtual bool GetResetChildOnMirrorStateChange() const;

	// Set Mirror State
	ANIMGRAPHRUNTIME_API virtual bool SetMirror(bool bInMirror);

	// Set how long to blend using inertialization when switching  mirrored state
	// @return true if the value was set (it is dynamic), or false if it could not (it is not dynamic or pin exposed)
	ANIMGRAPHRUNTIME_API virtual bool SetBlendTimeOnMirrorStateChange(float InBlendTime);

	// Set if bones mirror
	// @return true if the value was set (it is dynamic), or false if it could not (it is not dynamic or pin exposed)
	ANIMGRAPHRUNTIME_API virtual bool SetBoneMirroring(bool bInBoneMirroring);

	// Set if curves mirror
	// @return true if the value was set (it is dynamic), or false if it could not (it is not dynamic or pin exposed)
	ANIMGRAPHRUNTIME_API virtual bool SetCurveMirroring(bool bInCurveMirroring);

	// Set if attributes mirror
	// @return true if the value was set (it is dynamic), or false if it could not (it is not dynamic or pin exposed)
	ANIMGRAPHRUNTIME_API virtual bool SetAttributeMirroring(bool bInAttributeMirroring);

	// Set whether to reset (reinitialize) the child (source) pose when the mirror state changes
	// @return true if the value was set (it is dynamic), or false if it could not (it is not dynamic or pin exposed)
	ANIMGRAPHRUNTIME_API virtual bool SetResetChildOnMirrorStateChange(bool bInResetChildOnMirrorStateChange);

	/** This only used by custom handlers, and it is advanced feature. */
	ANIMGRAPHRUNTIME_API virtual void SetSourceLinkNode(FAnimNode_Base* NewLinkNode);

	/** This only used by custom handlers, and it is advanced feature. */
	ANIMGRAPHRUNTIME_API virtual FAnimNode_Base* GetSourceLinkNode();
protected:
	UPROPERTY(EditAnywhere, Category = Links)
	FPoseLink Source;

private:
	FAlphaBlend Blend;

	bool bMirrorState;
	bool bMirrorStateIsValid;

	void MirrorPoseInPlace(FPoseContext& InPose);
	void FillCompactPoseAndComponentRefRotations(const FBoneContainer& BoneContainer);
	// Compact pose format of Mirror Bone Map
	TCustomBoneIndexArray<FCompactPoseBoneIndex, FCompactPoseBoneIndex> CompactPoseMirrorBones;

	// Pre-calculated component space of reference pose, which allows mirror to work with any joint orient 
	TCustomBoneIndexArray<FQuat, FCompactPoseBoneIndex> ComponentSpaceRefRotations;
};


USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_Mirror : public FAnimNode_MirrorBase
{
	GENERATED_BODY()

	friend class UAnimGraphNode_Mirror;

public:
	ANIMGRAPHRUNTIME_API FAnimNode_Mirror();
	ANIMGRAPHRUNTIME_API ~FAnimNode_Mirror();

	ANIMGRAPHRUNTIME_API virtual UMirrorDataTable* GetMirrorDataTable() const override;
	ANIMGRAPHRUNTIME_API virtual bool SetMirrorDataTable(UMirrorDataTable* MirrorTable) override;

	ANIMGRAPHRUNTIME_API virtual EMirrorBlendType GetBlendType() const;
	ANIMGRAPHRUNTIME_API virtual bool GetMirror() const override;
	ANIMGRAPHRUNTIME_API virtual float GetBlendTimeOnMirrorStateChange() const override;
	ANIMGRAPHRUNTIME_API virtual bool GetBoneMirroring() const override;
	ANIMGRAPHRUNTIME_API virtual bool GetCurveMirroring() const override;
	ANIMGRAPHRUNTIME_API virtual bool GetAttributeMirroring() const override;
	ANIMGRAPHRUNTIME_API virtual bool GetResetChildOnMirrorStateChange() const override;

	ANIMGRAPHRUNTIME_API virtual bool SetMirror(bool bInMirror) override;
	ANIMGRAPHRUNTIME_API virtual bool SetBlendTimeOnMirrorStateChange(float InBlendTime) override;
	ANIMGRAPHRUNTIME_API virtual bool SetBoneMirroring(bool bInBoneMirroring) override;
	ANIMGRAPHRUNTIME_API virtual bool SetCurveMirroring(bool bInCurveMirroring) override;
	ANIMGRAPHRUNTIME_API virtual bool SetAttributeMirroring(bool bInAttributeMirroring) override;
	ANIMGRAPHRUNTIME_API virtual bool SetResetChildOnMirrorStateChange(bool bInResetChildOnMirrorStateChange) override;

protected:

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinShownByDefault, FoldProperty))
	bool bMirror = true; 

	UPROPERTY(EditAnywhere, Category = Settings, meta = (FoldProperty))
	TObjectPtr<UMirrorDataTable> MirrorDataTable = nullptr;

	// Inertialization blend time to use when transitioning between mirrored and unmirrored states
	UPROPERTY(EditAnywhere, Category = MirrorTransition, meta = (PinHiddenByDefault, FoldProperty))
	float BlendTime = 0.0f;

	UPROPERTY(EditAnywhere, Category = MirrorTransition, meta = (PinHiddenByDefault, FoldProperty))
	EMirrorBlendType BlendType = EMirrorBlendType::Inertialization;

	// Whether to reset (reinitialize) the child (source) pose when the mirror state changes
	UPROPERTY(EditAnywhere, Category = MirrorTransition, meta = (FoldProperty))
	bool bResetChild = false;

	UPROPERTY(EditAnywhere, Category = MirroredChannels, meta=(DisplayName="Bone", FoldProperty))
	bool bBoneMirroring = true;

	UPROPERTY(EditAnywhere, Category = MirroredChannels, meta=(DisplayName = "Curve", FoldProperty))
	bool bCurveMirroring = true;

	UPROPERTY(EditAnywhere, Category = MirroredChannels, meta=(DisplayName = "Attributes", FoldProperty))
	bool bAttributeMirroring = true;
#endif
};


USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_Mirror_Standalone : public FAnimNode_MirrorBase
{
	GENERATED_BODY()

	friend class UAnimGraphNode_Mirror;

public:
	ANIMGRAPHRUNTIME_API FAnimNode_Mirror_Standalone();
	ANIMGRAPHRUNTIME_API FAnimNode_Mirror_Standalone(const FAnimNode_Mirror_Standalone&);
	ANIMGRAPHRUNTIME_API ~FAnimNode_Mirror_Standalone();

	ANIMGRAPHRUNTIME_API virtual UMirrorDataTable* GetMirrorDataTable() const override;
	ANIMGRAPHRUNTIME_API virtual bool SetMirrorDataTable(UMirrorDataTable* MirrorTable) override;

	ANIMGRAPHRUNTIME_API virtual EMirrorBlendType GetBlendType() const;
	ANIMGRAPHRUNTIME_API virtual bool GetMirror() const override;
	ANIMGRAPHRUNTIME_API virtual float GetBlendTimeOnMirrorStateChange() const override;
	ANIMGRAPHRUNTIME_API virtual bool GetBoneMirroring() const override;
	ANIMGRAPHRUNTIME_API virtual bool GetCurveMirroring() const override;
	ANIMGRAPHRUNTIME_API virtual bool GetAttributeMirroring() const override;
	ANIMGRAPHRUNTIME_API virtual bool GetResetChildOnMirrorStateChange() const override;

	ANIMGRAPHRUNTIME_API virtual bool SetMirror(bool bInMirror) override;
	ANIMGRAPHRUNTIME_API virtual bool SetBlendTimeOnMirrorStateChange(float InBlendTime) override;
	ANIMGRAPHRUNTIME_API virtual bool SetBoneMirroring(bool bInBoneMirroring) override;
	ANIMGRAPHRUNTIME_API virtual bool SetCurveMirroring(bool bInCurveMirroring) override;
	ANIMGRAPHRUNTIME_API virtual bool SetAttributeMirroring(bool bInAttributeMirroring) override;
	ANIMGRAPHRUNTIME_API virtual bool SetResetChildOnMirrorStateChange(bool bInResetChildOnMirrorStateChange) override;

protected:

	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinShownByDefault))
	bool bMirror = true;

	UPROPERTY(EditAnywhere, Category = Settings)
	TObjectPtr<UMirrorDataTable> MirrorDataTable = nullptr;

	// Inertialization blend time to use when transitioning between mirrored and unmirrored states
	UPROPERTY(EditAnywhere, Category = MirrorTransition, meta = (PinHiddenByDefault, FoldProperty))
	float BlendTime = 0.0f;

	UPROPERTY(EditAnywhere, Category = MirrorTransition, meta = (PinHiddenByDefault, FoldProperty))
	EMirrorBlendType BlendType = EMirrorBlendType::Inertialization;

	// Whether to reset (reinitialize) the child (source) pose when the mirror state changes
	UPROPERTY(EditAnywhere, Category = MirrorTransition, meta = (FoldProperty))
	bool bResetChild = false;

	UPROPERTY(EditAnywhere, Category = MirroredChannels, meta = (DisplayName = "Bone", FoldProperty))
	bool bBoneMirroring = true;

	UPROPERTY(EditAnywhere, Category = MirroredChannels, meta = (DisplayName = "Curve", FoldProperty))
	bool bCurveMirroring = true;

	UPROPERTY(EditAnywhere, Category = MirroredChannels, meta = (DisplayName = "Attributes", FoldProperty))
	bool bAttributeMirroring = true;
};
