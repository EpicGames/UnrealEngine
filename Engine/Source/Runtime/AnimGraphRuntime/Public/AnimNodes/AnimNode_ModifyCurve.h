// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNodeBase.h"
#include "AnimNode_ModifyCurve.generated.h"

UENUM()
enum class EModifyCurveApplyMode : uint8
{
	/** Add new value to input curve value */
	Add,

	/** Scale input value by new value */
	Scale,

	/** Blend input with new curve value, using Alpha setting on the node */
	Blend,

	/** Blend the new curve value with the last curve value using Alpha to determine the weighting (.5 is a moving average, higher values react to new values faster, lower slower) */
	WeightedMovingAverage,

	/** Remaps the new curve value between the CurveValues entry and 1.0 (.5 in CurveValues makes 0.51 map to 0.02) */
	RemapCurve
};

/** Easy way to modify curve values on a pose */
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_ModifyCurve : public FAnimNode_Base
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, EditFixedSize, BlueprintReadWrite, Category = Links)
	FPoseLink SourcePose;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, editfixedsize, Category = ModifyCurve, meta = (PinHiddenByDefault))
	TMap<FName, float> CurveMap;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, editfixedsize, Category = ModifyCurve, meta = (BlueprintCompilerGeneratedDefaults, PinShownByDefault))
	TArray<float> CurveValues;

	UPROPERTY(meta = (BlueprintCompilerGeneratedDefaults))
	TArray<FName> CurveNames;

	FBlendedHeapCurve LastCurve;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ModifyCurve, meta = (PinShownByDefault))
	float Alpha;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ModifyCurve)
	EModifyCurveApplyMode ApplyMode;

	ANIMGRAPHRUNTIME_API FAnimNode_ModifyCurve();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;

	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

#if WITH_EDITOR
	/** Add new curve being modified */
	ANIMGRAPHRUNTIME_API void AddCurve(const FName& InName, float InValue);
	/** Remove a curve from being modified */
	ANIMGRAPHRUNTIME_API void RemoveCurve(int32 PoseIndex);
#endif // WITH_EDITOR

private:
	ANIMGRAPHRUNTIME_API float ProcessCurveOperation(float CurrentValue, float NewValue) const;
	ANIMGRAPHRUNTIME_API float ProcessCurveWMAOperation(float CurrentValue, float LastValue) const;
};
