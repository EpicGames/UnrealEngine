// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNodeBase.h"
#include "Animation/InputScaleBias.h"
#include "AnimNode_MultiWayBlend.generated.h"

// This represents a baked transition
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_MultiWayBlend : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Links, meta = (BlueprintCompilerGeneratedDefaults))
	TArray<FPoseLink> Poses;

	UPROPERTY(EditAnywhere, EditFixedSize, BlueprintReadWrite, Category=Settings, meta=(BlueprintCompilerGeneratedDefaults, PinShownByDefault))
	TArray<float> DesiredAlphas;

private:
	TArray<float> CachedAlphas;

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings)
	FInputScaleBias AlphaScaleBias;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bAdditiveNode;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bNormalizeAlpha;

public:
	FAnimNode_MultiWayBlend()
		: bAdditiveNode(false)
		, bNormalizeAlpha(true)
	{
	}

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	int32 AddPose()
	{
		Poses.AddDefaulted();
		DesiredAlphas.AddZeroed();
		UpdateCachedAlphas();

		return Poses.Num();
	}

	void RemovePose(int32 PoseIndex)
	{
		Poses.RemoveAt(PoseIndex);
		CachedAlphas.RemoveAt(PoseIndex);
		DesiredAlphas.RemoveAt(PoseIndex);
	}

	void ResetPoses()
	{
		Poses.Reset();
		CachedAlphas.Reset();
		DesiredAlphas.Reset();
	}

	float GetTotalAlpha() const
	{
		float TotalAlpha = 0.f;

		for (float Alpha : DesiredAlphas)
		{
			TotalAlpha += Alpha;
		}

		return TotalAlpha;
	}

private:
	// process new weights and then return out
	ANIMGRAPHRUNTIME_API void UpdateCachedAlphas();

	friend class UAnimGraphNode_MultiWayBlend;
};

