// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimNodes/AnimNode_PoseHandler.h"
#include "AlphaBlend.h"
#include "Animation/AnimBulkCurves.h"
#include "AnimNode_PoseBlendNode.generated.h"

// Evaluates a point in an anim sequence, using a specific time input rather than advancing time internally.
// Typically the playback position of the animation for this node will represent something other than time, like jump height.
// This node will not trigger any notifies present in the associated sequence.
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_PoseBlendNode : public FAnimNode_PoseHandler
{
	GENERATED_USTRUCT_BODY()
public:
	UPROPERTY(EditAnywhere, EditFixedSize, BlueprintReadWrite, Category = Links)
	FPoseLink SourcePose;

	/** Type of blending used (Linear, Cubic, etc.) */
	UPROPERTY(EditAnywhere, Category = "Blend")
	EAlphaBlendOption BlendOption;

	/** If you're using Custom BlendOption, you can specify curve */
	UPROPERTY(EditAnywhere, Category = "Blend")
	TObjectPtr<UCurveFloat> CustomCurve;

	UPROPERTY(EditAnywhere, Category = "Blend", BlueprintReadWrite, meta = (PinHiddenByDefault))
	bool bIgnorePoseAssetSourceAnimationCurves = false;

private:
	// Cached curves to extract
	UE::Anim::TNamedValueArray<FDefaultAllocator, UE::Anim::FNamedIndexElement> BulkCurves;

public:	
	ANIMGRAPHRUNTIME_API FAnimNode_PoseBlendNode();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void UpdateAssetPlayer(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// FAnimNode_PoseHandler interface 
	ANIMGRAPHRUNTIME_API virtual void RebuildPoseList(const FBoneContainer& InBoneContainer, const UPoseAsset* InPoseAsset) override;
};

