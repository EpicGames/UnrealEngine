// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimNodes/AnimNode_PoseHandler.h"
#include "AnimNode_PoseByName.generated.h"

// Evaluates a point in an anim sequence, using a specific time input rather than advancing time internally.
// Typically the playback position of the animation for this node will represent something other than time, like jump height.
// This node will not trigger any notifies present in the associated sequence.
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_PoseByName : public FAnimNode_PoseHandler
{
	GENERATED_USTRUCT_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinHiddenByDefault))
	FName PoseName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinHiddenByDefault))
	float PoseWeight;

private:
	/** Cached current pose name used for invalidation */
	FName CurrentPoseName;

public:	
	FAnimNode_PoseByName()
		: PoseWeight(1.f)
		, CurrentPoseName(NAME_None)
	{
	}

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void UpdateAssetPlayer(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface
private:
	ANIMGRAPHRUNTIME_API virtual void RebuildPoseList(const FBoneContainer& InBoneContainer, const UPoseAsset* InPoseAsset) override;
};

