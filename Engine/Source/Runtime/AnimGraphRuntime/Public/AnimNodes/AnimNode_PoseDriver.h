// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneContainer.h"
#include "Animation/AnimNodeBase.h"
#include "AnimNodes/AnimNode_PoseHandler.h"
#include "RBF/RBFSolver.h"
#include "AnimNode_PoseDriver.generated.h"

// Deprecated
UENUM()
enum class EPoseDriverType : uint8
{
	SwingAndTwist,
	SwingOnly,
	Translation
};

/** Transform aspect used to drive interpolation */
UENUM(BlueprintType)
enum class EPoseDriverSource : uint8
{
	/** Drive using rotation */
	Rotation,

	/** Driver using translation */
	Translation
};

/** Options for what PoseDriver should be driving */
UENUM(BlueprintType)
enum class EPoseDriverOutput : uint8
{
	/** Use target's DrivenName to drive poses from the assigned PoseAsset */
	DrivePoses,

	/** Use the target's DrivenName to drive curves */
	DriveCurves
};

/** Translation and rotation for a particular bone at a particular target */
USTRUCT()
struct FPoseDriverTransform
{
	GENERATED_BODY()

	/** Translation of this target */
	UPROPERTY(EditAnywhere, Category = PoseDriver)
	FVector TargetTranslation;

	/** Rotation of this target */
	UPROPERTY(EditAnywhere, Category = PoseDriver)
	FRotator TargetRotation;

	FPoseDriverTransform()
	: TargetTranslation(FVector::ZeroVector)
	, TargetRotation(FRotator::ZeroRotator)
	{}
};

/** Information about each target in the PoseDriver */
USTRUCT()
struct FPoseDriverTarget
{
	GENERATED_BODY()
		
	/** Translation of this target */
	UPROPERTY(EditAnywhere, Category = PoseDriver)
	TArray<FPoseDriverTransform> BoneTransforms;

	/** Rotation of this target */
	UPROPERTY(EditAnywhere, Category = PoseDriver)
	FRotator TargetRotation;

	/** Scale applied to this target's function - a larger value will activate this target sooner */
	UPROPERTY(EditAnywhere, Category = PoseDriver)
	float TargetScale;

	/** Override for the distance method to use for each target */
	UPROPERTY(EditAnywhere, Category = RBFData)
	ERBFDistanceMethod DistanceMethod;

	/** Override for the function method to use for each target */
	UPROPERTY(EditAnywhere, Category = RBFData)
	ERBFFunctionType FunctionType;

	/** If we should apply a custom curve mapping to how this target activates */
	UPROPERTY(EditAnywhere, Category = PoseDriver)
	bool bApplyCustomCurve;

	/** Custom curve mapping to apply if bApplyCustomCurve is true */
	UPROPERTY(EditAnywhere, Category = PoseDriver)
	FRichCurve CustomCurve;

	/** 
	 *	Name of item to drive - depends on DriveOutput setting.  
	 *	If DriveOutput is DrivePoses, this should be the name of a pose in the assigned PoseAsset
	 *	If DriveOutput is DriveCurves, this is the name of the curve (morph target, material param etc) to drive
	 */
	UPROPERTY(EditAnywhere, Category = PoseDriver)
	FName DrivenName;

	/** Pose buffer index */
	int32 PoseCurveIndex;

	/** If we should hide this pose from the UI */
	UPROPERTY(EditAnywhere, Category = PoseDriver)
	bool bIsHidden;

	// removing deprecation for default copy operator/constructor to avoid deprecation warnings
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPoseDriverTarget(const FPoseDriverTarget&) = default;
	FPoseDriverTarget& operator=(const FPoseDriverTarget&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FPoseDriverTarget()
		: TargetRotation(ForceInitToZero)
		, TargetScale(1.f)
		, DistanceMethod(ERBFDistanceMethod::DefaultMethod)
		, FunctionType(ERBFFunctionType::DefaultFunction)
		, bApplyCustomCurve(false)
		, DrivenName(NAME_None)
		, PoseCurveIndex(INDEX_NONE)
		, bIsHidden(false)
	{}
};

/** RBF based orientation driver */
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_PoseDriver : public FAnimNode_PoseHandler
{
	GENERATED_BODY()

	/** Bones to use for driving parameters based on their transform */
	UPROPERTY(EditAnywhere, EditFixedSize, BlueprintReadWrite, Category = PoseDriver)
	FPoseLink SourcePose;

	/** Bone to use for driving parameters based on its orientation */
	UPROPERTY(EditAnywhere, Category = PoseDriver)
	TArray<FBoneReference> SourceBones;

	/**
	 *	Optional other bone space to use when reading SourceBone transform.
	 *	If not specified, the local space of SourceBone will be used. (ie relative to parent bone)
	 */
	UPROPERTY(EditAnywhere, Category = PoseDriver)
	FBoneReference EvalSpaceBone;

	/**
	 *	Evaluate SourceBone transform relative from its Reference Pose.
	 *  This is recommended when using Swing and Twist Angle as Distance Method, since the twist will be computed from RefPose.
	 *
	 *	If not specified, the local space of SourceBone will be used. (ie relative to parent bone)
	 *  This mode won't work in conjunction with EvalSpaceBone;
	 */
	UPROPERTY(EditAnywhere, Category = PoseDriver)
	bool bEvalFromRefPose = false;

	/** List of bones that will modified by this node. If no list is provided, all bones bones with a track in the PoseAsset will be modified */
	UPROPERTY(EditAnywhere, Category = PoseDriver, meta = (EditCondition = "DriveOutput == EPoseDriverOutput::DrivePoses"))
	TArray<FBoneReference> OnlyDriveBones;

	/** Targets used to compare with current pose and drive morphs/poses */
	UPROPERTY(EditAnywhere, Category = PoseDriver)
	TArray<FPoseDriverTarget> PoseTargets;

	/** Parameters used by RBF solver */
	UPROPERTY(EditAnywhere, Category = PoseDriver, meta = (ShowOnlyInnerProperties))
	FRBFParams RBFParams;

	/** Which part of the transform is read */
	UPROPERTY(EditAnywhere, Category = PoseDriver)
	EPoseDriverSource DriveSource;

	/** Whether we should drive poses or curves */
	UPROPERTY(EditAnywhere, Category = PoseDriver)
	EPoseDriverOutput DriveOutput;

	/** Last set of output weights from RBF solve */
	TArray<FRBFOutputWeight> OutputWeights;

	/** Input source bone TM, used for debug drawing */
	TArray<FTransform> SourceBoneTMs;
	
	/** Checks if there are valid bones in OnlyDriveBones, since some entries can be None */
	bool bHasOnlyDriveBones = false;

	/** This array lists bones that we should filter out (ie have a track in the PoseAsset, but are not listed in OnlyDriveBones) */
	TArray<FCompactPoseBoneIndex> BonesToFilter;

#if WITH_EDITORONLY_DATA
	// Deprecated
	UPROPERTY()
	FBoneReference SourceBone_DEPRECATED;
	UPROPERTY()
	TEnumAsByte<EBoneAxis> TwistAxis_DEPRECATED;
	UPROPERTY()
	EPoseDriverType Type_DEPRECATED;
	UPROPERTY()
	float RadialScaling_DEPRECATED;
	//
#endif

	/** If true, will recalculate DrivenUID values in PoseTargets array on next eval */
	uint8 bCachedDrivenIDsAreDirty : 1;

#if WITH_EDITORONLY_DATA
	/** The target to solo on, or INDEX_NONE if to use the normal weight computation. 
	    Not a UPROPERTY to ensure it stays transient and doesn't affect PIE. */
	int32 SoloTargetIndex;

	/** Only solo the driven poses, and don't move the source joint(s) to match */
	UPROPERTY()
	bool bSoloDrivenOnly;
#endif

	/*
	 * Max LOD that this node is allowed to run
	 * For example if you have LODThreshold to be 2, it will run until LOD 2 (based on 0 index)
	 * when the component LOD becomes 3, it will stop update/evaluate
	 * currently transition would be issue and that has to be re-visited
	 */
	UPROPERTY(EditAnywhere, Category = Performance, meta = (PinHiddenByDefault, DisplayName = "LOD Threshold"))
	int32 LODThreshold;

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void UpdateAssetPlayer(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	virtual int32 GetLODThreshold() const override { return LODThreshold; }
	// End of FAnimNode_Base interface

	ANIMGRAPHRUNTIME_API FAnimNode_PoseDriver();

	/** Returns the radius for a given target */
	ANIMGRAPHRUNTIME_API float GetRadiusForTarget(const FRBFTarget& Target) const;

	/** Util for seeing if BoneName is in the list of driven bones (and bFilterDrivenBones is true) */
	ANIMGRAPHRUNTIME_API bool IsBoneDriven(FName BoneName) const;

	/** Return array of FRBFTarget structs, derived from PoseTargets array and DriveSource setting */
	ANIMGRAPHRUNTIME_API void GetRBFTargets(TArray<FRBFTarget>& OutTargets, const FBoneContainer* BoneContainer) const;

	/* Rebuild Pose List*/
	ANIMGRAPHRUNTIME_API virtual void RebuildPoseList(const FBoneContainer& InBoneContainer, const UPoseAsset* InPoseAsset) override;

private:
	TSharedPtr<const FRBFSolverData> SolverData;
	FRBFEntry RBFInput;
	TArray<FRBFTarget> RBFTargets;
};
