// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimationAsset.h"
#include "Animation/PoseAsset.h"
#include "Animation/AnimNode_AssetPlayerBase.h"
#include "AnimNode_PoseHandler.generated.h"

// Evaluates a point in an anim sequence, using a specific time input rather than advancing time internally.
// Typically the playback position of the animation for this node will represent something other than time, like jump height.
// This node will not trigger any notifies present in the associated sequence.
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_PoseHandler : public FAnimNode_AssetPlayerBase
{
	GENERATED_USTRUCT_BODY()
public:
	// The animation sequence asset to evaluate
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinHiddenByDefault))
	TObjectPtr<UPoseAsset> PoseAsset;

public:	
	FAnimNode_PoseHandler()
		:PoseAsset(nullptr)
	{
	}

	// FAnimNode_AssetPlayerBase interface
	virtual float GetCurrentAssetTime() const { return 0.f; }
	virtual float GetCurrentAssetLength() const { return 0.f; }
	// End of FAnimNode_AssetPlayerBase interface

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void UpdateAssetPlayer(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// FAnimNode_AssetPlayerBase Interface
	virtual float GetAccumulatedTime() const {return 0.f;}
	virtual void SetAccumulatedTime(float NewTime) {}
	virtual UAnimationAsset* GetAnimAsset() const {return PoseAsset;}
	// End of FAnimNode_AssetPlayerBase Interface

#if WITH_EDITORONLY_DATA
	// Set the pose asset to use for this node 
	ANIMGRAPHRUNTIME_API void SetPoseAsset(UPoseAsset* InPoseAsset);
#endif
	
protected:
	/** Called after CurrentPoseAsset is changed.  */
	virtual void OnPoseAssetChange() {}

	TWeakObjectPtr<UPoseAsset> CurrentPoseAsset;
	FAnimExtractContext PoseExtractContext;
	// weight to blend pose per joint - has to be cached whenever cache bones for LOD
	// note this is for mesh bone
	TArray<float> BoneBlendWeights;

	/* Rebuild pose list */
	ANIMGRAPHRUNTIME_API virtual void RebuildPoseList(const FBoneContainer& InBoneContainer, const UPoseAsset* InPoseAsset);

	/** Cache bone blend weights - called when pose asset changes */
	ANIMGRAPHRUNTIME_API void CacheBoneBlendWeights(FAnimInstanceProxy* InstanceProxy);
	
private:
	ANIMGRAPHRUNTIME_API void UpdatePoseAssetProperty(struct FAnimInstanceProxy* InstanceProxy);
};

