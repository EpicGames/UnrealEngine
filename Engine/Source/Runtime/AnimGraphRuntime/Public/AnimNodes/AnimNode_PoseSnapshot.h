// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/PoseSnapshot.h"
#include "AnimNode_PoseSnapshot.generated.h"

class UAnimInstance;

/** How to access the snapshot */
UENUM()
enum class ESnapshotSourceMode : uint8
{
	/** 
	 * Refer to an internal snapshot by name (previously stored with SavePoseSnapshot). 
	 * This can be more efficient than access via pin.
	 */
	NamedSnapshot,

	/** 
	 * Use a snapshot variable (previously populated using SnapshotPose).
	 * This is more flexible and allows poses to be modified and managed externally to the animation blueprint.
	 */
	SnapshotPin
};

/** Provide a snapshot pose, either from the internal named pose cache or via a supplied snapshot */
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_PoseSnapshot : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()

public:	

	ANIMGRAPHRUNTIME_API FAnimNode_PoseSnapshot();

	/** FAnimNode_Base interface */
	virtual bool HasPreUpdate() const override  { return true; }
	ANIMGRAPHRUNTIME_API virtual void PreUpdate(const UAnimInstance* InAnimInstance) override;
	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;

	/** The name of the snapshot previously stored with SavePoseSnapshot */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Snapshot", meta = (PinShownByDefault))
	FName SnapshotName;

	/** Snapshot to use. This should be populated at first by calling SnapshotPose */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Snapshot", meta = (PinHiddenByDefault))
	FPoseSnapshot Snapshot;

	/** How to access the snapshot */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Snapshot", meta = (PinHiddenByDefault))
	ESnapshotSourceMode Mode;

private:
	/** Cache of target space bases to source space bases */
	TArray<int32> SourceBoneMapping;

	/** Cached array of bone names for target's ref skeleton */
	TArray<FName> TargetBoneNames;

	/** Cached skeletal meshes we use to invalidate the bone mapping */
	FName MappedSourceMeshName;
	FName MappedTargetMeshName;

	/** Cached skeletal mesh we used for updating the target bone name array */
	FName TargetBoneNameMesh;

private:
	/** Evaluation helper function - apply a snapshot pose to a pose */
	ANIMGRAPHRUNTIME_API void ApplyPose(const FPoseSnapshot& PoseSnapshot, FCompactPose& OutPose);

	/** Evaluation helper function - cache the bone mapping between two skeletal meshes */
	ANIMGRAPHRUNTIME_API void CacheBoneMapping(FName SourceMeshName, FName TargetMeshName, const TArray<FName>& InSourceBoneNames, const TArray<FName>& InTargetBoneNames);
};
