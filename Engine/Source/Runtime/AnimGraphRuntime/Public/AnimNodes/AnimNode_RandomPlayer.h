// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AlphaBlend.h"
#include "Animation/AnimNode_RelevantAssetPlayerBase.h"
#include "Animation/AnimationAsset.h"
#include "CoreMinimal.h"
#include "Math/RandomStream.h"
#include "UObject/ObjectMacros.h"

#include "AnimNode_RandomPlayer.generated.h"

enum class ERandomDataIndexType
{
	Current,
	Next,
};

/** The random player node holds a list of sequences and parameter ranges which will be played continuously
  * In a random order. If shuffle mode is enabled then each entry will be played once before repeating any
  */
USTRUCT(BlueprintInternalUseOnly)
struct FRandomPlayerSequenceEntry
{
	GENERATED_BODY()

	FRandomPlayerSequenceEntry()
	    : Sequence(nullptr)
	    , ChanceToPlay(1.0f)
	    , MinLoopCount(0)
	    , MaxLoopCount(0)
	    , MinPlayRate(1.0f)
	    , MaxPlayRate(1.0f)
	{
	}

	/** Sequence to play when this entry is picked */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (DisallowedClasses = "/Script/Engine.AnimMontage"))
	TObjectPtr<UAnimSequenceBase> Sequence;

	/** When not in shuffle mode, this is the chance this entry will play (normalized against all other sample chances) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (UIMin = "0", ClampMin = "0"))
	float ChanceToPlay;

	/** Minimum number of times this entry will loop before ending */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (UIMin = "0", ClampMin = "0"))
	int32 MinLoopCount;

	/** Maximum number of times this entry will loop before ending */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (UIMin = "0", ClampMin = "0"))
	int32 MaxLoopCount;

	/** Minimum playrate for this entry */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (UIMin = "0", ClampMin = "0"))
	float MinPlayRate;

	/** Maximum playrate for this entry */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (UIMin = "0", ClampMin = "0"))
	float MaxPlayRate;

	/** Blending properties used when this entry is blending in ontop of another entry */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FAlphaBlend BlendIn;
};

struct FRandomAnimPlayData
{
	// Index into the real sequence entry list, not the valid entry list.
	FRandomPlayerSequenceEntry* Entry = nullptr;

	// The time at which the animation started playing. Used to initialize
	// the play for this animation and detect when a loop has occurred.
	float PlayStartTime = 0.0f;

	// The time at which the animation is currently playing.
	float CurrentPlayTime = 0.0f;

	// Delta time record for this play through
	FDeltaTimeRecord DeltaTimeRecord;

	// Calculated play rate
	float PlayRate = 0.0f;

	// Current blend weight
	float BlendWeight = 0.0f;

	// Calculated loops remaining
	int32 RemainingLoops = 0;

	// Marker tick record for this play through
	FMarkerTickRecord MarkerTickRecord;
};

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_RandomPlayer : public FAnimNode_AssetPlayerRelevancyBase
{
	GENERATED_BODY()

	ANIMGRAPHRUNTIME_API FAnimNode_RandomPlayer();

public:
	/** List of sequences to randomly step through */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	TArray<FRandomPlayerSequenceEntry> Entries;

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// FAnimNode_RelevantAssetPlayerBase
	ANIMGRAPHRUNTIME_API virtual UAnimationAsset* GetAnimAsset() const override;
	ANIMGRAPHRUNTIME_API virtual float GetAccumulatedTime() const override;
	ANIMGRAPHRUNTIME_API virtual bool GetIgnoreForRelevancyTest() const override;
	ANIMGRAPHRUNTIME_API virtual bool SetIgnoreForRelevancyTest(bool bInIgnoreForRelevancyTest) override;
	ANIMGRAPHRUNTIME_API virtual float GetCachedBlendWeight() const override;
	ANIMGRAPHRUNTIME_API virtual void ClearCachedBlendWeight() override;
	ANIMGRAPHRUNTIME_API virtual const FDeltaTimeRecord* GetDeltaTimeRecord() const override;
	// End of FAnimNode_RelevantAssetPlayerBase

private:
	// Return the index of the next FRandomPlayerSequenceEntry to play, from the list
	// of valid playable entries (ValidEntries).
	int32 GetNextValidEntryIndex();

	// Return the play data for either the currently playing animation or the next
	// animation to blend into.
	FRandomAnimPlayData& GetPlayData(ERandomDataIndexType Type);
	const FRandomAnimPlayData& GetPlayData(ERandomDataIndexType Type) const;

	// Initialize the play data with the given index into the ValidEntries array and
	// a specific blend weight. All other member data will be reset to their default values.
	void InitPlayData(FRandomAnimPlayData& Data, int32 InValidEntryIndex, float InBlendWeight);

	// Advance to the next playable sequence. This is only called once a sequence is fully
	// blended or there's a hard switch to the same playable entry.
	void AdvanceToNextSequence();

	// Build a new ShuffleList array, which is a shuffled index list of all the valid
	// playable entries in ValidEntries. The LastEntry can be set to a valid entry index to
	// ensure that the top/last item in the shuffle list will be a different value from it;
	// pass in INDEX_NONE to disable the check.
	void BuildShuffleList(int32 LastEntry);

	// List of valid sequence entries
	TArray<FRandomPlayerSequenceEntry*> ValidEntries;

	// Normalized list of play chances when we aren't using shuffle mode
	TArray<float> NormalizedPlayChances;

	// Play data for the current and next sequence
	TArray<FRandomAnimPlayData> PlayData;

	// Index of the 'current' data set in the PlayData array.
	int32 CurrentPlayDataIndex;

	// List to store transient shuffle stack in shuffle mode.
	TArray<int32> ShuffleList;

	// Random number source
	FRandomStream RandomStream;

#if WITH_EDITORONLY_DATA
	// If true, "Relevant anim" nodes that look for the highest weighted animation in a state will ignore this node
	UPROPERTY(EditAnywhere, Category = Relevancy, meta = (FoldProperty, PinHiddenByDefault))
	bool bIgnoreForRelevancyTest = false;
#endif // WITH_EDITORONLY_DATA

protected:
	/** Last encountered blend weight for this node */
	UPROPERTY(BlueprintReadWrite, Transient, Category = DoNotEdit)
	float BlendWeight = 0.0f;

public:
	/** When shuffle mode is active we will never loop a sequence beyond MaxLoopCount
	  * without visiting each sequence in turn (no repeats). Enabling this will ignore
	  * ChanceToPlay for each entry
	  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bShuffleMode;
};
