// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNodeBase.h"
#include "AnimNode_RefPose.generated.h"

UENUM()
enum ERefPoseType : int
{
	EIT_LocalSpace, 
	EIT_Additive
};

// RefPose pose nodes - ref pose or additive RefPose pose
USTRUCT()
struct FAnimNode_RefPose : public FAnimNode_Base
{
	GENERATED_BODY()

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY(meta=(FoldProperty))
	TEnumAsByte<ERefPoseType> RefPoseType = EIT_LocalSpace;
#endif	// #if WITH_EDITORONLY_DATA
	
public:	
	FAnimNode_RefPose() = default;

#if WITH_EDITORONLY_DATA
	// Set the ref pose type of this node
	void SetRefPoseType(ERefPoseType InType) { RefPoseType = InType; }
#endif

	// Get the type of this ref pose
	ANIMGRAPHRUNTIME_API ERefPoseType GetRefPoseType() const;
	
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
};

USTRUCT()
struct FAnimNode_MeshSpaceRefPose : public FAnimNode_Base
{
	GENERATED_BODY()
public:	
	FAnimNode_MeshSpaceRefPose() = default;

	ANIMGRAPHRUNTIME_API virtual void EvaluateComponentSpace_AnyThread(FComponentSpacePoseContext& Output);
};
