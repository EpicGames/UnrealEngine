// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNodeBase.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/NamedValueArray.h"
#include "CoreMinimal.h"
#include "AnimNode_RemoveCurve.generated.h"

UENUM()
enum class ERemoveCurveMode : uint8
{
	/** Removes animation curves which names are specified in the Curves variable from the Pose Input. */
	RemoveSpecifiedCurves,

	/** Removes ALL animation curves in the Pose Input, EXCEPT those specified in the Curves variable */
	KeepOnlySpecifiedCurves
};


//** Removes animation curves on a pose */
USTRUCT(BlueprintInternalUseOnly, Experimental)
struct FAnimNode_RemoveCurve : public FAnimNode_Base
{
	GENERATED_BODY()

public:
	// Input link (the upstream pose)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Links")
	FPoseLink SourcePose;

	// Is Enabled
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Alpha", meta = (PinShownByDefault))
	bool bEnabled;

	/** Node mode selection.
	RemoveSpecifiedCurves -  will remove animation curves which names are specified in the Curves variable from the Pose Input
	KeepOnlySpecifiedCurves - will remove ALL animation curves in the Pose Input, EXCEPT those specified in the Curves variable	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Remove Curves", meta = (BlueprintCompilerGeneratedDefaults, PinHiddenByDefault))
	ERemoveCurveMode RemoveMode;

	/** List of animation curves.*/	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Remove Curves", meta = (BlueprintCompilerGeneratedDefaults, PinHiddenByDefault))
	TSet<FName> Curves;

	/** Affect curves of a Morph Target type when True.
	Ignores them if False. I.e. curves of Morph Target type will be left completely unaffected by this node*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Remove Curves Filter by Type", meta = (NeverAsPin))
	bool bAffectMorphTargetCurves;

	/** Affect curves of a Material type. 
	Ignores them if False. I.e. curves of Material type will be left completely unaffected by this node */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Remove Curves Filter by Type", meta = (NeverAsPin))
	bool bAffectMaterialCurves;

	/** Affect normal animation curves (not Material or Morph Target type). 
	Ignores them if False. I.e. regualar animation curves will be left completely unaffected by this node */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Remove Curves Filter by Type", meta = (NeverAsPin))
	bool bAffectRegularCurves;

	// Internal vars.
	UPROPERTY(transient)
	TArray<FName> MorphTargetCurveNames;

	UPROPERTY(transient)
	TArray<FName> MaterialCurveNames;

	ANIMGRAPHRUNTIME_API FAnimNode_RemoveCurve();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;

	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

private:
	// Final list of curves to remove. Global, so it can be used in debug printouts anywhere.
	UE::Anim::TNamedValueArray<FDefaultAllocator, UE::Anim::FCurveElement> CurvesForRemoval;
};
