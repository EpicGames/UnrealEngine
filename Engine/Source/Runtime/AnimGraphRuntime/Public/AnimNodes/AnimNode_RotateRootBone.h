// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/InputScaleBias.h"
#include "AnimNode_RotateRootBone.generated.h"

//@TODO: Comment
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_RotateRootBone : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Links)
	FPoseLink BasePose;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings, meta=(PinShownByDefault))
	float Pitch;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings, meta=(PinShownByDefault))
	float Yaw;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	FInputScaleBiasClamp PitchScaleBiasClamp;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	FInputScaleBiasClamp YawScaleBiasClamp;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings, meta=(PinShownByDefault))
	FRotator MeshToComponent;

	// If enabled, rotating the root bone using this node will also rotate the direction of the root motion custom attribute
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Advanced)
	bool bRotateRootMotionAttribute;

	float ActualPitch;

	float ActualYaw;

public:	
	ANIMGRAPHRUNTIME_API FAnimNode_RotateRootBone();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface
};
