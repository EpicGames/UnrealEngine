// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNode_BlendSpaceGraphBase.h"
#include "Animation/InputScaleBias.h"
#include "AnimNode_RotationOffsetBlendSpaceGraph.generated.h"

// Allows multiple animations to be blended between based on input parameters
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_RotationOffsetBlendSpaceGraph : public FAnimNode_BlendSpaceGraphBase
{
	GENERATED_BODY()

	friend class UAnimGraphNode_RotationOffsetBlendSpaceGraph;
	friend struct FAnimNodeAlphaOptions;
	friend struct FAnimGraphNodeAlphaOptions;

	// @return the sync group that this blendspace uses
	FName GetGroupName() const { return GroupName; }

private:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Links, meta = (AllowPrivateAccess))
	FPoseLink BasePose;

	// Apply the rotation additive in root space, effectively ignoring any root bone rotation.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinHiddenByDefault, AllowPrivateAccess))
	bool bApplyAdditiveInRootSpace = false;

	/*
	* Max LOD that this node is allowed to run
	* For example if you have LODThreshold to be 2, it will run until LOD 2 (based on 0 index)
	* when the component LOD becomes 3, it will stop update/evaluate
	* currently transition would be issue and that has to be re-visited
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Performance, meta = (DisplayName = "LOD Threshold", AllowPrivateAccess))
	int32 LODThreshold = INDEX_NONE;

	// Current strength of the AimOffset
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (PinShownByDefault, AllowPrivateAccess))
	float Alpha = 1.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (AllowPrivateAccess))
	FInputScaleBias AlphaScaleBias;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (DisplayName = "Blend Settings", AllowPrivateAccess))
	FInputAlphaBoolBlend AlphaBoolBlend;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (PinShownByDefault, AllowPrivateAccess))
	FName AlphaCurveName = NAME_None;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (AllowPrivateAccess))
	FInputScaleBiasClamp AlphaScaleBiasClamp;

	float ActualAlpha = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (AllowPrivateAccess))
	EAnimAlphaInputType AlphaInputType = EAnimAlphaInputType::Float;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (PinShownByDefault, DisplayName = "bEnabled", DisplayAfter = "AlphaScaleBias", AllowPrivateAccess))
	bool bAlphaBoolEnabled = false;

	bool bIsLODEnabled = false;

private:
	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	virtual int32 GetLODThreshold() const override { return LODThreshold; }
	// End of FAnimNode_Base interface
};
