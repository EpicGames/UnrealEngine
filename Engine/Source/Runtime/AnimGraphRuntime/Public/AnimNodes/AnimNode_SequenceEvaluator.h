// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNode_AssetPlayerBase.h"
#include "Animation/AnimSequenceBase.h"
#include "AnimNode_SequenceEvaluator.generated.h"

UENUM(BlueprintType)
namespace ESequenceEvalReinit
{
	enum Type : int
	{
		/** Do not reset InternalTimeAccumulator */
		NoReset,
		/** Reset InternalTimeAccumulator to StartPosition */
		StartPosition,
		/** Reset InternalTimeAccumulator to ExplicitTime */
		ExplicitTime,
	};
}

// Abstract base class. Evaluates a point in an anim sequence, using a specific time input rather than advancing time internally.
// Typically the playback position of the animation for this node will represent something other than time, like jump height.
// This node will not trigger any notifies present in the associated sequence.
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_SequenceEvaluatorBase : public FAnimNode_AssetPlayerBase
{
	GENERATED_BODY()

private:
	bool bReinitialized = false;

public:
	// FAnimNode_AssetPlayerBase interface
	ANIMGRAPHRUNTIME_API virtual float GetCurrentAssetTime() const override;
	ANIMGRAPHRUNTIME_API virtual float GetCurrentAssetLength() const override;
	// End of FAnimNode_AssetPlayerBase interface

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void UpdateAssetPlayer(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// FAnimNode_AssetPlayerBase Interface
	virtual float GetAccumulatedTime() const override {return GetExplicitTime();}
	virtual void SetAccumulatedTime(float NewTime) override { SetExplicitTime(NewTime); }
	virtual UAnimationAsset* GetAnimAsset() const override { return GetSequence(); }
	// End of FAnimNode_AssetPlayerBase Interface

	void SetExplicitPreviousTime(float PreviousTime) { InternalTimeAccumulator = PreviousTime; }

	// Get the effective delta time between the previous and current frame internal time
	ANIMGRAPHRUNTIME_API virtual float GetEffectiveDeltaTime(float ExplicitTime, float PrevExplicitTime) const;

	// Set the animation sequence asset to evaluate
	virtual bool SetSequence(UAnimSequenceBase* InSequence) { return false; }

	// Set the time at which to evaluate the associated sequence
	virtual bool SetExplicitTime(float InTime) { return false; }

	// Set whether to teleport to explicit time when it is set
	virtual void SetTeleportToExplicitTime(bool bInTeleport) {}

	/** Set what to do when SequenceEvaluator is reinitialized */
	virtual void SetReinitializationBehavior(TEnumAsByte<ESequenceEvalReinit::Type> InBehavior) {}

	// The animation sequence asset to evaluate
	virtual UAnimSequenceBase* GetSequence() const { return nullptr; }

	// The time at which to evaluate the associated sequence
	virtual float GetExplicitTime() const { return 0.0f; }

	// Set the animation to continue looping when it reaches the end
	virtual bool SetShouldLoop(bool bInShouldLoop) { return false; }

	/** If true, teleport to explicit time, does NOT advance time (does not trigger notifies, does not extract Root Motion, etc.)
	If false, will advance time (will trigger notifies, extract root motion if applicable, etc.)
	Note: using a sync group forces advancing time regardless of what this option is set to. */
	virtual bool GetTeleportToExplicitTime() const { return true; }

	/** What to do when SequenceEvaluator is reinitialized */
	virtual TEnumAsByte<ESequenceEvalReinit::Type> GetReinitializationBehavior() const { return ESequenceEvalReinit::ExplicitTime; }

	// The start up position, it only applies when ReinitializationBehavior == StartPosition. Only used when bTeleportToExplicitTime is false.
	virtual float GetStartPosition() const { return 0.0f; }

	// Called after executing the input pins, split for subclass convenience.
	ANIMGRAPHRUNTIME_API void PostExposedInputsUpdateAssetPlayer(const FAnimationUpdateContext& Context);

};

// Sequence evaluator node that can be used with constant folding
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_SequenceEvaluator : public FAnimNode_SequenceEvaluatorBase
{
	GENERATED_BODY()

private:
	friend class UAnimGraphNode_SequenceEvaluator;

#if WITH_EDITORONLY_DATA
	// The group name that we synchronize with (NAME_None if it is not part of any group). 
	UPROPERTY(EditAnywhere, Category=Sync, meta=(FoldProperty))
	FName GroupName = NAME_None;

	// The role this node can assume within the group (ignored if GroupName is not set)
	UPROPERTY(EditAnywhere, Category=Sync, meta=(FoldProperty))
	TEnumAsByte<EAnimGroupRole::Type> GroupRole = EAnimGroupRole::CanBeLeader;

	// How this node will synchronize with other animations.
	UPROPERTY(EditAnywhere, Category=Sync, meta=(FoldProperty))
	EAnimSyncMethod Method = EAnimSyncMethod::DoNotSync;

	// If true, "Relevant anim" nodes that look for the highest weighted animation in a state will ignore this node
	UPROPERTY(EditAnywhere, Category=Relevancy, meta=(FoldProperty, PinHiddenByDefault))
	bool bIgnoreForRelevancyTest = false;
	
	// The animation sequence asset to evaluate
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, FoldProperty))
	TObjectPtr<UAnimSequenceBase> Sequence = nullptr;

	// The time at which to evaluate the associated sequence
	UPROPERTY(EditAnywhere, Category=Settings, meta=(PinShownByDefault, FoldProperty, EditCondition="!bUseExplicitFrame"))
	float ExplicitTime = 0.0f;

	// Whether to use ExplicitFrame (true) or ExplicitTime (false) when evaluating the associated sequence
	UPROPERTY(EditAnywhere, Category=Settings, meta=(NeverAsPin, FoldProperty))
	bool bUseExplicitFrame = false;

	// The frame at which to evaluate the associated sequence
	UPROPERTY(EditAnywhere, Category=Settings, meta=(PinHiddenByDefault, FoldProperty, EditCondition="bUseExplicitFrame"))
	int32 ExplicitFrame = 0;

	/** This only works if bTeleportToExplicitTime is false OR this node is set to use SyncGroup */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, FoldProperty))
	bool bShouldLoop = true;

	/** If true, teleport to explicit time, does NOT advance time (does not trigger notifies, does not extract Root Motion, etc.)
	If false, will advance time (will trigger notifies, extract root motion if applicable, etc.)
	Note: using a sync group forces advancing time regardless of what this option is set to. */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, FoldProperty))
	bool bTeleportToExplicitTime = true;

	/** What to do when SequenceEvaluator is reinitialized */
	UPROPERTY(EditAnywhere, Category = Settings, meta=(DisplayAfter="StartPosition", FoldProperty))
	TEnumAsByte<ESequenceEvalReinit::Type> ReinitializationBehavior = ESequenceEvalReinit::ExplicitTime;

	// The start up position, it only applies when ReinitializationBehavior == StartPosition. Only used when bTeleportToExplicitTime is false.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, FoldProperty))
	float StartPosition = 0.0f;
#endif

public:
	// FAnimNode_SequenceEvaluatorBase interface
	ANIMGRAPHRUNTIME_API virtual bool SetSequence(UAnimSequenceBase* InSequence) override;
	ANIMGRAPHRUNTIME_API virtual UAnimSequenceBase* GetSequence() const override;
	ANIMGRAPHRUNTIME_API virtual float GetExplicitTime() const override;
	ANIMGRAPHRUNTIME_API virtual bool SetExplicitTime(float InTime) override;
	ANIMGRAPHRUNTIME_API virtual bool SetShouldLoop(bool bInShouldLoop) override;
	ANIMGRAPHRUNTIME_API virtual bool GetTeleportToExplicitTime() const override;
	ANIMGRAPHRUNTIME_API virtual TEnumAsByte<ESequenceEvalReinit::Type> GetReinitializationBehavior() const override;
	ANIMGRAPHRUNTIME_API virtual float GetStartPosition() const override;

	// FAnimNode_AssetPlayerBase interface
	ANIMGRAPHRUNTIME_API virtual FName GetGroupName() const override;
	ANIMGRAPHRUNTIME_API virtual EAnimGroupRole::Type GetGroupRole() const override;
	ANIMGRAPHRUNTIME_API virtual EAnimSyncMethod GetGroupMethod() const override;
	ANIMGRAPHRUNTIME_API virtual bool GetIgnoreForRelevancyTest() const override;
	ANIMGRAPHRUNTIME_API virtual bool IsLooping() const override;
	ANIMGRAPHRUNTIME_API virtual bool SetGroupName(FName InGroupName) override;
	ANIMGRAPHRUNTIME_API virtual bool SetGroupRole(EAnimGroupRole::Type InRole) override;
	ANIMGRAPHRUNTIME_API virtual bool SetGroupMethod(EAnimSyncMethod InMethod) override;
	ANIMGRAPHRUNTIME_API virtual bool SetIgnoreForRelevancyTest(bool bInIgnoreForRelevancyTest) override;
	
	ANIMGRAPHRUNTIME_API bool ShouldUseExplicitFrame() const;
	ANIMGRAPHRUNTIME_API bool SetShouldUseExplicitFrame(bool bFlag);
	ANIMGRAPHRUNTIME_API bool SetExplicitFrame(int32 InFrame);
	ANIMGRAPHRUNTIME_API int32 GetExplicitFrame() const;
};

// Sequence evaluator node that can be used standalone (without constant folding)
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_SequenceEvaluator_Standalone : public FAnimNode_SequenceEvaluatorBase
{
	GENERATED_BODY()

private:
	// The group name that we synchronize with (NAME_None if it is not part of any group). 
	UPROPERTY(EditAnywhere, Category=Sync)
	FName GroupName = NAME_None;

	// The role this node can assume within the group (ignored if GroupName is not set)
	UPROPERTY(EditAnywhere, Category=Sync)
	TEnumAsByte<EAnimGroupRole::Type> GroupRole = EAnimGroupRole::CanBeLeader;

	// How this node will synchronize with other animations.
	UPROPERTY(EditAnywhere, Category=Sync)
	EAnimSyncMethod Method = EAnimSyncMethod::DoNotSync;

	// If true, "Relevant anim" nodes that look for the highest weighted animation in a state will ignore this node
	UPROPERTY(EditAnywhere, Category=Relevancy, meta=(PinHiddenByDefault))
	bool bIgnoreForRelevancyTest = false;
	
	// The animation sequence asset to evaluate
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	TObjectPtr<UAnimSequenceBase> Sequence = nullptr;

	// The time at which to evaluate the associated sequence
	UPROPERTY(EditAnywhere, Category=Settings, meta=(PinShownByDefault, EditCondition="!bUseFrame"))
	float ExplicitTime = 0.0f;

	UPROPERTY(EditAnywhere, Category=Settings, meta=(NeverAsPin))
	bool bUseExplicitFrame = false;
	
	UPROPERTY(EditAnywhere, Category=Settings, meta=(PinHiddenByDefault, EditCondition="bUseFrame"))
	int32 ExplicitFrame = 0;

	/** This only works if bTeleportToExplicitTime is false OR this node is set to use SyncGroup */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	bool bShouldLoop = true;

	/** If true, teleport to explicit time, does NOT advance time (does not trigger notifies, does not extract Root Motion, etc.)
	If false, will advance time (will trigger notifies, extract root motion if applicable, etc.)
	Note: using a sync group forces advancing time regardless of what this option is set to. */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	bool bTeleportToExplicitTime = true;

	/** What to do when SequenceEvaluator is reinitialized */
	UPROPERTY(EditAnywhere, Category = Settings, meta=(DisplayAfter="StartPosition"))
	TEnumAsByte<ESequenceEvalReinit::Type> ReinitializationBehavior = ESequenceEvalReinit::ExplicitTime;

	// The start up position, it only applies when ReinitializationBehavior == StartPosition. Only used when bTeleportToExplicitTime is false.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	float StartPosition = 0.0f;

public:
	// FAnimNode_SequenceEvaluatorBase interface
	virtual bool SetSequence(UAnimSequenceBase* InSequence) override { Sequence = InSequence; return true; }
	virtual bool SetExplicitTime(float InTime) override { ExplicitTime = InTime; return true; }
	virtual void SetTeleportToExplicitTime(bool bInTeleport) override { bTeleportToExplicitTime = bInTeleport; }
	virtual void SetReinitializationBehavior(TEnumAsByte<ESequenceEvalReinit::Type> InBehavior) override { ReinitializationBehavior = InBehavior; }
	virtual UAnimSequenceBase* GetSequence() const override { return Sequence; }
	virtual float GetExplicitTime() const override
	{
		if(ShouldUseExplicitFrame())
		{
			if(const UAnimSequenceBase* SequenceBase = GetSequence())
			{
				return SequenceBase->GetSamplingFrameRate().AsSeconds(ExplicitFrame);
			}

			return 0.f;
		}
		return ExplicitTime;
	}
	virtual bool GetTeleportToExplicitTime() const override { return bTeleportToExplicitTime; }
	virtual TEnumAsByte<ESequenceEvalReinit::Type> GetReinitializationBehavior() const override { return ReinitializationBehavior; }
	virtual float GetStartPosition() const override { return StartPosition; }

	// FAnimNode_AssetPlayerBase interface
	virtual FName GetGroupName() const override { return GroupName; }
	virtual EAnimGroupRole::Type GetGroupRole() const override { return GroupRole; }
	virtual EAnimSyncMethod GetGroupMethod() const override { return Method; }
	virtual bool GetIgnoreForRelevancyTest() const override { return bIgnoreForRelevancyTest; }
	virtual bool IsLooping() const override { return bShouldLoop; }
	virtual bool SetGroupName(FName InGroupName) override { GroupName = InGroupName; return true; }
	virtual bool SetGroupRole(EAnimGroupRole::Type InRole) override { GroupRole = InRole; return true; }
	virtual bool SetGroupMethod(EAnimSyncMethod InMethod) override { Method = InMethod; return true; }
	virtual bool SetIgnoreForRelevancyTest(bool bInIgnoreForRelevancyTest) override { bIgnoreForRelevancyTest = bInIgnoreForRelevancyTest; return true; }

	bool ShouldUseExplicitFrame() const { return bUseExplicitFrame; }
	void SetShouldUseExplicitFrame(bool bFlag) { bUseExplicitFrame = bFlag; }
	void SetExplicitFrame(int32 InFrame) { ExplicitFrame = InFrame; }
	int32 GetExplicitFrame() const { return ExplicitFrame; }
};
