// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimNodeBase.h"
#include "AnimNode_Slot.generated.h"

// An animation slot node normally acts as a passthru, but a montage or PlaySlotAnimation call from
// game code can cause an animation to blend in and be played on the slot temporarily, overriding the
// Source input.
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_Slot : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()

	// The source input, passed thru to the output unless a montage or slot animation is currently playing
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Links)
	FPoseLink Source;

	// The name of this slot, exposed to gameplay code, etc...
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings, meta=(CustomizeProperty))
	FName SlotName;

	//Whether we should continue to update the source pose regardless of whether it would be used.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bAlwaysUpdateSourcePose;

protected:
	virtual void PostEvaluateSourcePose(FPoseContext& SourceContext) {}

	FSlotNodeWeightInfo WeightData;
	FGraphTraversalCounter SlotNodeInitializationCounter;

public:	
	ANIMGRAPHRUNTIME_API FAnimNode_Slot();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface
};
