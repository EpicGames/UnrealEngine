// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Deprecated

UE_DEPRECATED_HEADER(5.3, "Including AnimGraphRuntime/AnimNode_StateResult.h is deprecated. The file location changed. Use AnimNode_StateResult.h from the Engine module (Engine/Animation/AnimNode_StateResult.h) instead of the AnimGraphRuntime module.")

#include "Animation/AnimNode_StateResult.h"