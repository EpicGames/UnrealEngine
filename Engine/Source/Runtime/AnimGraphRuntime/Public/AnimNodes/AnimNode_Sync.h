// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimSync.h"
#include "AnimNode_Sync.generated.h"

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_Sync : public FAnimNode_Base
{
	GENERATED_BODY()

	friend class UAnimGraphNode_Sync;

public:
	// Get the group name that we synchronize with
	FName GetGroupName() const { return GroupName; }

private:
	UPROPERTY(EditAnywhere, Category = Links)
	FPoseLink Source;

	// The group name that we synchronize with (NAME_None if it is not part of any group). 
	UPROPERTY(EditAnywhere, Category = Settings)
	FName GroupName;

	// The role this animation can assume within the group (ignored if GroupName is not set)
	UPROPERTY(EditAnywhere, Category = Settings)
	TEnumAsByte<EAnimGroupRole::Type> GroupRole = EAnimGroupRole::CanBeLeader;

private:
	// FAnimNode_Base
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
};
