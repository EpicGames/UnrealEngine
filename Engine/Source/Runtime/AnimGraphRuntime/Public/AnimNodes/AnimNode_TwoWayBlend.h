// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/InputScaleBias.h"
#include "AnimNode_TwoWayBlend.generated.h"

// This represents a baked transition
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_TwoWayBlend : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()
public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Links)
	FPoseLink A;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Links)
	FPoseLink B;

	/** The data type used to control the alpha blending between the A and B poses. 
		Note: Changing this value will disconnect alpha input pins. 
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	EAnimAlphaInputType AlphaInputType;

	/** The boolean value that controls the alpha blending when the alpha input type is set to 'Bool' */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinShownByDefault, DisplayName = "bEnabled", DisplayAfter="AlphaScaleBias"))
	uint8 bAlphaBoolEnabled:1;

protected:
	uint8 bAIsRelevant:1;

	uint8 bBIsRelevant:1;

	/** This reinitializes child pose when re-activated. For example, when active child changes */
	UPROPERTY(EditAnywhere, Category = Option)
	uint8 bResetChildOnActivation:1;

	/** Always update children, regardless of whether or not that child has weight. */
	UPROPERTY(EditAnywhere, Category = Option, meta=(PinHiddenByDefault))
	uint8 bAlwaysUpdateChildren:1;

public:
	/** The float value that controls the alpha blending when the alpha input type is set to 'Float' */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings, meta=(PinShownByDefault))
	float Alpha;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings)
	FInputScaleBias AlphaScaleBias;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (DisplayName = "Blend Settings"))
	FInputAlphaBoolBlend AlphaBoolBlend;

	/** The animation curve that controls the alpha blending when the alpha input type is set to 'Curve' */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinShownByDefault))
	FName AlphaCurveName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	FInputScaleBiasClamp AlphaScaleBiasClamp;

protected:
	float InternalBlendAlpha;

public:
	FAnimNode_TwoWayBlend()
		: AlphaInputType(EAnimAlphaInputType::Float)
		, bAlphaBoolEnabled(true)
		, bAIsRelevant(false)
		, bBIsRelevant(false)
		, bResetChildOnActivation(false)
		, bAlwaysUpdateChildren(false)
		, Alpha(0.0f)
		, AlphaCurveName(NAME_None)
		, InternalBlendAlpha(0.0f)
	{
	}

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface
};

