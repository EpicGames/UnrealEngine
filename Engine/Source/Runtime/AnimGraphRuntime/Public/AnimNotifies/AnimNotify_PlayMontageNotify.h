// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "AnimNotify_PlayMontageNotify.generated.h"


//////////////////////////////////////////////////////////////////////////
// UAnimNotify_PlayMontageNotify
//////////////////////////////////////////////////////////////////////////

UCLASS(editinlinenew, const, hidecategories = Object, collapsecategories, meta = (DisplayName = "Montage Notify"), MinimalAPI)
class UAnimNotify_PlayMontageNotify : public UAnimNotify
{
	GENERATED_UCLASS_BODY()

public:

	ANIMGRAPHRUNTIME_API virtual void BranchingPointNotify(FBranchingPointNotifyPayload& BranchingPointPayload) override;
	
	ANIMGRAPHRUNTIME_API virtual FString GetNotifyName_Implementation() const override;

#if WITH_EDITOR
	ANIMGRAPHRUNTIME_API virtual bool CanBePlaced(UAnimSequenceBase* Animation) const override;
#endif
protected:

	// Name of notify that is passed to the PlayMontage K2Node.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Notify")
	FName NotifyName;
};


//////////////////////////////////////////////////////////////////////////
// UAnimNotify_PlayMontageNotifyWindow
//////////////////////////////////////////////////////////////////////////

UCLASS(editinlinenew, const, hidecategories = Object, collapsecategories, meta = (DisplayName = "Montage Notify Window"), MinimalAPI)
class UAnimNotify_PlayMontageNotifyWindow : public UAnimNotifyState
{
	GENERATED_UCLASS_BODY()

public:
	ANIMGRAPHRUNTIME_API virtual void BranchingPointNotifyBegin(FBranchingPointNotifyPayload& BranchingPointPayload) override;
	ANIMGRAPHRUNTIME_API virtual void BranchingPointNotifyEnd(FBranchingPointNotifyPayload& BranchingPointPayload) override;

	ANIMGRAPHRUNTIME_API virtual FString GetNotifyName_Implementation() const override;

#if WITH_EDITOR
	ANIMGRAPHRUNTIME_API virtual bool CanBePlaced(UAnimSequenceBase* Animation) const override;
#endif
protected:

	// Name of notify that is passed to ability.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Notify")
	FName NotifyName;
};
