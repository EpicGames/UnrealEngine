// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This Instance only contains one AnimationAsset, and produce poses
 * Used by Preview in AnimGraph, Playing single animation in Kismet2 and etc
 */

#pragma once
#include "Animation/AnimInstance.h"
#include "SequencerAnimationSupport.h"
#include "AnimSequencerInstance.generated.h"

struct FRootMotionOverride;
struct FAnimSequencerData;

UCLASS(transient, NotBlueprintable, MinimalAPI)
class UAnimSequencerInstance : public UAnimInstance, public ISequencerAnimationSupport
{
	GENERATED_UCLASS_BODY()

public:

	/** Update an animation sequence player in this instance */
	ANIMGRAPHRUNTIME_API virtual void UpdateAnimTrack(UAnimSequenceBase* InAnimSequence, int32 SequenceId, float InPosition, float Weight, bool bFireNotifies);
	ANIMGRAPHRUNTIME_API virtual void UpdateAnimTrack(UAnimSequenceBase* InAnimSequence, int32 SequenceId, float InFromPosition, float InToPosition, float Weight, bool bFireNotifies);

	ANIMGRAPHRUNTIME_API void UpdateAnimTrackWithRootMotion(const FAnimSequencerData& InAnimSequencerData);

	/** Construct all nodes in this instance */
	ANIMGRAPHRUNTIME_API virtual void ConstructNodes() override;

	/** Reset all nodes in this instance */
	ANIMGRAPHRUNTIME_API virtual void ResetNodes() override;

	/** Reset the pose for this instance*/
	ANIMGRAPHRUNTIME_API virtual void ResetPose() override;

	/** Saved the named pose to restore after */
	ANIMGRAPHRUNTIME_API virtual void SavePose() override;

	virtual UAnimInstance* GetSourceAnimInstance() override { return this; }
	virtual void SetSourceAnimInstance(UAnimInstance* SourceAnimInstance) {  /* nothing to do */ ensure(false); }
	virtual bool DoesSupportDifferentSourceAnimInstance() const override { return false; }

protected:
	// UAnimInstance interface
	ANIMGRAPHRUNTIME_API virtual FAnimInstanceProxy* CreateAnimInstanceProxy() override;



public:
	static ANIMGRAPHRUNTIME_API const FName SequencerPoseName;

};

