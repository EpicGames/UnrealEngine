// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimInstanceProxy.h"
#include "AnimNodes/AnimNode_SequenceEvaluator.h"
#include "AnimNodes/AnimNode_ApplyAdditive.h"
#include "AnimNodes/AnimNode_MultiWayBlend.h"
#include "AnimNodes/AnimNode_PoseSnapshot.h"
#include "AnimNodes/AnimNode_Mirror.h"
#include "AnimSequencerInstanceProxy.generated.h"

UENUM(BlueprintType)
enum class ESwapRootBone : uint8
{
	/* Swap the root bone to the component */
	SwapRootBone_Component,
	/* Swap root bone to the actor root component */
	SwapRootBone_Actor,
	/* Do not swap the root bone */
	SwapRootBone_None
};

/** Base class for all 'players' that can attach to and be blended into a sequencer instance's output */
struct FSequencerPlayerBase
{
public:
	FSequencerPlayerBase()
		: PoseIndex(INDEX_NONE)
		, bAdditive(false)
	{}

	template<class TType> 
	bool IsOfType() const 
	{
		return IsOfTypeImpl(TType::GetTypeId());
	}

	/** Virtual destructor. */
	virtual ~FSequencerPlayerBase() = default;

public:
	/** Index of this players pose in the set of blend slots */
	int32 PoseIndex;

	/** Whether this pose is additive or not */
	bool bAdditive;

protected:
	/**
	* Checks whether this drag and drop operation can cast safely to the specified type.
	*/
	virtual bool IsOfTypeImpl(const FName& Type) const
	{
		return false;
	}

};


/** Optional Override To Specify RootMotion*/
struct FRootMotionOverride
{
	FRootMotionOverride() :bBlendFirstChildOfRoot(false), ChildBoneIndex(INDEX_NONE), RootMotion(FTransform::Identity),
	PreviousTransform(FTransform::Identity) {};
	/** If true we use the ChildBoneIndex otherwise we use the root*/
	bool bBlendFirstChildOfRoot;
	int32 ChildBoneIndex;
	FTransform RootMotion;
	FTransform PreviousTransform;
};

struct FAnimSequencerData
{
	FAnimSequencerData(UAnimSequenceBase* InAnimSequence, int32 InSequenceId, const TOptional<FRootMotionOverride>& InRootMotion, float InFromPosition, float InToPosition, float InWeight, bool bInFireNotifies, ESwapRootBone InSwapRootBone, TOptional<FTransform> InInitialTransform, UMirrorDataTable* InMirrorDataTable)
		: AnimSequence(InAnimSequence)
		, SequenceId(InSequenceId)
		, RootMotion(InRootMotion)
		, FromPosition(InFromPosition)
		, ToPosition(InToPosition)
		, Weight(InWeight)
		, bFireNotifies(bInFireNotifies)
		, SwapRootBone(InSwapRootBone)
		, InitialTransform(InInitialTransform)
		, MirrorDataTable(InMirrorDataTable)
	{
	}

	UAnimSequenceBase* AnimSequence;
	int32 SequenceId;
	const TOptional<FRootMotionOverride>& RootMotion;
	float FromPosition;
	float ToPosition;
	float Weight;
	bool bFireNotifies;
	ESwapRootBone SwapRootBone;
	TOptional<FTransform> InitialTransform;
	UMirrorDataTable* MirrorDataTable;
};

/** Quick n dirty RTTI to allow for derived classes to insert nodes of different types */
#define SEQUENCER_INSTANCE_PLAYER_TYPE(TYPE, BASE) \
	static const FName& GetTypeId() { static FName Type(TEXT(#TYPE)); return Type; } \
	virtual bool IsOfTypeImpl(const FName& Type) const override { return GetTypeId() == Type || BASE::IsOfTypeImpl(Type); }

/** Player type that evaluates a sequence-specified UAnimSequence */
struct FSequencerPlayerAnimSequence : public FSequencerPlayerBase
{
	SEQUENCER_INSTANCE_PLAYER_TYPE(FSequencerPlayerAnimSequence, FSequencerPlayerBase)
	TOptional<FRootMotionOverride> RootMotion;
	FAnimNode_SequenceEvaluator_Standalone PlayerNode;
};


/** Proxy override for this UAnimInstance-derived class */
USTRUCT()
struct FAnimSequencerInstanceProxy : public FAnimInstanceProxy
{
	GENERATED_BODY()

public:
	FAnimSequencerInstanceProxy()
	{
	}

	FAnimSequencerInstanceProxy(UAnimInstance* InAnimInstance)
		: FAnimInstanceProxy(InAnimInstance)
	{
	}

	ANIMGRAPHRUNTIME_API virtual ~FAnimSequencerInstanceProxy();

	// FAnimInstanceProxy interface
	ANIMGRAPHRUNTIME_API virtual void Initialize(UAnimInstance* InAnimInstance) override;
	ANIMGRAPHRUNTIME_API virtual bool Evaluate(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void PostEvaluate(UAnimInstance* InAnimInstance) override;
	ANIMGRAPHRUNTIME_API virtual void UpdateAnimationNode(const FAnimationUpdateContext& InContext) override;

	/** Update an animation sequence player in this instance */
	ANIMGRAPHRUNTIME_API void UpdateAnimTrack(UAnimSequenceBase* InAnimSequence, uint32 SequenceId, float InPosition, float Weight, bool bFireNotifies);
	ANIMGRAPHRUNTIME_API void UpdateAnimTrack(UAnimSequenceBase* InAnimSequence, uint32 SequenceId, TOptional<float> InFromPosition, float InToPosition, float Weight, bool bFireNotifies);
	
	ANIMGRAPHRUNTIME_API void UpdateAnimTrackWithRootMotion(const FAnimSequencerData& InAnimSequencerData);

	/** Reset all nodes in this instance */
	ANIMGRAPHRUNTIME_API virtual void ResetNodes();

	/** Reset the pose in this instance*/
	ANIMGRAPHRUNTIME_API virtual void ResetPose();

	/** Construct and link the base part of the blend tree */
	ANIMGRAPHRUNTIME_API virtual void ConstructNodes();

	ANIMGRAPHRUNTIME_API virtual void AddReferencedObjects(UAnimInstance* InAnimInstance, FReferenceCollector& Collector) override;

protected:

	ANIMGRAPHRUNTIME_API void UpdateAnimTrack(UAnimSequenceBase* InAnimSequence, uint32 SequenceId, const TOptional<FRootMotionOverride>& RootMomtionOverride, TOptional<float> InFromPosition, float InToPosition, float Weight, bool bFireNotifies, UMirrorDataTable* InMirrorDataTable);

	/** Find a player of a specified type */
	template<typename Type>
	Type* FindPlayer(uint32 SequenceId) const
	{
		FSequencerPlayerBase* Player = SequencerToPlayerMap.FindRef(SequenceId);
		if (Player && Player->IsOfType<Type>())
		{
			return static_cast<Type*>(Player);
		}

		return nullptr;
	}

	/** custom root node for this sequencer player. Didn't want to use RootNode in AnimInstance because it's used with lots of AnimBP functionality */
	struct FAnimNode_ApplyAdditive SequencerRootNode;
	struct FAnimNode_MultiWayBlend FullBodyBlendNode;
	struct FAnimNode_MultiWayBlend AdditiveBlendNode;
	struct FAnimNode_PoseSnapshot  SnapshotNode;

	/** mapping from sequencer index to internal player index */
	TMap<uint32, FSequencerPlayerBase*> SequencerToPlayerMap;

	/** mapping from sequencer index to internal mirror node index */
	TMap<uint32, FAnimNode_Mirror_Standalone*> SequencerToMirrorMap;

	/** custom root motion override sent in from sequencer */
	TOptional<FRootMotionOverride> RootMotionOverride;

	ANIMGRAPHRUNTIME_API void InitAnimTrack(UAnimSequenceBase* InAnimSequence, uint32 SequenceId);
	ANIMGRAPHRUNTIME_API void EnsureAnimTrack(UAnimSequenceBase* InAnimSequence, uint32 SequenceId);
	ANIMGRAPHRUNTIME_API void ClearSequencePlayerAndMirrorMaps();

	ESwapRootBone SwapRootBone = ESwapRootBone::SwapRootBone_None;
	TOptional<FTransform> InitialTransform;
	TOptional<FTransform> RootBoneTransform;
};
