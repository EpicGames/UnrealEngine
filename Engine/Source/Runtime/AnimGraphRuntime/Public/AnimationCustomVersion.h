// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

#define UE_API ANIMGRAPHRUNTIME_API

// Custom serialization version for assets/classes in the AnimGraphRuntime and AnimGraph modules
struct FAnimationCustomVersion
{
	enum Type
	{
		// Before any version changes were made in the plugin
		BeforeCustomVersionWasAdded = 0,

		// Added support for one-to-many component mappings to FAnimNode_BoneDrivenController,
		// changed the range to apply to the input, and added a configurable method for updating the components
		BoneDrivenControllerMatchingMaya = 1,

		// Converted the range clamp into a remap function, rather than just clamping
		BoneDrivenControllerRemapping = 2,

		// Added ability to offset angular ranges for constraints
		AnimDynamicsAddAngularOffsets = 3,

		// Renamed Stretch Limits to better names
		RenamedStretchLimits = 4,

		// Convert IK to support FBoneSocketTarget
		ConvertIKToSupportBoneSocketTarget = 5,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	UE_API const static FGuid GUID;

private:
	FAnimationCustomVersion() {}
};

#undef UE_API
