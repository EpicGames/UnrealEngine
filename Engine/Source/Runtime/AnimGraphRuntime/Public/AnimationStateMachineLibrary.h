// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Animation/AnimExecutionContext.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimNodeReference.h"
#include "Animation/AnimStateMachineTypes.h"
#include "AnimationStateMachineLibrary.generated.h"

struct FAnimNode_StateMachine;
struct FAnimNode_StateResult;

USTRUCT(BlueprintType, DisplayName = "Animation State Reference")
struct FAnimationStateResultReference : public FAnimNodeReference
{
	GENERATED_BODY()

	typedef FAnimNode_StateResult FInternalNodeType;
};

USTRUCT(BlueprintType, DisplayName = "Animation State Machine")
struct FAnimationStateMachineReference : public FAnimNodeReference
{
	GENERATED_BODY()

	typedef FAnimNode_StateMachine FInternalNodeType;
};

// Exposes operations to be performed on anim state machine node contexts
UCLASS(Experimental, MinimalAPI)
class UAnimationStateMachineLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Get an anim state reference from an anim node reference */
	UFUNCTION(BlueprintCallable, Category = "State Machine", meta=(BlueprintThreadSafe, DisplayName = "Convert to Animation State", ExpandEnumAsExecs = "Result"))
	static ANIMGRAPHRUNTIME_API void ConvertToAnimationStateResult(const FAnimNodeReference& Node, FAnimationStateResultReference& AnimationState, EAnimNodeReferenceConversionResult& Result);

	/** Get an anim state reference from an anim node reference (pure) */
	UFUNCTION(BlueprintPure, Category = "State Machine", meta=(BlueprintThreadSafe, DisplayName = "Convert to Animation State"))
	static void ConvertToAnimationStateResultPure(const FAnimNodeReference& Node, FAnimationStateResultReference& AnimationState, bool& Result)
	{
		EAnimNodeReferenceConversionResult ConversionResult;
		ConvertToAnimationStateResult(Node, AnimationState, ConversionResult);
		Result = (ConversionResult == EAnimNodeReferenceConversionResult::Succeeded);	
	}

	/** Get an anim state machine from an anim node reference */
	UFUNCTION(BlueprintCallable, Category = "State Machine", meta=(BlueprintThreadSafe, DisplayName = "Convert to Animation State Machine", ExpandEnumAsExecs = "Result"))
	static ANIMGRAPHRUNTIME_API void ConvertToAnimationStateMachine(const FAnimNodeReference& Node, FAnimationStateMachineReference& AnimationState, EAnimNodeReferenceConversionResult& Result);


	/** Get an anim state machine from an anim node reference (pure) */
	UFUNCTION(BlueprintPure, Category = "State Machine", meta = (BlueprintThreadSafe, DisplayName = "Convert to Animation State Machine"))
	static void ConvertToAnimationStateMachinePure(const FAnimNodeReference& Node, FAnimationStateMachineReference& AnimationState, bool& Result)
	{
		EAnimNodeReferenceConversionResult ConversionResult;
		ConvertToAnimationStateMachine(Node, AnimationState, ConversionResult);
		Result = (ConversionResult == EAnimNodeReferenceConversionResult::Succeeded);
	}
	
	/** Returns whether the state the node belongs to is blending in */
	UFUNCTION(BlueprintPure, Category = "State Machine", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API bool IsStateBlendingIn(const FAnimUpdateContext& UpdateContext, const FAnimationStateResultReference& Node);

	/** Returns whether the state the node belongs to is blending out */
	UFUNCTION(BlueprintPure, Category = "State Machine", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API bool IsStateBlendingOut(const FAnimUpdateContext& UpdateContext, const FAnimationStateResultReference& Node);

	/** Manually set the current state of the state machine
		NOTE: Custom blend type is not supported */
	UFUNCTION(BlueprintCallable, Category = "State Machine", meta=(BlueprintThreadSafe, AdvancedDisplay = "4"))
	static ANIMGRAPHRUNTIME_API void SetState(const FAnimUpdateContext& UpdateContext, const FAnimationStateMachineReference& Node, FName TargetState, float Duration
		, TEnumAsByte<ETransitionLogicType::Type> BlendType, UBlendProfile* BlendProfile, EAlphaBlendOption AlphaBlendOption, UCurveFloat* CustomBlendCurve);

	/** Returns the name of the current state of this state machine */
	UFUNCTION(BlueprintPure, Category = "State Machine", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FName GetState(const FAnimUpdateContext& UpdateContext, const FAnimationStateMachineReference& Node);

	/** Returns the remaining animation time of the state's most relevant asset player */
	UFUNCTION(BlueprintPure, Category = "State Machine", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API float GetRelevantAnimTimeRemaining(const FAnimUpdateContext& UpdateContext, const FAnimationStateResultReference& Node);

	/** Returns the remaining animation time as a fraction of the duration for the state's most relevant asset player */
	UFUNCTION(BlueprintPure, Category = "State Machine", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API float GetRelevantAnimTimeRemainingFraction(const FAnimUpdateContext& UpdateContext, const FAnimationStateResultReference& Node);
};
