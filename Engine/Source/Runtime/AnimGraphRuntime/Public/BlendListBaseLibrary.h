// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Animation/AnimExecutionContext.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimNodeReference.h"
#include "BlendListBaseLibrary.generated.h"

struct FAnimNode_BlendListBase;
USTRUCT(BlueprintType)
struct FBlendListBaseReference : public FAnimNodeReference
{
	GENERATED_BODY()
	typedef FAnimNode_BlendListBase FInternalNodeType;
};

// Exposes operations to be performed on anim state machine node contexts
UCLASS(Experimental, MinimalAPI)
class UBlendListBaseLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Get a blend list base context from an anim node context. */
	UFUNCTION(BlueprintCallable, Category = "Blend List Base", meta = (BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static ANIMGRAPHRUNTIME_API FBlendListBaseReference ConvertToBlendListBase(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result);

	/** Reset target blend list node to that the next blend is executed from a blank state */
	UFUNCTION(BlueprintCallable, Category = "Blend List Base", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API void ResetNode(const FBlendListBaseReference& BlendListBase);


};
