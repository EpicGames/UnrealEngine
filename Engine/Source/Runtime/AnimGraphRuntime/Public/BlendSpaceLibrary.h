// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimExecutionContext.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimNodeReference.h"
#include "BlendSpaceLibrary.generated.h"

struct FAnimNode_BlendSpaceGraph;

USTRUCT(BlueprintType)
struct FBlendSpaceReference : public FAnimNodeReference
{
	GENERATED_BODY()

	typedef FAnimNode_BlendSpaceGraph FInternalNodeType;
};

/**
 * Exposes operations to be performed on a blend space anim node.
 */
UCLASS(MinimalAPI)
class UBlendSpaceLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Get a blend space context from an anim node context. */
	UFUNCTION(BlueprintCallable, Category = "Blend Space", meta = (BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static ANIMGRAPHRUNTIME_API FBlendSpaceReference ConvertToBlendSpace(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result);

	/** Get a blend space context from an anim node context (pure). */
	UFUNCTION(BlueprintPure, Category = "Blend Space", meta = (BlueprintThreadSafe, DisplayName = "Convert to Blend Space (Pure)"))
	static void ConvertToBlendSpacePure(const FAnimNodeReference& Node, FBlendSpaceReference& BlendSpace, bool& Result)
	{
		EAnimNodeReferenceConversionResult ConversionResult;
		BlendSpace = ConvertToBlendSpace(Node, ConversionResult);
		Result = (ConversionResult == EAnimNodeReferenceConversionResult::Succeeded);
	}

	/** Get the current position of the blend space. */
	UFUNCTION(BlueprintPure, Category = "Blend Space", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FVector GetPosition(const FBlendSpaceReference& BlendSpace);

	/** Get the current sample coordinates after going through the filtering. */
	UFUNCTION(BlueprintPure, Category = "Blend Space", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FVector GetFilteredPosition(const FBlendSpaceReference& BlendSpace);

	/** Forces the Position to the specified value */
	UFUNCTION(BlueprintCallable, Category = "Blend Space", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API void SnapToPosition(const FBlendSpaceReference& BlendSpace, FVector NewPosition);
};



