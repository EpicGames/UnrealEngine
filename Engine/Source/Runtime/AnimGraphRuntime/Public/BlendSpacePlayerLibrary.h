// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimExecutionContext.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimNodeReference.h"
#include "BlendSpacePlayerLibrary.generated.h"

struct FAnimNode_BlendSpacePlayer;

USTRUCT(BlueprintType)
struct FBlendSpacePlayerReference : public FAnimNodeReference
{
	GENERATED_BODY()

	typedef FAnimNode_BlendSpacePlayer FInternalNodeType;
};

/**
 * Exposes operations to be performed on a blend space player anim node.
 */
UCLASS(MinimalAPI)
class UBlendSpacePlayerLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Get a blend space player context from an anim node context. */
	UFUNCTION(BlueprintCallable, Category = "Blend Space Player", meta = (BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static ANIMGRAPHRUNTIME_API FBlendSpacePlayerReference ConvertToBlendSpacePlayer(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result);

	/** Get a blend space player context from an anim node context (pure). */
	UFUNCTION(BlueprintPure, Category = "Blend Space Player", meta = (BlueprintThreadSafe, DisplayName = "Convert to Blend Space Player"))
	static void ConvertToBlendSpacePlayerPure(const FAnimNodeReference& Node, FBlendSpacePlayerReference& BlendSpacePlayer, bool& Result)
	{
		EAnimNodeReferenceConversionResult ConversionResult;
		BlendSpacePlayer = ConvertToBlendSpacePlayer(Node, ConversionResult);
		Result = (ConversionResult == EAnimNodeReferenceConversionResult::Succeeded);
	}

	/** Set the current BlendSpace of the blend space player. */
	UFUNCTION(BlueprintCallable, Category = "Blend Space Player", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FBlendSpacePlayerReference SetBlendSpace(const FBlendSpacePlayerReference& BlendSpacePlayer, UBlendSpace* BlendSpace);

	/** Set the current BlendSpace of the blend space player with an interial blend time. */
	UFUNCTION(BlueprintCallable, Category = "Blend Space Player", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FBlendSpacePlayerReference SetBlendSpaceWithInertialBlending(const FAnimUpdateContext& UpdateContext, const FBlendSpacePlayerReference& BlendSpacePlayer, UBlendSpace* BlendSpace, float BlendTime = 0.2f);

	/** Set whether the current play time should reset when BlendSpace changes of the blend space player. */
	UFUNCTION(BlueprintCallable, Category = "Blend Space Player", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FBlendSpacePlayerReference SetResetPlayTimeWhenBlendSpaceChanges(const FBlendSpacePlayerReference& BlendSpacePlayer, bool bReset);
	
	/** Set the play rate of the blend space player. */
	UFUNCTION(BlueprintCallable, Category = "Blend Space Player", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FBlendSpacePlayerReference SetPlayRate(const FBlendSpacePlayerReference& BlendSpacePlayer, float PlayRate);

	/** Set the loop of the blend space player. */
	UFUNCTION(BlueprintCallable, Category = "Blend Space Player", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FBlendSpacePlayerReference SetLoop(const FBlendSpacePlayerReference& BlendSpacePlayer, bool bLoop);

	/** Get the current BlendSpace of the blend space player. */
	UFUNCTION(BlueprintPure, Category = "Blend Space Player", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API UBlendSpace* GetBlendSpace(const FBlendSpacePlayerReference& BlendSpacePlayer);

	/** Get the current position of the blend space player. */
	UFUNCTION(BlueprintPure, Category = "Blend Space Player", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FVector GetPosition(const FBlendSpacePlayerReference& BlendSpacePlayer);

	/** Get the current start position of the blend space player. */
	UFUNCTION(BlueprintPure, Category = "Blend Space Player", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API float GetStartPosition(const FBlendSpacePlayerReference& BlendSpacePlayer);

	/** Get the current play rate of the blend space player. */
	UFUNCTION(BlueprintPure, Category = "Blend Space Player", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API float GetPlayRate(const FBlendSpacePlayerReference& BlendSpacePlayer);

	/** Get the current loop of the blend space player.  */
	UFUNCTION(BlueprintPure, Category = "Blend Space Player", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API bool GetLoop(const FBlendSpacePlayerReference& BlendSpacePlayer);

	/** Get the current value of whether the current play time should reset when BlendSpace changes of the blend space player. */
	UFUNCTION(BlueprintPure, Category = "Blend Space Player", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API bool ShouldResetPlayTimeWhenBlendSpaceChanges(const FBlendSpacePlayerReference& BlendSpacePlayer); 

	/** Forces the Position to the specified value */
	UFUNCTION(BlueprintCallable, Category = "Blend Space Player", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API void SnapToPosition(const FBlendSpacePlayerReference& BlendSpacePlayer, FVector NewPosition);

	/** Set the current accumulated time of the blend space player. For blend spaces, this is normalized (0 - 1) */
	UFUNCTION(BlueprintCallable, Category = "Blend Space Player", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FBlendSpacePlayerReference SetAccumulatedTime(const FBlendSpacePlayerReference& BlendSpacePlayer, float Time);

	/** Gets the current accumulated time of the blend space player. For blend spaces, this is normalized (0 - 1) */
	UFUNCTION(BlueprintPure, Category = "Blend Space Player", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API float GetAccumulatedTime(const FBlendSpacePlayerReference& BlendSpacePlayer);
};



