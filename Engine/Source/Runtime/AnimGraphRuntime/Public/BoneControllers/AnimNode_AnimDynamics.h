// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "BoneContainer.h"
#include "Animation/AnimPhysicsSolver.h"
#include "BonePose.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "Animation/AnimInstanceProxy.h"
#include "CommonAnimationTypes.h"
#include "AnimNode_AnimDynamics.generated.h"

class UAnimInstance;
class USkeletalMeshComponent;
template <class T> class TAutoConsoleVariable;

extern TAutoConsoleVariable<int32> CVarEnableDynamics;
extern ANIMGRAPHRUNTIME_API TAutoConsoleVariable<int32> CVarLODThreshold;
extern TAutoConsoleVariable<int32> CVarEnableWind;

#if ENABLE_ANIM_DRAW_DEBUG

extern TAutoConsoleVariable<int32> CVarShowDebug;
extern TAutoConsoleVariable<FString> CVarDebugBone;

#endif

DECLARE_CYCLE_STAT_EXTERN(TEXT("Anim Dynamics Overall"), STAT_AnimDynamicsOverall, STATGROUP_Physics, ANIMGRAPHRUNTIME_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Anim Dynamics Wind Data Update"), STAT_AnimDynamicsWindData, STATGROUP_Physics, ANIMGRAPHRUNTIME_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Anim Dynamics Bone Evaluation"), STAT_AnimDynamicsBoneEval, STATGROUP_Physics, ANIMGRAPHRUNTIME_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Anim Dynamics Sub-Steps"), STAT_AnimDynamicsSubSteps, STATGROUP_Physics, ANIMGRAPHRUNTIME_API);

/** Supported angular constraint types */
UENUM()
enum class AnimPhysAngularConstraintType : uint8
{
	Angular,
	Cone
};

/** Supported linear axis constraints */
UENUM()
enum class AnimPhysLinearConstraintType : uint8
{
	Free,
	Limited,
};

UENUM(BlueprintType)
enum class AnimPhysSimSpaceType : uint8
{
	Component UMETA(ToolTip = "Sim origin is the location/orientation of the skeletal mesh component."),
	Actor UMETA(ToolTip = "Sim origin is the location/orientation of the actor containing the skeletal mesh component."),
	World UMETA(ToolTip = "Sim origin is the world origin. Teleporting characters is not recommended in this mode."),
	RootRelative UMETA(ToolTip = "Sim origin is the location/orientation of the root bone."),
	BoneRelative UMETA(ToolTip = "Sim origin is the location/orientation of the bone specified in RelativeSpaceBone"),
};

/** Helper mapping a rigid body to a bone reference */
struct FAnimPhysBoneRigidBody
{
	FAnimPhysBoneRigidBody(TArray<FAnimPhysShape>& Shapes, const FVector& Position, const FBoneReference& LinkedBone)
	: PhysBody(Shapes, Position)
	, BoundBone(LinkedBone)
	{}

	FAnimPhysRigidBody PhysBody;
	FBoneReference BoundBone;
};

/** Helper describing a body linked to an optional parent (can be nullptr) */
struct FAnimPhysLinkedBody
{
	FAnimPhysLinkedBody(TArray<FAnimPhysShape>& Shapes, const FVector& Position, const FBoneReference& LinkedBone)
	: RigidBody(Shapes, Position, LinkedBone)
	, ParentBody(nullptr)
	{}

	FAnimPhysBoneRigidBody RigidBody;
	FAnimPhysBoneRigidBody* ParentBody;
};

/** Constraint setup struct, holds data required to build a physics constraint */
USTRUCT()
struct FAnimPhysConstraintSetup
{
	GENERATED_BODY()

	FAnimPhysConstraintSetup()
	: LinearXLimitType(AnimPhysLinearConstraintType::Limited)
	, LinearYLimitType(AnimPhysLinearConstraintType::Limited)
	, LinearZLimitType(AnimPhysLinearConstraintType::Limited)
	, bLinearFullyLocked(false)
	, LinearAxesMin(ForceInitToZero)
	, LinearAxesMax(ForceInitToZero)
	, AngularConstraintType(AnimPhysAngularConstraintType::Angular)
	, TwistAxis(AnimPhysTwistAxis::AxisX)
	, AngularTargetAxis(AnimPhysTwistAxis::AxisX)
	, ConeAngle(0.0f)
#if WITH_EDITORONLY_DATA
	, AngularXAngle_DEPRECATED(0.0f)
	, AngularYAngle_DEPRECATED(0.0f)
	, AngularZAngle_DEPRECATED(0.0f)
#endif
	, AngularLimitsMin(ForceInitToZero)
	, AngularLimitsMax(ForceInitToZero)
	, AngularTarget(ForceInitToZero)
	{}

	/** Whether to limit the linear X axis */
	UPROPERTY(EditAnywhere, Category = Linear)
	AnimPhysLinearConstraintType LinearXLimitType;

	/** Whether to limit the linear Y axis */
	UPROPERTY(EditAnywhere, Category = Linear)
	AnimPhysLinearConstraintType LinearYLimitType;

	/** Whether to limit the linear Z axis */
	UPROPERTY(EditAnywhere, Category = Linear)
	AnimPhysLinearConstraintType LinearZLimitType;

	/** If all axes are locked we can use 3 linear limits instead of the 6 needed for limited axes */
	bool bLinearFullyLocked;

	/** Minimum linear movement per-axis (Set zero here and in the max limit to lock) */
	UPROPERTY(EditAnywhere, Category = Linear, meta = (UIMax = "0", ClampMax = "0"))
	FVector LinearAxesMin;

	/** Maximum linear movement per-axis (Set zero here and in the min limit to lock) */
	UPROPERTY(EditAnywhere, Category = Linear, meta = (UIMin = "0", ClampMin = "0"))
	FVector LinearAxesMax;

	/** Method to use when constraining angular motion */
	UPROPERTY(EditAnywhere, Category = Angular)
	AnimPhysAngularConstraintType AngularConstraintType;

	/** Axis to consider for twist when constraining angular motion (forward axis) */
	UPROPERTY(EditAnywhere, Category = Angular)
	AnimPhysTwistAxis TwistAxis;

	/**
	 * The axis in the simulation pose to align to the Angular Target.
	 * This is typically the axis pointing along the bone.
	 * Note: This is affected by the Angular Spring Constant.
	 */
	UPROPERTY(EditAnywhere, Category = Angular, meta=(DisplayAfter=AngularLimitsMax))
	AnimPhysTwistAxis AngularTargetAxis;

	/** Angle to use when constraining using a cone */
	UPROPERTY(EditAnywhere, Category = Angular, meta = (UIMin = "0", UIMax = "90", ClampMin = "0", ClampMax = "90"))
	float ConeAngle;

#if WITH_EDITORONLY_DATA
	/** X-axis limit for angular motion when using the "Angular" constraint type (Set to 0 to lock, or 180 to remain free) */
	UPROPERTY()
	float AngularXAngle_DEPRECATED;

	/** Y-axis limit for angular motion when using the "Angular" constraint type (Set to 0 to lock, or 180 to remain free) */
	UPROPERTY()
	float AngularYAngle_DEPRECATED;

	/** Z-axis limit for angular motion when using the "Angular" constraint type (Set to 0 to lock, or 180 to remain free) */
	UPROPERTY()
	float AngularZAngle_DEPRECATED;
#endif

	UPROPERTY(EditAnywhere, Category = Angular, meta = (UIMin = "-180", UIMax = "180", ClampMin = "-180", ClampMax = "180"))
	FVector AngularLimitsMin;

	UPROPERTY(EditAnywhere, Category = Angular, meta = (UIMin = "-180", UIMax = "180", ClampMin = "-180", ClampMax = "180"))
	FVector AngularLimitsMax;

	/**
	 * The axis to align the angular spring constraint to in the animation pose.
	 * This typically points down the bone - so values of (1.0, 0.0, 0.0) are common,
	 * but you can pick other values to align the spring to a different direction.
	 * Note: This is affected by the Angular Spring Constant.
	 */
	UPROPERTY(EditAnywhere, Category = Angular)
	FVector AngularTarget;
};

USTRUCT()
struct FAnimPhysPlanarLimit
{
	GENERATED_BODY();

	/** When using a driving bone, the plane transform will be relative to the bone transform */
	UPROPERTY(EditAnywhere, Category=PlanarLimit)
	FBoneReference DrivingBone;

	/** Transform of the plane, this is either in component-space if no DrivinBone is specified
	 *  or in bone-space if a driving bone is present.
	 */
	UPROPERTY(EditAnywhere, Category=PlanarLimit)
	FTransform PlaneTransform;
};

/** Whether spheres keep bodies inside, or outside of their shape */
UENUM()
enum class ESphericalLimitType : uint8
{
	Inner,
	Outer
};

USTRUCT()
struct FAnimPhysSphericalLimit
{
	GENERATED_BODY();

	FAnimPhysSphericalLimit()
		: SphereLocalOffset(FVector::ZeroVector)
		, LimitRadius(0.0f)
		, LimitType(ESphericalLimitType::Outer)
	{}

	/** Bone to attach the sphere to */
	UPROPERTY(EditAnywhere, Category = SphericalLimit)
	FBoneReference DrivingBone;

	/** Local offset for the sphere, if no driving bone is set this is in node space, otherwise bone space */
	UPROPERTY(EditAnywhere, Category = SphericalLimit)
	FVector SphereLocalOffset;

	/** Radius of the sphere */
	UPROPERTY(EditAnywhere, Category = SphericalLimit)
	float LimitRadius;

	/** Whether to lock bodies inside or outside of the sphere */
	UPROPERTY(EditAnywhere, Category = SphericalLimit)
	ESphericalLimitType LimitType;
};

USTRUCT()
struct FAnimPhysBodyDefinition
{
	GENERATED_BODY();

	FAnimPhysBodyDefinition()
	: BoxExtents(10.0f, 10.0f, 10.0f)
	, LocalJointOffset(FVector::ZeroVector)
	, CollisionType(AnimPhysCollisionType::CoM)
	, SphereCollisionRadius(10.0f)
	{}

	FAnimPhysBodyDefinition& operator=(const FAnimPhysBodyDefinition& Other)
	{
		this->BoundBone = Other.BoundBone;
		this->BoxExtents = Other.BoxExtents;
		this->LocalJointOffset = Other.LocalJointOffset;
		this->ConstraintSetup = Other.ConstraintSetup;
		this->CollisionType = Other.CollisionType;
		this->SphereCollisionRadius = Other.SphereCollisionRadius;

		return *this;
	}

	UPROPERTY(VisibleAnywhere, Category = PhysicsBodyDefinition, meta = (EditCondition = "false"))
	FBoneReference BoundBone;

	/** Extents of the box to use for simulation */
	UPROPERTY(EditAnywhere, Category = PhysicsBodyDefinition, meta = (UIMin = "1", ClampMin = "1"))
	FVector BoxExtents;

	/** Vector relative to the body being simulated to attach the constraint to */
	UPROPERTY(EditAnywhere, Category = PhysicsBodyDefinition)
	FVector LocalJointOffset;

	/** Data describing the constraints we will apply to the body */
	UPROPERTY(EditAnywhere, Category = Constraint)
	FAnimPhysConstraintSetup ConstraintSetup;

	/** Resolution method for planar limits */
	UPROPERTY(EditAnywhere, Category = Collision)
	AnimPhysCollisionType CollisionType;

	/** Radius to use if CollisionType is set to CustomSphere */
	UPROPERTY(EditAnywhere, Category = PhysicsBodyDefinition, meta = (UIMin = "1", ClampMin = "1", EditCondition = "CollisionType == AnimPhysCollisionType::CustomSphere"))
	float SphereCollisionRadius;
};

struct FAnimConstraintOffsetPair
{
	FAnimConstraintOffsetPair(const FVector& InBody0Offset, const FVector InBody1Offset)
	: Body0Offset(InBody0Offset)
	, Body1Offset(InBody1Offset)
	{}

	FVector Body0Offset;
	FVector Body1Offset;
};

/**
 * Settings for the system which passes rotational motion of the simulation's space into the simulation. This allows the simulation to pass a 
 * fraction of the world space motion onto the bodies which allows Bone-Space and Component-Space simulations to react to world-space 
 * movement in a controllable way.
 */
template <> struct TIsPODType<FAnimPhysSimSpaceSettings> { enum { Value = true }; };

USTRUCT(BlueprintType)
struct FAnimPhysSimSpaceSettings
{
	GENERATED_BODY()

	ANIMGRAPHRUNTIME_API FAnimPhysSimSpaceSettings();

	~FAnimPhysSimSpaceSettings() = default;
	FAnimPhysSimSpaceSettings(FAnimPhysSimSpaceSettings const&) = default;
	FAnimPhysSimSpaceSettings& operator=(const FAnimPhysSimSpaceSettings&) = default;

	/**
	 * Global multipler on the effects of simulation space rotational movement. Must be in range[0, 1].If SimSpaceAngularAlpha = 0.0, the system is disabled and the simulation will
	 * be fully local (i.e., world-space actor movement and rotation does not affect the simulation).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float SimSpaceAngularAlpha;

	/**
	 * A clamp on the effective world-space angular velocity that is passed to the simulation. Units are radian/s, so a value of about 6.0 is one rotation per second.
	 * The default value effectively means "unlimited". You would reduce this (and MaxAngularAcceleration) to limit how much bodies "fly out" when the actor spins on the spot.
	 * This is especially useful if you have characters than can rotate very quickly and you would probably want values around or less than 10 in this case.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ClampMin = "0.0"))
	float MaxAngularVelocity;
	
	/**
	 * A clamp on the effective world-space angular accleration that is passed to the simulation. Units are radian/s/s. The default value effectively means "unlimited".
	 * This has a similar effect to MaxAngularVelocity, except that it is related to the flying out of bodies when the rotation speed suddenly changes. Typical limits for
	 * a character might be around 100.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ClampMin = "0.0"))
	float MaxAngularAcceleration;

	/**
	 * Additional angular velocity that is added to the component angular velocity. This can be used to make the simulation act as if the actor is rotating
	 * even when it is not. E.g., to apply physics to a character on a podium as the camera rotates around it, to emulate the podium itself rotating.
	 * Vector is in world space. Units are rad/s.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	FVector ExternalAngularVelocity;

};


USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_AnimDynamics : public FAnimNode_SkeletalControlBase
{
	GENERATED_BODY();

	ANIMGRAPHRUNTIME_API FAnimNode_AnimDynamics();

	/**
	* Overridden linear damping value. The default is 0.7. Values below 0.7 won't have an effect.
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = PhysicsParameters, meta = (PinHiddenByDefault, DisplayAfter = "NumSolverIterationsPostUpdate", EditCondition = bOverrideLinearDamping))
	float LinearDampingOverride;

	/**
	 * Overridden angular damping value. The default is 0.7. Values below 0.7 won't have an effect.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = PhysicsParameters, meta = (PinHiddenByDefault, DisplayAfter = "LinearDampingOverride", EditCondition = bOverrideAngularDamping))
	float AngularDampingOverride;

	// Previous component & actor transforms, used to account for teleports
	FTransform PreviousCompWorldSpaceTM;
	FTransform PreviousActorWorldSpaceTM;
	FTransform PreviousSimSpaceTM;

	/** When in BoneRelative sim space, the simulation will use this bone as the origin */
	UPROPERTY(EditAnywhere, Category = Setup, meta=(DisplayAfter="SimulationSpace", EditCondition = "SimulationSpace == AnimPhysSimSpaceType::BoneRelative"))
	FBoneReference RelativeSpaceBone;

	/** The bone to attach the physics body to, if bChain is true this is the top of the chain */
	UPROPERTY(EditAnywhere, Category = Setup)
	FBoneReference BoundBone;

	/** If bChain is true this is the bottom of the chain, otherwise ignored */
	UPROPERTY(EditAnywhere, Category = Setup, meta = (EditCondition = bChain, DisplayAfter = "BoundBone"))
	FBoneReference ChainEnd;

	UPROPERTY(EditAnywhere, EditFixedSize, Category = PhysicsParameters, meta = (DisplayName = "Body Definitions", EditFixedOrder, DisplayAfter = "ChainEnd"))
	TArray< FAnimPhysBodyDefinition > PhysicsBodyDefinitions;

	/** Scale for gravity, higher values increase forces due to gravity */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsParameters, meta = (PinHiddenByDefault, DisplayAfter = "bGravityOverrideInSimSpace", EditCondition = "!bUseGravityOverride"))
	float GravityScale;

	/** Gravity Override Value */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsParameters, meta = (PinHiddenByDefault, DisplayAfter = "bUseGravityOverride", EditCondition = "bUseGravityOverride"))
	FVector GravityOverride;

	/** 
	 * Spring constant to use when calculating linear springs, higher values mean a stronger spring.
	 * You need to enable the Linear Spring checkbox for this to have an effect.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsParameters, meta = (EditCondition = bLinearSpring, PinHiddenByDefault, DisplayAfter = "AngularSpringConstant"))
	float LinearSpringConstant;

	/** 
	 * Spring constant to use when calculating angular springs, higher values mean a stronger spring.
	 * You need to enable the Angular Spring checkbox for this to have an effect.
	 * Note: Make sure to also set the Angular Target Axis and Angular Target in the Constraint Setup for this to have an effect.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsParameters, meta = (DisplayAfter = "PhysicsBodyDefinitions", EditCondition = bAngularSpring, PinHiddenByDefault))
	float AngularSpringConstant;

	/**
		For world-space simulations, if the magnitude of the component's 3D scale is less than WorldSpaceMinimumScale, do not update the node.
	*/
	UPROPERTY(EditAnywhere, Category = Settings)
	float WorldSpaceMinimumScale;

	/** Scale to apply to calculated wind velocities in the solver */
	UPROPERTY(EditAnywhere, Category = Wind, meta=(DisplayAfter="bEnableWind"))
	float WindScale;

	/** When using non-world-space sim, this controls how much of the components world-space acceleration is passed on to the local-space simulation. */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	FVector ComponentLinearAccScale;

	/** When using non-world-space sim, this applies a 'drag' to the bodies in the local space simulation, based on the components world-space velocity. */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	FVector ComponentLinearVelScale;

	/** When using non-world-space sim, this is an overall clamp on acceleration derived from ComponentLinearAccScale and ComponentLinearVelScale, to ensure it is not too large. */
	UPROPERTY(EditAnywhere, Category = Settings)
	FVector	ComponentAppliedLinearAccClamp;

	/**
	 * Settings for the system which passes motion of the simulation's space
	 * into the simulation. This allows the simulation to pass a
	 * fraction of the world space motion onto the bodies which allows Bone-Space
	 * and Component-Space simulations to react to world-space movement in a
	 * controllable way.
	 * This system is a superset of the functionality provided by ComponentLinearAccScale,
	 * ComponentLinearVelScale, and ComponentAppliedLinearAccClamp. In general
	 * you should not have both systems enabled.
	 */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	FAnimPhysSimSpaceSettings SimSpaceSettings;

	/** Overridden angular bias value
	 *  Angular bias is essentially a twist reduction for chain forces and defaults to a value to keep chains stability
	*  in check. When using single-body systems sometimes angular forces will look like they are "catching-up" with
	*  the mesh, if that's the case override this and push it towards 1.0f until it settles correctly
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = PhysicsParameters, meta = (PinHiddenByDefault, DisplayAfter = "AngularDampingOverride", EditCondition = bOverrideAngularBias))
	float AngularBiasOverride;

	/** Number of update passes on the linear and angular limits before we solve the position of the bodies recommended to be four times the value of NumSolverIterationsPostUpdate */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = PhysicsParameters)
	int32 NumSolverIterationsPreUpdate;

	/** Number of update passes on the linear and angular limits after we solve the position of the bodies, recommended to be around a quarter of NumSolverIterationsPreUpdate */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = PhysicsParameters, meta = (PinHiddenByDefault, DisplayAfter = "NumSolverIterationsPreUpdate"))
	int32 NumSolverIterationsPostUpdate;

	/** List of available spherical limits for this node */
	UPROPERTY(EditAnywhere, Category = SphericalLimit, meta=(DisplayAfter="bUseSphericalLimits"))
	TArray<FAnimPhysSphericalLimit> SphericalLimits;

	/** An external force to apply to all bodies in the simulation when ticked, specified in world space */
	UPROPERTY(EditAnywhere, Category = Forces, meta = (PinShownByDefault))
	FVector ExternalForce;

	/** List of available planar limits for this node */
	UPROPERTY(EditAnywhere, Category=PlanarLimit, meta=(DisplayAfter="bUsePlanarLimit"))
	TArray<FAnimPhysPlanarLimit> PlanarLimits;

	/** The space used to run the simulation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Setup, meta = (PinHiddenByDefault, DisplayPriority=0))
	AnimPhysSimSpaceType SimulationSpace;

	// Cached sim space that we last used
	AnimPhysSimSpaceType LastSimSpace;

	// We can't get clean bone positions unless we are in the evaluate step.
	// Requesting an init or reinit sets this flag for us to pick up during evaluate
	ETeleportType InitTeleportType;

	/** Whether to evaluate spherical limits */
	UPROPERTY(EditAnywhere, Category = SphericalLimit)
	uint8 bUseSphericalLimits:1;

	/** Whether to evaluate planar limits */
	UPROPERTY(EditAnywhere, Category=PlanarLimit)
	uint8 bUsePlanarLimit:1;

	/** If true we will perform physics update, otherwise skip - allows visualization of the initial state of the bodies */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = PhysicsParameters, meta = (DisplayAfter = "bDoEval"))
	uint8 bDoUpdate : 1;

	/** If true we will perform bone transform evaluation, otherwise skip - allows visualization of the initial anim state compared to the physics sim */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = PhysicsParameters, meta = (DisplayAfter = "AngularBiasOverride"))
	uint8 bDoEval : 1;

	/** If true, the override value will be used for linear damping */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = PhysicsParameters, meta=(InlineEditConditionToggle, DisplayAfter="AngularSpringConstraint"))
	uint8 bOverrideLinearDamping:1;

	/** If true, the override value will be used for the angular bias for bodies in this node. 
	 *  Angular bias is essentially a twist reduction for chain forces and defaults to a value to keep chains stability
	 *  in check. When using single-body systems sometimes angular forces will look like they are "catching-up" with
	 *  the mesh, if that's the case override this and push it towards 1.0f until it settles correctly
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = PhysicsParameters, meta=(InlineEditConditionToggle, DisplayAfter="AngularDampingOverride"))
	uint8 bOverrideAngularBias:1;

	/** If true, the override value will be used for angular damping */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = PhysicsParameters, meta=(InlineEditConditionToggle, DisplayAfter="LinearDampingOverride"))
	uint8 bOverrideAngularDamping:1;

	/** Whether or not wind is enabled for the bodies in this simulation */
	UPROPERTY(EditAnywhere, Category = Wind)
	uint8 bEnableWind:1;

	uint8 bWindWasEnabled:1;

	/** Use gravity override value vs gravity scale */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsParameters, meta = (DisplayAfter = "LinearSpringConstant"))
	uint8 bUseGravityOverride:1;

	/** If true the gravity override value is defined in simulation space, by default it is in world space */
	UPROPERTY(EditAnywhere, Category = PhysicsParameters, meta=(DisplayAfter = "GravityOverride", DisplayName = "Gravity Override In Sim Space", EditCondition = "bUseGravityOverride"))
	uint8 bGravityOverrideInSimSpace : 1;

	/** If true the body will attempt to spring back to its initial position */
	UPROPERTY(EditAnywhere, Category = PhysicsParameters, meta = (InlineEditConditionToggle))
	uint8 bLinearSpring:1;

	/** If true the body will attempt to align itself with the specified angular target */
	UPROPERTY(EditAnywhere, Category = PhysicsParameters, meta = (InlineEditConditionToggle))
	uint8 bAngularSpring:1;

	/** Set to true to use the solver to simulate a connected chain */
	UPROPERTY(EditAnywhere, Category = Setup, meta=(InlineEditConditionToggle))
	uint8 bChain:1;

	/** The settings for rotation retargeting */
	UPROPERTY(EditAnywhere, Category = Retargeting)
	FRotationRetargetingInfo RetargetingSettings;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
		FVector BoxExtents_DEPRECATED;
	UPROPERTY()
		FVector LocalJointOffset_DEPRECATED;
	UPROPERTY()
		FAnimPhysConstraintSetup ConstraintSetup_DEPRECATED;
	UPROPERTY()
		AnimPhysCollisionType CollisionType_DEPRECATED;
	UPROPERTY()
		float SphereCollisionRadius_DEPRECATED;
#endif

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void UpdateInternal(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	virtual bool HasPreUpdate() const override { return true; }
	ANIMGRAPHRUNTIME_API virtual void PreUpdate(const UAnimInstance* InAnimInstance) override;
	virtual bool NeedsDynamicReset() const override { return true; }
	virtual void ResetDynamics(ETeleportType InTeleportType) override { RequestInitialise(InTeleportType); }
	ANIMGRAPHRUNTIME_API virtual int32 GetLODThreshold() const override;
	// End of FAnimNode_SkeletalControlBase interface

	ANIMGRAPHRUNTIME_API void RequestInitialise(ETeleportType InTeleportType);
	ANIMGRAPHRUNTIME_API void InitPhysics(FComponentSpacePoseContext& Output);
	ANIMGRAPHRUNTIME_API void TermPhysics();

	ANIMGRAPHRUNTIME_API void UpdateChainPhysicsBodyDefinitions(const FReferenceSkeleton& ReferenceSkeleton);
	ANIMGRAPHRUNTIME_API void ValidateChainPhysicsBodyDefinitions(const FReferenceSkeleton& ReferenceSkeleton);
	ANIMGRAPHRUNTIME_API void FindChainBoneNames(const FReferenceSkeleton& ReferenceSkeleton, TArray<FName>& ChainBoneNames);
	ANIMGRAPHRUNTIME_API void UpdateLimits(FComponentSpacePoseContext& Output);

	ANIMGRAPHRUNTIME_API int32 GetNumBodies() const;
	ANIMGRAPHRUNTIME_API const FAnimPhysRigidBody& GetPhysBody(int32 BodyIndex) const;

	ANIMGRAPHRUNTIME_API FTransform GetBodyComponentSpaceTransform(const FAnimPhysRigidBody& Body, const USkeletalMeshComponent* const SkelComp) const;

#if WITH_EDITOR

	// Accessors for editor code (mainly for visualization functions)
	ANIMGRAPHRUNTIME_API FVector GetBodyLocalJointOffset(const int32 BodyIndex) const;

	// True by default, if false physics simulation will not update this frame. Used to prevent the rig moving whilst interactively editing parameters with a widget in the viewport.
	bool bDoPhysicsUpdateInEditor;

#endif

	ANIMGRAPHRUNTIME_API bool ShouldDoPhysicsUpdate() const;

protected:

	// FAnimNode_SkeletalControlBase protected interface
	ANIMGRAPHRUNTIME_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	ANIMGRAPHRUNTIME_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones);
	// End of FAnimNode_SkeletalControlBase protected interface

private:
	// Given a bone index, get it's transform in the currently selected simulation space
	ANIMGRAPHRUNTIME_API FTransform GetBoneTransformInSimSpace(FComponentSpacePoseContext& Output, const FCompactPoseBoneIndex& BoneIndex) const;

	// Given a transform in simulation space, convert it back to component space
	ANIMGRAPHRUNTIME_API FTransform GetComponentSpaceTransformFromSimSpace(AnimPhysSimSpaceType SimSpace, FComponentSpacePoseContext& Output, const FTransform& InSimTransform) const;
	ANIMGRAPHRUNTIME_API FTransform GetComponentSpaceTransformFromSimSpace(AnimPhysSimSpaceType SimSpace, FComponentSpacePoseContext& Output, const FTransform& InSimTransform, const FTransform& InCompWorldSpaceTM, const FTransform& InActorWorldSpaceTM) const;
	ANIMGRAPHRUNTIME_API FTransform GetComponentSpaceTransformFromSimSpace(AnimPhysSimSpaceType SimSpace, const USkeletalMeshComponent* const SkelComp, const FTransform& InSimTransform) const;

	FTransform GetSimToWorldSpaceTransform(AnimPhysSimSpaceType SimSpace, FComponentSpacePoseContext& Output) const;

	// Given a transform in component space, convert it to the current sim space
	ANIMGRAPHRUNTIME_API FTransform GetSimSpaceTransformFromComponentSpace(AnimPhysSimSpaceType SimSpace, FComponentSpacePoseContext& Output, const FTransform& InComponentTransform) const;

	// Given a world-space vector, convert it into the current simulation space
	ANIMGRAPHRUNTIME_API FVector TransformWorldVectorToSimSpace(FComponentSpacePoseContext& Output, const FVector& InVec) const;

	ANIMGRAPHRUNTIME_API FVector TransformWorldVectorToSimSpaceScaled(FComponentSpacePoseContext& Output, const FVector& InVec) const;

	ANIMGRAPHRUNTIME_API void ConvertSimulationSpace(FComponentSpacePoseContext& Output, AnimPhysSimSpaceType From, AnimPhysSimSpaceType To) const;

	void CalculateSimulationSpaceMotion(AnimPhysSimSpaceType Space, FAnimPhysSimSpaceSettings Settings, FComponentSpacePoseContext& Output, const float Dt,
		FVector& SpaceAngularVel, FVector& SpaceAngularAcc);

	void UpdateBodyScaling(const int32 BodyIndex, float Scale) const;

	// Maximum time to consider when accumulating time debt to avoid spiraling
	static ANIMGRAPHRUNTIME_API const float MaxTimeDebt;

	// Cached timestep from the update phase (needed in evaluate phase)
	float NextTimeStep;

	// Current amount of time debt
	float TimeDebt;

	// Cached physics settings. We cache these on initialise to avoid the cost of accessing UPhysicsSettings a lot each frame
	float AnimPhysicsMinDeltaTime;
	float MaxPhysicsDeltaTime;
	float MaxSubstepDeltaTime;
	int32 MaxSubsteps;
	//////////////////////////////////////////////////////////////////////////

	// Active body list
	TArray<FAnimPhysLinkedBody> Bodies;

	// Pointers to bodies that need to be reset to their bound bone.
	// This happens on LOD change so we don't make the simulation unstable
	TArray<FAnimPhysLinkedBody*> BodiesToReset;

	// Pointers back to the base bodies to pass to the simulation
	TArray<FAnimPhysRigidBody*> BaseBodyPtrs;

	// List of current linear limits built for the current frame
	TArray<FAnimPhysLinearLimit> LinearLimits;

	// List of current angular limits built for the current frame
	TArray<FAnimPhysAngularLimit> AngularLimits;

	// List of spring force generators created for this frame
	TArray<FAnimPhysSpring> Springs;

	// Position of the physics object relative to the transform if its bound bone.
	TArray<FVector> PhysicsBodyJointOffsets;

	// A pair of positions (relative to their associated physics bodies) for each pair of bodies in a chain. These positions should be driven to match each other in sim space by the physics contstraints - See UpdateLimits() fns.
	TArray<FAnimConstraintOffsetPair> ConstraintOffsets;
	
	// Depending on the LOD we might not be running all of the bound bodies (for chains)
	// this tracks the active bodies.
	TArray<int32> ActiveBoneIndices;

	// Gravity direction in sim space
	FVector SimSpaceGravityDirection;

	// Previous linear velocity to resolve world accelerations when not using world space simulation
	FVector PreviousComponentLinearVelocity;

	// Previous angular velocity to resolve world angular acceleration when not using world space simulation
	FVector PreviousSimSpaceAngularVelocity;

	// When true, physics simulation is enabled.
	uint8 bEnabled : 1;

	// When true, input bone transform scale has changed since the last physics evaluation. Will trigger an update of physics body scales in the next physics evaluation.
	uint8 bShouldUpdateBodiesOnScaleChange : 1;

	//////////////////////////////////////////////////////////////////////////
	// Live debug
	//////////////////////////////////////////////////////////////////////////
#if ENABLE_ANIM_DRAW_DEBUG
	ANIMGRAPHRUNTIME_API void DrawBodies(FComponentSpacePoseContext& InContext, const TArray<FAnimPhysRigidBody*>& InBodies);

	int32 FilteredBoneIndex;
#endif
public: 
	static ANIMGRAPHRUNTIME_API bool IsAnimDynamicsSystemEnabledFor(int32 InLOD);
};
