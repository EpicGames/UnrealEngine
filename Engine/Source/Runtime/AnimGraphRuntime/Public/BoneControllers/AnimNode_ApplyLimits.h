// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneContainer.h"
#include "BonePose.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "AnimNode_ApplyLimits.generated.h"

class USkeletalMeshComponent;

USTRUCT()
struct FAngularRangeLimit
{
	GENERATED_BODY()

	FAngularRangeLimit()
		: LimitMin(-180, -180, -180)
		, LimitMax(+180, +180, +180)
	{}

	UPROPERTY(EditAnywhere, Category = Angular, meta = (UIMin = "-180", UIMax = "180", ClampMin = "-180", ClampMax = "180"))
	FVector LimitMin;

	UPROPERTY(EditAnywhere, Category = Angular, meta = (UIMin = "-180", UIMax = "180", ClampMin = "-180", ClampMax = "180"))
	FVector LimitMax;

	UPROPERTY(EditAnywhere, Category = Angular)
	FBoneReference Bone;
};

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_ApplyLimits : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY();

	ANIMGRAPHRUNTIME_API FAnimNode_ApplyLimits();

	UPROPERTY(EditAnywhere, Category = Angular)
	TArray<FAngularRangeLimit> AngularRangeLimits;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, EditFixedSize, Category = "Angular", meta = (PinHiddenByDefault))
	TArray<FVector> AngularOffsets;

	ANIMGRAPHRUNTIME_API void RecalcLimits();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	ANIMGRAPHRUNTIME_API virtual void OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance) override;
	virtual bool NeedsOnInitializeAnimInstance() const override { return true; }
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	ANIMGRAPHRUNTIME_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

private:
	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface
};
