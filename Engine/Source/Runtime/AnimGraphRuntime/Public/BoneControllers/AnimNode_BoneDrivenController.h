// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneContainer.h"
#include "BonePose.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "Animation/AnimTypes.h"
#include "AnimNode_BoneDrivenController.generated.h"

class UCurveFloat;
class USkeletalMeshComponent;

// Evaluation of the bone transforms relies on the size and ordering of this
// enum, if this needs to change make sure EvaluateSkeletalControl_AnyThread is updated.

// The type of modification to make to the destination component(s)
UENUM()
enum class EDrivenBoneModificationMode : uint8
{
	// Add the driven value to the input component value(s)
	AddToInput,

	// Replace the input component value(s) with the driven value
	ReplaceComponent,

	// Add the driven value to the reference pose value
	AddToRefPose
};

// Type of destination value to drive
UENUM()
enum class EDrivenDestinationMode : uint8
{	
	Bone,
	MorphTarget,
	MaterialParameter
};

/**
 * This is the runtime version of a bone driven controller, which maps part of the state from one bone to another (e.g., 2 * source.x -> target.z)
 */
USTRUCT()
struct FAnimNode_BoneDrivenController : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY()

	// Bone to use as controller input
	UPROPERTY(EditAnywhere, Category="Source (driver)")
	FBoneReference SourceBone;

	/** Curve used to map from the source attribute to the driven attributes if present (otherwise the Multiplier will be used) */
	UPROPERTY(EditAnywhere, Category=Mapping)
	TObjectPtr<UCurveFloat> DrivingCurve;

	// Multiplier to apply to the input value (Note: Ignored when a curve is used)
	UPROPERTY(EditAnywhere, Category=Mapping)
	float Multiplier;

	// Minimum limit of the input value (mapped to RemappedMin, only used when limiting the source range)
	// If this is rotation, the unit is radian
	UPROPERTY(EditAnywhere, Category=Mapping, meta=(EditCondition=bUseRange, DisplayName="Source Range Min"))
	double RangeMin;

	// Maximum limit of the input value (mapped to RemappedMax, only used when limiting the source range)
	// If this is rotation, the unit is radian
	UPROPERTY(EditAnywhere, Category=Mapping, meta=(EditCondition=bUseRange, DisplayName="Source Range Max"))
	double RangeMax;

	// Minimum value to apply to the destination (remapped from the input range)
	// If this is rotation, the unit is radian
	UPROPERTY(EditAnywhere, Category=Mapping, meta=(EditCondition=bUseRange, DisplayName="Mapped Range Min"))
	double RemappedMin;

	// Maximum value to apply to the destination (remapped from the input range)
	// If this is rotation, the unit is radian
	UPROPERTY(EditAnywhere, Category = Mapping, meta = (EditCondition = bUseRange, DisplayName="Mapped Range Max"))
	double RemappedMax;

	/** Name of Morph Target to drive using the source attribute */
	UPROPERTY(EditAnywhere, Category = "Destination (driven)")
	FName ParameterName;

	// Bone to drive using controller input
	UPROPERTY(EditAnywhere, Category="Destination (driven)")
	FBoneReference TargetBone;

#if WITH_EDITORONLY_DATA
private:
	UPROPERTY()
	TEnumAsByte<EComponentType::Type> TargetComponent_DEPRECATED;
#endif

public:
	// Type of destination to drive, currently either bone or morph target
	UPROPERTY(EditAnywhere, Category = "Destination (driven)")
	EDrivenDestinationMode DestinationMode;

	// The type of modification to make to the destination component(s)
	UPROPERTY(EditAnywhere, Category="Destination (driven)")
	EDrivenBoneModificationMode ModificationMode;

	// Transform component to use as input
	UPROPERTY(EditAnywhere, Category="Source (driver)")
	TEnumAsByte<EComponentType::Type> SourceComponent;

	// Whether or not to clamp the driver value and remap it before scaling it
	UPROPERTY(EditAnywhere, Category=Mapping, meta=(DisplayName="Remap Source"))
	uint8 bUseRange : 1;

	// Affect the X component of translation on the target bone
	UPROPERTY(EditAnywhere, Category="Destination (driven)", meta=(DisplayName="X"))
	uint8 bAffectTargetTranslationX : 1;

	// Affect the Y component of translation on the target bone
	UPROPERTY(EditAnywhere, Category="Destination (driven)", meta=(DisplayName="Y"))
	uint8 bAffectTargetTranslationY : 1;

	// Affect the Z component of translation on the target bone
	UPROPERTY(EditAnywhere, Category="Destination (driven)", meta=(DisplayName="Z"))
	uint8 bAffectTargetTranslationZ : 1;

	// Affect the X component of rotation on the target bone
	UPROPERTY(EditAnywhere, Category="Destination (driven)", meta=(DisplayName="X"))
	uint8 bAffectTargetRotationX : 1;

	// Affect the Y component of rotation on the target bone
	UPROPERTY(EditAnywhere, Category="Destination (driven)", meta=(DisplayName="Y"))
	uint8 bAffectTargetRotationY : 1;

	// Affect the Z component of rotation on the target bone
	UPROPERTY(EditAnywhere, Category="Destination (driven)", meta=(DisplayName="Z"))
	uint8 bAffectTargetRotationZ : 1;

	// Affect the X component of scale on the target bone
	UPROPERTY(EditAnywhere, Category = "Destination (driven)", meta=(DisplayName="X"))
	uint8 bAffectTargetScaleX : 1;

	// Affect the Y component of scale on the target bone
	UPROPERTY(EditAnywhere, Category = "Destination (driven)", meta=(DisplayName="Y"))
	uint8 bAffectTargetScaleY : 1;

	// Affect the Z component of scale on the target bone
	UPROPERTY(EditAnywhere, Category="Destination (driven)", meta=(DisplayName="Z"))
	uint8 bAffectTargetScaleZ : 1;

public:
	ANIMGRAPHRUNTIME_API FAnimNode_BoneDrivenController();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	ANIMGRAPHRUNTIME_API virtual void EvaluateComponentSpaceInternal(FComponentSpacePoseContext& Context);
	ANIMGRAPHRUNTIME_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

#if WITH_EDITORONLY_DATA
	// Upgrade a node from the output enum to the output bits (change made in FAnimationCustomVersion::BoneDrivenControllerMatchingMaya)
	ANIMGRAPHRUNTIME_API void ConvertTargetComponentToBits();
#endif

protected:
	
	// FAnimNode_SkeletalControlBase protected interface
	ANIMGRAPHRUNTIME_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;

	/** Extracts the value used to drive the target bone or parameter */
	ANIMGRAPHRUNTIME_API const double ExtractSourceValue(const FTransform& InCurrentBoneTransform, const FTransform& InRefPoseBoneTransform);
	// End of FAnimNode_SkeletalControlBase protected interface
};
