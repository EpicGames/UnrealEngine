// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNode_SkeletalControlBase.h"
#include "CCDIK.h"
#include "EngineDefines.h"
#include "AnimNode_CCDIK.generated.h"

/**
*	Controller which implements the CCDIK IK approximation algorithm
*/
USTRUCT()
struct FAnimNode_CCDIK : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY()

	/** Coordinates for target location of tip bone - if EffectorLocationSpace is bone, this is the offset from Target Bone to use as target location*/
	UPROPERTY(EditAnywhere, Category = Effector, meta = (PinShownByDefault))
	FVector EffectorLocation;

	/** Reference frame of Effector Transform. */
	UPROPERTY(EditAnywhere, Category = Effector)
	TEnumAsByte<enum EBoneControlSpace> EffectorLocationSpace;

	/** If EffectorTransformSpace is a bone, this is the bone to use. **/
	UPROPERTY(EditAnywhere, Category = Effector)
	FBoneSocketTarget EffectorTarget;

	/** Name of tip bone */
	UPROPERTY(EditAnywhere, Category = Solver)
	FBoneReference TipBone;

	/** Name of the root bone*/
	UPROPERTY(EditAnywhere, Category = Solver)
	FBoneReference RootBone;

	/** Tolerance for final tip location delta from EffectorLocation*/
	UPROPERTY(EditAnywhere, Category = Solver)
	float Precision;

	/** Maximum number of iterations allowed, to control performance. */
	UPROPERTY(EditAnywhere, Category = Solver)
	int32 MaxIterations;

	/** Toggle drawing of axes to debug joint rotation*/
	UPROPERTY(EditAnywhere, Category = Solver)
	bool bStartFromTail;

	/** Tolerance for final tip location delta from EffectorLocation*/
	UPROPERTY(EditAnywhere, Category = Solver)
	bool bEnableRotationLimit;

private:
	/** symmetry rotation limit per joint. Index 0 matches with root bone and last index matches with tip bone. */
	UPROPERTY(EditAnywhere, EditFixedSize, Category = Solver)
	TArray<float> RotationLimitPerJoints;

public:
	ANIMGRAPHRUNTIME_API FAnimNode_CCDIK();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	ANIMGRAPHRUNTIME_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

private:
	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	// Convenience function to get current (pre-translation iteration) component space location of bone by bone index
	FVector GetCurrentLocation(FCSPose<FCompactPose>& MeshBases, const FCompactPoseBoneIndex& BoneIndex);

	static FTransform GetTargetTransform(const FTransform& InComponentTransform, FCSPose<FCompactPose>& MeshBases, FBoneSocketTarget& InTarget, EBoneControlSpace Space, const FVector& InOffset);

public:
#if WITH_EDITOR
#if UE_ENABLE_DEBUG_DRAWING
	TArray<FVector> DebugLines;
#endif // #if UE_ENABLE_DEBUG_DRAWING
	// resize rotation limit array based on set up
	ANIMGRAPHRUNTIME_API void ResizeRotationLimitPerJoints(int32 NewSize);
#endif
};
