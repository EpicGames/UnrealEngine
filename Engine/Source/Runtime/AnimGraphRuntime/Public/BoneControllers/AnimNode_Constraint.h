// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "Constraint.h"
#include "EngineDefines.h"
#include "AnimNode_Constraint.generated.h"

/** 
 * Enum value to describe how you'd like to maintain offset
 */
UENUM()
enum class EConstraintOffsetOption : uint8
{
	None, // no offset
	Offset_RefPose // offset is based on reference pose
};


/** 
 * Constraint Set up
 *
 */
USTRUCT()
struct FConstraint
{
	GENERATED_USTRUCT_BODY()

	/** Target Bone this is constraint to */
	UPROPERTY(EditAnywhere, Category = FConstraint)
	FBoneReference TargetBone;

	/** Maintain offset based on refpose or not.
	 * 
	 * None - no offset
	 * Offset_RefPose - offset is created based on reference pose
	 * 
	 * In the future, we'd like to support custom offset, not just based on ref pose
	 */
	UPROPERTY(EditAnywhere, Category = FConstraint)
	EConstraintOffsetOption	OffsetOption;

	/** What transform type is constraint to - Translation, Rotation, Scale OR Parent. Parent overrides all component */
	UPROPERTY(EditAnywhere, Category = FConstraint)
	ETransformConstraintType TransformType;

	/** Per axis filter options - applied in their local space not in world space */
	UPROPERTY(EditAnywhere, Category = FConstraint)
	FFilterOptionPerAxis PerAxis;
	
	/** transient constraint data index */
	int32 ConstraintDataIndex;

	void Initialize(const FBoneContainer& RequiredBones)
	{
		TargetBone.Initialize(RequiredBones);
	}

	bool IsValidToEvaluate(const FBoneContainer& RequiredBones) const
	{
		return TargetBone.IsValidToEvaluate(RequiredBones) && PerAxis.IsValid();
	}

	FConstraint()
		: OffsetOption(EConstraintOffsetOption::Offset_RefPose)
		, TransformType(ETransformConstraintType::Translation)
		, ConstraintDataIndex(INDEX_NONE)
	{}
};
/**
 *	Constraint node to parent or world transform for rotation/translation
 */
USTRUCT()
struct FAnimNode_Constraint : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY()

	/** Name of bone to control. This is the main bone chain to modify from. **/
	UPROPERTY(EditAnywhere, Category = SkeletalControl) 
	FBoneReference BoneToModify;

	/** List of constraints */
	UPROPERTY(EditAnywhere, Category = Constraints)
	TArray<FConstraint> ConstraintSetup;

	/** Weight data - post edit syncs up to ConstraintSetups */
	UPROPERTY(EditAnywhere, editfixedsize, Category = Runtime, meta = (PinShownByDefault))
	TArray<float> ConstraintWeights;

	ANIMGRAPHRUNTIME_API FAnimNode_Constraint();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	ANIMGRAPHRUNTIME_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

#if WITH_EDITOR
	ANIMGRAPHRUNTIME_API void ConditionalDebugDraw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* MeshComp) const;
#endif // WITH_EDITOR

private:
	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	TArray<FConstraintData>	ConstraintData;

#if UE_ENABLE_DEBUG_DRAWING
	/** Debug draw cached data */
	FTransform CachedOriginalTransform;
	FTransform CachedConstrainedTransform;
	TArray<FTransform> CachedTargetTransforms;
#endif // UE_BUILD_SHIPPING
};
