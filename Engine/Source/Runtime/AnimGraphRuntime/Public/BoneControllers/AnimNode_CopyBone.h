// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneContainer.h"
#include "BonePose.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "AnimNode_CopyBone.generated.h"

class USkeletalMeshComponent;

/**
 *	Simple controller to copy a bone's transform to another one.
 */

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_CopyBone : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY()

	/** Source Bone Name to get transform from */
	UPROPERTY(EditAnywhere, Category = Copy)
	FBoneReference SourceBone;

	/** Name of bone to control. This is the main bone chain to modify from. **/
	UPROPERTY(EditAnywhere, Category=Copy) 
	FBoneReference TargetBone;

	/** If Translation should be copied */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Copy, meta=(PinShownByDefault))
	bool bCopyTranslation;

	/** If Rotation should be copied */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Copy, meta=(PinShownByDefault))
	bool bCopyRotation;

	/** If Scale should be copied */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Copy, meta=(PinShownByDefault))
	bool bCopyScale;

	/** Space to convert transforms into prior to copying components */
	UPROPERTY(EditAnywhere, Category = Copy)
	TEnumAsByte<EBoneControlSpace> ControlSpace;

	ANIMGRAPHRUNTIME_API FAnimNode_CopyBone();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	ANIMGRAPHRUNTIME_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

private:
	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface
};
