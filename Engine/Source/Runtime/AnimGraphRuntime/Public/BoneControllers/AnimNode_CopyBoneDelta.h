// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneContainer.h"
#include "BonePose.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "AnimNode_CopyBoneDelta.generated.h"

class USkeletalMeshComponent;

UENUM()
enum class CopyBoneDeltaMode : uint8
{
	Accumulate,
	Copy
};

/**
 *	Simple controller to copy a transform relative to the ref pose to the target bone,
 *	instead of the copy bone node which copies the absolute transform
 */

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_CopyBoneDelta : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FBoneReference SourceBone;

	UPROPERTY(EditAnywhere, Category = Settings)
	FBoneReference TargetBone;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinShownByDefault))
	bool bCopyTranslation;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinShownByDefault))
	bool bCopyRotation;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinShownByDefault))
	bool bCopyScale;

	UPROPERTY(EditAnywhere, Category = Settings)
	CopyBoneDeltaMode CopyMode;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinHiddenByDefault))
	float TranslationMultiplier;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinHiddenByDefault))
	float RotationMultiplier;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinHiddenByDefault))
	float ScaleMultiplier;

	ANIMGRAPHRUNTIME_API FAnimNode_CopyBoneDelta();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	ANIMGRAPHRUNTIME_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

private:
	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface
};
