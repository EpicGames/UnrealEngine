// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneIndices.h"
#include "BoneContainer.h"
#include "BonePose.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "AnimNode_Fabrik.generated.h"

class FPrimitiveDrawInterface;
class USkeletalMeshComponent;

/**
*	Controller which implements the FABRIK IK approximation algorithm -  see http://www.academia.edu/9165835/FABRIK_A_fast_iterative_solver_for_the_Inverse_Kinematics_problem for details
*/

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_Fabrik : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY()

	/** Coordinates for target location of tip bone - if EffectorLocationSpace is bone, this is the offset from Target Bone to use as target location*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = EndEffector, meta = (PinShownByDefault))
	FTransform EffectorTransform;

	/** If EffectorTransformSpace is a bone, this is the bone to use. **/
	UPROPERTY(EditAnywhere, Category = EndEffector)
	FBoneSocketTarget EffectorTarget;

	/** Name of tip bone */
	UPROPERTY(EditAnywhere, Category = Solver)
	FBoneReference TipBone;

	/** Name of the root bone*/
	UPROPERTY(EditAnywhere, Category = Solver)
	FBoneReference RootBone;

	/** Tolerance for final tip location delta from EffectorLocation*/
	UPROPERTY(EditAnywhere, Category = Solver)
	float Precision;

	/** Maximum number of iterations allowed, to control performance. */
	UPROPERTY(EditAnywhere, Category = Solver)
	int32 MaxIterations;

	/** Reference frame of Effector Transform. */
	UPROPERTY(EditAnywhere, Category = EndEffector)
	TEnumAsByte<enum EBoneControlSpace> EffectorTransformSpace;

	UPROPERTY(EditAnywhere, Category = EndEffector)
	TEnumAsByte<enum EBoneRotationSource> EffectorRotationSource;

#if WITH_EDITORONLY_DATA
	/** Toggle drawing of axes to debug joint rotation*/
	UPROPERTY(EditAnywhere, Category = Solver)
	bool bEnableDebugDraw;

	/** If EffectorTransformSpace is a bone, this is the bone to use. **/
	UPROPERTY()
	FBoneReference EffectorTransformBone_DEPRECATED;
#endif

public:
	ANIMGRAPHRUNTIME_API FAnimNode_Fabrik();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	ANIMGRAPHRUNTIME_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	ANIMGRAPHRUNTIME_API virtual void ConditionalDebugDraw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* PreviewSkelMeshComp) const;

private:
	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	// Convenience function to get current (pre-translation iteration) component space location of bone by bone index
	FVector GetCurrentLocation(FCSPose<FCompactPose>& MeshBases, const FCompactPoseBoneIndex& BoneIndex);
	static FTransform GetTargetTransform(const FTransform& InComponentTransform, FCSPose<FCompactPose>& MeshBases, FBoneSocketTarget& InTarget, EBoneControlSpace Space, const FTransform& InOffset);
#if WITH_EDITORONLY_DATA
	// Cached CS location when in editor for debug drawing
	FTransform CachedEffectorCSTransform;
#endif
};
