// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneContainer.h"
#include "BonePose.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "AnimNode_HandIKRetargeting.generated.h"

class USkeletalMeshComponent;

/**
 * Node to handle re-targeting of Hand IK bone chain.
 * It looks at position in Mesh Space of Left and Right FK bones, and moves Left and Right IK bones to those.
 * based on HandFKWeight. (0 = favor left hand, 1 = favor right hand, 0.5 = equal weight).
 * This is used so characters of different proportions can handle the same props.
 */

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_HandIKRetargeting : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY()

	/** Bone for Right Hand FK */
	UPROPERTY(EditAnywhere, Category = "HandIKRetargeting")
	FBoneReference RightHandFK;

	/** Bone for Left Hand FK */
	UPROPERTY(EditAnywhere, Category = "HandIKRetargeting")
	FBoneReference LeftHandFK;

	/** Bone for Right Hand IK */
	UPROPERTY(EditAnywhere, Category = "HandIKRetargeting")
	FBoneReference RightHandIK;

	/** Bone for Left Hand IK */
	UPROPERTY(EditAnywhere, Category = "HandIKRetargeting")
	FBoneReference LeftHandIK;

	/** IK Bones to move. */
	UPROPERTY(EditAnywhere, Category = "HandIKRetargeting")
	TArray<FBoneReference> IKBonesToMove;

	// Alpha values per axis to apply on the resulting retargeting translation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "HandIKRetargeting", meta = (PinHiddenByDefault))
	FVector PerAxisAlpha;

	/** Which hand to favor. 0.5 is equal weight for both, 1 = right hand, 0 = left hand. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "HandIKRetargeting", meta = (PinShownByDefault))
	float HandFKWeight;

	ANIMGRAPHRUNTIME_API FAnimNode_HandIKRetargeting();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	ANIMGRAPHRUNTIME_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

private:
	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface
};
