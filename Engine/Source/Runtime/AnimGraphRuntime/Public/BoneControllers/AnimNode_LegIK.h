// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneIndices.h"
#include "BoneContainer.h"
#include "BonePose.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "AnimNode_LegIK.generated.h"

class USkeletalMeshComponent;
struct FAnimLegIKData;

USTRUCT()
struct FIKChainLink
{
	GENERATED_USTRUCT_BODY()

public:
	FVector Location;
	double Length;
	FVector LinkAxisZ;
	FVector RealBendDir;
	FVector BaseBendDir;
	FName BoneName;

	FIKChainLink()
		: Location(FVector::ZeroVector)
		, Length(0.0)
		, LinkAxisZ(FVector::ZeroVector)
		, RealBendDir(FVector::ZeroVector)
		, BaseBendDir(FVector::ZeroVector)
		, BoneName(NAME_None)
	{}

	FIKChainLink(FVector InLocation, double InLength)
		: Location(InLocation)
		, Length(InLength)
		, LinkAxisZ(FVector::ZeroVector)
		, RealBendDir(FVector::ZeroVector)
		, BaseBendDir(FVector::ZeroVector)
		, BoneName(NAME_None)
	{}
};

USTRUCT()
struct FIKChain
{
	GENERATED_USTRUCT_BODY()

public:
	TArray<FIKChainLink> Links;
	float MinRotationAngleRadians;

private:
	FAnimInstanceProxy* MyAnimInstanceProxy;
	double TotalChainLength;
	int32 NumLinks;
	FVector HingeRotationAxis;
	bool bEnableRotationLimit;
	bool bInitialized;

public:
	FIKChain()
		: MyAnimInstanceProxy(nullptr)
		, TotalChainLength(0.0)
		, NumLinks(INDEX_NONE)
		, HingeRotationAxis(FVector::ZeroVector)
		, bEnableRotationLimit(false)
		, bInitialized(false)
	{}

	void InitializeFromLegData(FAnimLegIKData& InLegData, FAnimInstanceProxy* InAnimInstanceProxy);
	void ReachTarget(
		const FVector& InTargetLocation,
		double InReachPrecision,
		int32 InMaxIterations,
		float SoftPercentLength,
		float SoftAlpha);
	void ApplyTwistOffset(const float InTwistOffsetDegrees);

	double GetMaximumReach() const
	{
		return TotalChainLength;
	}

private:
	void OrientAllLinksToDirection(const FVector& InDirection);
	void SolveTwoBoneIK(const FVector& InTargetLocation);
	void SolveFABRIK(const FVector& InTargetLocation, double InReachPrecision, int32 InMaxIterations);

	static void FABRIK_ForwardReach(const FVector& InTargetLocation, FIKChain& IKChain);
	static void FABRIK_BackwardReach(const FVector& InRootTargetLocation, FIKChain& IKChain);
	static void FABRIK_ApplyLinkConstraints_Forward(FIKChain& IKChain, int32 LinkIndex);
	static void FABRIK_ApplyLinkConstraints_Backward(FIKChain& IKChain, int32 LinkIndex);

	static void DrawDebugIKChain(const FIKChain& IKChain, const FColor& InColor);
};

/** Per foot definitions */
USTRUCT()
struct FAnimLegIKDefinition
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference IKFootBone;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference FKFootBone;

	UPROPERTY(EditAnywhere, Category = "Settings")
	int32 NumBonesInLimb;

	/** Only used if bEnableRotationLimit is enabled. Prevents the leg from folding onto itself,
	* and forces at least this angle between Parent and Child bone. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float MinRotationAngle;

	/** Forward Axis for Foot bone. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	TEnumAsByte<EAxis::Type> FootBoneForwardAxis;

	/** Hinge Bones Rotation Axis. This is essentially the plane normal for (hip - knee - foot). */
	UPROPERTY(EditAnywhere, Category = "Settings")
	TEnumAsByte<EAxis::Type> HingeRotationAxis;

	/** If enabled, we prevent the leg from bending backwards and enforce a min compression angle */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bEnableRotationLimit;

	/** Enable Knee Twist correction, by comparing Foot FK with Foot IK orientation. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bEnableKneeTwistCorrection;

	/** Name of the curve to use as the twist offset angle(in degrees).
	* This is useful for injecting knee motion, while keeping the IK chain's goal/hand and root/hip locked in place. 
	* Reasonable values are usually between -+15 degrees, although this is depends on how far in/out the knee is in the original pose. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName TwistOffsetCurveName;

	FAnimLegIKDefinition()
		: NumBonesInLimb(2)
		, MinRotationAngle(15.f)
		, FootBoneForwardAxis(EAxis::Y)
		, HingeRotationAxis(EAxis::None)
		, bEnableRotationLimit(false)
		, bEnableKneeTwistCorrection(true)
		, TwistOffsetCurveName(NAME_None)
	{}
};

/** Runtime foot data after validation, we guarantee these bones to exist */
USTRUCT()
struct FAnimLegIKData
{
	GENERATED_USTRUCT_BODY()

public:
	FTransform IKFootTransform;
	FAnimLegIKDefinition* LegDefPtr;
	FCompactPoseBoneIndex IKFootBoneIndex;
	float TwistOffsetDegrees;
	int32 NumBones;
	TArray<FCompactPoseBoneIndex> FKLegBoneIndices;
	TArray<FTransform> FKLegBoneTransforms;

	FIKChain IKChain;

public:
	void InitializeTransforms(FAnimInstanceProxy* MyAnimInstanceProxy, FCSPose<FCompactPose>& MeshBases);

	FAnimLegIKData()
		: IKFootTransform(FTransform::Identity)
		, LegDefPtr(nullptr)
		, IKFootBoneIndex(INDEX_NONE)
		, TwistOffsetDegrees(0.0f)
		, NumBones(INDEX_NONE)
	{}
};

USTRUCT()
struct FAnimNode_LegIK : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY()

	ANIMGRAPHRUNTIME_API FAnimNode_LegIK();

	/** Tolerance for reaching IK Target, in unreal units. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float ReachPrecision;

	/** Max Number of Iterations. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	int32 MaxIterations;

	/** Default is 1.0 (off). Range is 0.1 to 1.0. When set to a value less than 1, will "softly" approach full extension starting when the effector
	 * distance from the root of the chain is greater than this percent length of the bone chain. Typical values are around 0.97.
	 * This is useful for preventing the knee from "popping" when approaching full extension. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PinHiddenByDefault, UIMin = "0.01", UIMax = "1", ClampMin = "0.01", ClampMax = "1"))
	float SoftPercentLength;

	/** Default is 1.0 (full). Range is 0 to 1. Blends the effect of the "softness" on/off. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PinHiddenByDefault, UIMin = "0.0", UIMax = "1", ClampMin = "0.0", ClampMax = "1"))
	float SoftAlpha;

	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FAnimLegIKDefinition> LegsDefinition;

	TArray<FAnimLegIKData> LegsData;

	FAnimInstanceProxy* MyAnimInstanceProxy;

public:
	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	ANIMGRAPHRUNTIME_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	ANIMGRAPHRUNTIME_API bool OrientLegTowardsIK(FAnimLegIKData& InLegData);
	ANIMGRAPHRUNTIME_API bool DoLegReachIK(FAnimLegIKData& InLegData);
	ANIMGRAPHRUNTIME_API bool AdjustKneeTwist(FAnimLegIKData& InLegData);

private:
	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface
};
