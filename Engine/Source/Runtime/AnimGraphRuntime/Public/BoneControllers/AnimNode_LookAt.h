// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneIndices.h"
#include "BoneContainer.h"
#include "BonePose.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "CommonAnimTypes.h"
#include "EngineDefines.h"
#include "AnimNode_LookAt.generated.h"

class FPrimitiveDrawInterface;
class USkeletalMeshComponent;

UENUM()
/** Various ways to interpolate TAlphaBlend. */
namespace EInterpolationBlend
{
	enum Type : int
	{
		Linear,
		Cubic,
		Sinusoidal,
		EaseInOutExponent2,
		EaseInOutExponent3,
		EaseInOutExponent4,
		EaseInOutExponent5,
		MAX
	};
}

/**
 *	Simple controller that make a bone to look at the point or another bone
 */
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_LookAt : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY()

	/** Name of bone to control. This is the main bone chain to modify from. **/
	UPROPERTY(EditAnywhere, Category=SkeletalControl) 
	FBoneReference BoneToModify;

	/** Target socket to look at. Used if LookAtBone is empty. - You can use  LookAtLocation if you need offset from this point. That location will be used in their local space. **/
	UPROPERTY(EditAnywhere, Category = Target)
	FBoneSocketTarget LookAtTarget;

	/** Target Offset. It's in world space if LookAtBone is empty or it is based on LookAtBone or LookAtSocket in their local space*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Target, meta = (PinHiddenByDefault))
	FVector LookAtLocation;

	UPROPERTY(EditAnywhere, Category = SkeletalControl)
	FAxis LookAt_Axis;

	/** Whether or not to use Look up axis */
	UPROPERTY(EditAnywhere, Category=SkeletalControl)
	bool bUseLookUpAxis;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SkeletalControl, meta=(PinHiddenByDefault))
	TEnumAsByte<EInterpolationBlend::Type>	InterpolationType;

	UPROPERTY(EditAnywhere, Category = SkeletalControl)
	FAxis LookUp_Axis;

	/** Look at Clamp value in degrees - if your look at axis is Z, only X, Y degree of clamp will be used */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SkeletalControl, meta=(PinHiddenByDefault))
	float LookAtClamp;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SkeletalControl, meta=(PinHiddenByDefault))
	float	InterpolationTime;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SkeletalControl, meta=(PinHiddenByDefault))
	float	InterpolationTriggerThreashold;

#if WITH_EDITORONLY_DATA
	/** Target Bone to look at - You can use  LookAtLocation if you need offset from this point. That location will be used in their local space. **/
	UPROPERTY()
	FBoneReference LookAtBone_DEPRECATED;

	UPROPERTY()
	FName LookAtSocket_DEPRECATED;

	/** Look at axis, which axis to align to look at point */
	UPROPERTY() 
	TEnumAsByte<EAxisOption::Type>	LookAtAxis_DEPRECATED;

	/** Custom look up axis in local space. Only used if LookAtAxis==EAxisOption::Custom */
	UPROPERTY()
	FVector	CustomLookAtAxis_DEPRECATED;

	/** Look up axis in local space */
	UPROPERTY()
	TEnumAsByte<EAxisOption::Type>	LookUpAxis_DEPRECATED;

	/** Custom look up axis in local space. Only used if LookUpAxis==EAxisOption::Custom */
	UPROPERTY()
	FVector	CustomLookUpAxis_DEPRECATED;
#endif

	// in the future, it would be nice to have more options, -i.e. lag, interpolation speed
	ANIMGRAPHRUNTIME_API FAnimNode_LookAt();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	ANIMGRAPHRUNTIME_API virtual void UpdateInternal(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void EvaluateComponentSpaceInternal(FComponentSpacePoseContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	ANIMGRAPHRUNTIME_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	FVector GetCachedTargetLocation() const {	return 	CachedCurrentTargetLocation;	}

#if WITH_EDITOR
	ANIMGRAPHRUNTIME_API void ConditionalDebugDraw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* MeshComp) const;
#endif // WITH_EDITOR

private:
	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	/** Turn a linear interpolated alpha into the corresponding AlphaBlendType */
	static ANIMGRAPHRUNTIME_API float AlphaToBlendType(float InAlpha, EInterpolationBlend::Type BlendType);

	/** Debug transient data */
	FVector CurrentLookAtLocation;

	/** Current Target Location */
	FVector CurrentTargetLocation;
	FVector PreviousTargetLocation;

	/** Current Alpha */
	float AccumulatedInterpoolationTime;


#if UE_ENABLE_DEBUG_DRAWING
	/** Debug draw cached data */
	FTransform CachedOriginalTransform;
	FTransform CachedLookAtTransform;
	FTransform CachedTargetCoordinate;
	FVector CachedPreviousTargetLocation;
	FVector CachedCurrentLookAtLocation;
#endif // UE_ENABLE_DEBUG_DRAWING
	FVector CachedCurrentTargetLocation;

protected:
	ANIMGRAPHRUNTIME_API virtual void ModifyPoseFromDeltaRotation(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms,	FTransform& InOutBoneToModifyTransform,	const FQuat& DeltaRotation);
};
