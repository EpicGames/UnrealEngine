// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneContainer.h"
#include "BonePose.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "AnimNode_ModifyBone.generated.h"

class USkeletalMeshComponent;

UENUM(BlueprintType)
enum EBoneModificationMode : int
{
	/** The modifier ignores this channel (keeps the existing bone translation, rotation, or scale). */
	BMM_Ignore UMETA(DisplayName = "Ignore"),

	/** The modifier replaces the existing translation, rotation, or scale. */
	BMM_Replace UMETA(DisplayName = "Replace Existing"),

	/** The modifier adds to the existing translation, rotation, or scale. */
	BMM_Additive UMETA(DisplayName = "Add to Existing")
};

/**
 *	Simple controller that replaces or adds to the translation/rotation of a single bone.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_ModifyBone : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY()

	/** Name of bone to control. This is the main bone chain to modify from. **/
	UPROPERTY(EditAnywhere, Category=SkeletalControl) 
	FBoneReference BoneToModify;

	/** New translation of bone to apply. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Translation, meta=(PinShownByDefault))
	FVector Translation;

	/** New rotation of bone to apply. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Rotation, meta=(PinShownByDefault))
	FRotator Rotation;

	/** New Scale of bone to apply. This is only worldspace. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Scale, meta=(PinShownByDefault))
	FVector Scale;

	/** Whether and how to modify the translation of this bone. */
	UPROPERTY(EditAnywhere, Category=Translation)
	TEnumAsByte<EBoneModificationMode> TranslationMode;

	/** Whether and how to modify the translation of this bone. */
	UPROPERTY(EditAnywhere, Category=Rotation)
	TEnumAsByte<EBoneModificationMode> RotationMode;

	/** Whether and how to modify the translation of this bone. */
	UPROPERTY(EditAnywhere, Category=Scale)
	TEnumAsByte<EBoneModificationMode> ScaleMode;

	/** Reference frame to apply Translation in. */
	UPROPERTY(EditAnywhere, Category=Translation)
	TEnumAsByte<enum EBoneControlSpace> TranslationSpace;

	/** Reference frame to apply Rotation in. */
	UPROPERTY(EditAnywhere, Category=Rotation)
	TEnumAsByte<enum EBoneControlSpace> RotationSpace;

	/** Reference frame to apply Scale in. */
	UPROPERTY(EditAnywhere, Category=Scale)
	TEnumAsByte<enum EBoneControlSpace> ScaleSpace;

	ANIMGRAPHRUNTIME_API FAnimNode_ModifyBone();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	ANIMGRAPHRUNTIME_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

private:
	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface
};
