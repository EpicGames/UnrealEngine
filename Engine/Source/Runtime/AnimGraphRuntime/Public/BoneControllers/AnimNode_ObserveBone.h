// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneContainer.h"
#include "BonePose.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "AnimNode_ObserveBone.generated.h"

class USkeletalMeshComponent;

/**
 *	Debugging node that displays the current value of a bone in a specific space.
 */
USTRUCT()
struct FAnimNode_ObserveBone : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY()

	/** Name of bone to observe. */
	UPROPERTY(EditAnywhere, Category=SkeletalControl)
	FBoneReference BoneToObserve;

	/** Reference frame to display the bone transform in. */
	UPROPERTY(EditAnywhere, Category=SkeletalControl)
	TEnumAsByte<EBoneControlSpace> DisplaySpace;

	/** Show the difference from the reference pose? */
	UPROPERTY(EditAnywhere, Category=SkeletalControl)
	bool bRelativeToRefPose;

	/** Translation of the bone being observed. */
	UPROPERTY()
	FVector Translation;

	/** Rotation of the bone being observed. */
	UPROPERTY()
	FRotator Rotation;

	/** Scale of the bone being observed. */
	UPROPERTY()
	FVector Scale;

public:
	ANIMGRAPHRUNTIME_API FAnimNode_ObserveBone();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	ANIMGRAPHRUNTIME_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

private:
	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface
};
