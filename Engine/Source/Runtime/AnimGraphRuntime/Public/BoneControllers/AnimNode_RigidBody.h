// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "Components/SkeletalMeshComponent.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsDeclares.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsProxy/PerSolverFieldSystem.h"
#include "Tasks/Task.h"
#include "AnimNode_RigidBody.generated.h"

struct FBodyInstance;
struct FConstraintInstance;
class FEvent;

extern ANIMGRAPHRUNTIME_API bool bEnableRigidBodyNode;
extern ANIMGRAPHRUNTIME_API FAutoConsoleVariableRef CVarEnableRigidBodyNode;
extern ANIMGRAPHRUNTIME_API TAutoConsoleVariable<int32> CVarEnableRigidBodyNodeSimulation;
extern ANIMGRAPHRUNTIME_API TAutoConsoleVariable<int32> CVarRigidBodyLODThreshold;

/** Determines in what space the simulation should run */
UENUM()
enum class ESimulationSpace : uint8
{
	/** Simulate in component space. Moving the entire skeletal mesh will have no affect on velocities */
	ComponentSpace,
	/** Simulate in world space. Moving the skeletal mesh will generate velocity changes */
	WorldSpace,
	/** Simulate in another bone space. Moving the entire skeletal mesh and individually modifying the base bone will have no affect on velocities */
	BaseBoneSpace,
};

/** Determines behaviour regarding deferral of simulation tasks. */
UENUM()
enum class ESimulationTiming : uint8
{
	/** Use the default project setting as defined by p.RigidBodyNode.DeferredSimulationDefault. */
	Default,
	/** Always run the simulation to completion during animation evaluation. */
	Synchronous,
	/** Always run the simulation in the background and retrieve the result on the next animation evaluation. */
	Deferred
};

/**
 * Settings for the system which passes motion of the simulation's space into the simulation. This allows the simulation to pass a 
 * fraction of the world space motion onto the bodies which allows Bone-Space and Component-Space simulations to react to world-space 
 * movement in a controllable way.
 */
template <> struct TIsPODType<FSimSpaceSettings> { enum { Value = true }; };

USTRUCT(BlueprintType)
struct FSimSpaceSettings
{
	GENERATED_USTRUCT_BODY()

	ANIMGRAPHRUNTIME_API FSimSpaceSettings();

	// Disable deprecation errors by providing defaults wrapped with pragma disable
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	~FSimSpaceSettings() = default;
	FSimSpaceSettings(FSimSpaceSettings const&) = default;
	FSimSpaceSettings& operator=(const FSimSpaceSettings &) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Global multipler on the effects of simulation space movement. Must be in range [0, 1]. If WorldAlpha = 0.0, the system is disabled and the simulation will
	// be fully local (i.e., world-space actor movement and rotation does not affect the simulation). When WorldAlpha = 1.0 the simulation effectively acts as a 
	// world-space sim, but with the ability to apply limits using the other parameters.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float WorldAlpha;

	// Multiplier on the Z-component of velocity and acceleration that is passed to the simulation. Usually from 0.0 to 1.0 to 
	// reduce the effects of jumping and crouching on the simulation, but it can be higher than 1.0 if you need to exaggerate this motion for some reason.
	// Should probably have been called "WorldAlphaScaleZ".
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ClampMin = "0.0"))
	float VelocityScaleZ;

	// A muliplier to control how much of the simulation space movement is used to calculate the drag forces from Linear/Angular Damping in the Physics Asset.
	// When DampingAlpha=1.0, Damping drag forces are equivalent to a world-space simulation. This is similar to air resistance.
	// When DampingAlpha=0.0, Damping drag forces depend only on local-space body velocity and not on the simulation space velocity.
	// It can be useful to set this to zero so that the Linear/Angular Damping settings on the BodyInstances do not contribute to air resistance. 
	// Air resistance can be re-added in a controlled way using the ExternalLinearDrag setting below.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ClampMin = "0.0"))
	float DampingAlpha;

	// A clamp on the effective world-space velocity that is passed to the simulation. Units are cm/s. The default value effectively means "unlimited". It is not usually required to
	// change this but you would reduce this to limit the effects of drag on the bodies in the simulation (if you have bodies that have LinearDrag set to non-zero in the physics asset). 
	// Expected values in this case would be somewhat less than the usual velocities of your object which is commonly a few hundred for a character.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ClampMin = "0.0"))
	float MaxLinearVelocity;

	// A clamp on the effective world-space angular velocity that is passed to the simulation. Units are radian/s, so a value of about 6.0 is one rotation per second.
	// The default value effectively means "unlimited". You would reduce this (and MaxAngularAcceleration) to limit how much bodies "fly out" when the actor spins on the spot. 
	// This is especially useful if you have characters than can rotate very quickly and you would probably want values around or less than 10 in this case.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ClampMin = "0.0"))
	float MaxAngularVelocity;
	
	// A clamp on the effective world-space acceleration that is passed to the simulation. Units are cm/s/s. The default value effectively means "unlimited". 
	// This property is used to stop the bodies of the simulation flying out when suddenly changing linear speed. It is useful when you have characters than can 
	// changes from stationary to running very quickly such as in an FPS. A common value for a character might be in the few hundreds.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ClampMin = "0.0"))
	float MaxLinearAcceleration;
	
	// A clamp on the effective world-space angular accleration that is passed to the simulation. Units are radian/s/s. The default value effectively means "unlimited". 
	// This has a similar effect to MaxAngularVelocity, except that it is related to the flying out of bodies when the rotation speed suddenly changes. Typical limist for
	// a character might be around 100.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ClampMin = "0.0"))
	float MaxAngularAcceleration;

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "ExternalLinearDrag is deprecated. Please use ExternalLinearDragV instead."))
	float ExternalLinearDrag_DEPRECATED;
#endif

	// Additional linear drag applied to each body based on total body velocity. This is in addition to per-body linear damping in the physics asset (but see DampingAlpha to control that).
	// (NOTE: The "V" suffix is to differentiate from the deprecated float property of the same name. It means "Vector" and not "Velocity").
	//
	// NOTE: ExternalLinearDragV is in simulation space, so if the RB AnimNode is set to Bone Space the ExternalLinearDragV.Z will be the drag in the
	// Up direction of the selected bone.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	FVector ExternalLinearDragV;

	// Additional velocity that is added to the component velocity so the simulation acts as if the actor is moving at speed, even when stationary. 
	// Vector is in world space. Units are cm/s. Could be used for a wind effects etc. Typical values are similar to the velocity of the object or effect, 
	// and usually around or less than 1000 for characters/wind.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	FVector ExternalLinearVelocity;

	// Additional angular velocity that is added to the component angular velocity. This can be used to make the simulation act as if the actor is rotating
	// even when it is not. E.g., to apply physics to a character on a podium as the camera rotates around it, to emulate the podium itself rotating.
	// Vector is in world space. Units are rad/s.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	FVector ExternalAngularVelocity;

	ANIMGRAPHRUNTIME_API void PostSerialize(const FArchive& Ar);
};

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<FSimSpaceSettings> : public TStructOpsTypeTraitsBase2<FSimSpaceSettings>
{
	enum
	{
		WithPostSerialize = true
	};
};
#endif


/**
 *	Controller that simulates physics based on the physics asset of the skeletal mesh component
 */
USTRUCT()
struct FAnimNode_RigidBody : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY()

	ANIMGRAPHRUNTIME_API FAnimNode_RigidBody();
	ANIMGRAPHRUNTIME_API ~FAnimNode_RigidBody();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void UpdateComponentPose_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void EvaluateComponentPose_AnyThread(FComponentSpacePoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	ANIMGRAPHRUNTIME_API virtual void OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance) override;
	virtual bool NeedsOnInitializeAnimInstance() const override { return true; }
	ANIMGRAPHRUNTIME_API virtual void PreUpdate(const UAnimInstance* InAnimInstance) override;
	ANIMGRAPHRUNTIME_API virtual void UpdateInternal(const FAnimationUpdateContext& Context) override;
	virtual bool HasPreUpdate() const override { return true; }
	ANIMGRAPHRUNTIME_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	ANIMGRAPHRUNTIME_API virtual bool NeedsDynamicReset() const override;
	ANIMGRAPHRUNTIME_API virtual void ResetDynamics(ETeleportType InTeleportType) override;
	ANIMGRAPHRUNTIME_API virtual int32 GetLODThreshold() const override;
	// End of FAnimNode_SkeletalControlBase interface

	ANIMGRAPHRUNTIME_API virtual void AddImpulseAtLocation(FVector Impulse, FVector Location, FName BoneName = NAME_None);

	// TEMP: Exposed for use in PhAt as a quick way to get drag handles working with Chaos
	virtual ImmediatePhysics::FSimulation* GetSimulation() { return PhysicsSimulation; }

	/**
	 * Set the override physics asset. This will automatically trigger a physics re-init in case the override physics asset changes. 
	 * Users can get access to this in the Animation Blueprint via the Animation Node Functions.
	 */
	void SetOverridePhysicsAsset(UPhysicsAsset* PhysicsAsset);

	UPhysicsAsset* GetPhysicsAsset() const { return UsePhysicsAsset; }

public:
	/** Physics asset to use. If empty use the skeletal mesh's default physics asset in case Default To Skeletal Mesh Physics Asset is set to True. */
	UPROPERTY(EditAnywhere, Category = Settings)
	TObjectPtr<UPhysicsAsset> OverridePhysicsAsset;

	/** Use the skeletal mesh physics asset as default in case set to True. The Override Physics Asset will always have priority over this. */
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bDefaultToSkeletalMeshPhysicsAsset = true;

	/** Treat parts in the Physics Asset with PhysicsType Default as Simulated for RBAN. */
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bUseDefaultAsSimulated = false;

private:
	/** Get the physics asset candidate to be used while respecting the bDefaultToSkeletalMeshPhysicsAsset and the priority to the override physics asset. */
	UPhysicsAsset* GetPhysicsAssetToBeUsed(const UAnimInstance* InAnimInstance) const;

	FTransform PreviousCompWorldSpaceTM;
	FTransform CurrentTransform;
	FTransform PreviousTransform;

	UPhysicsAsset* UsePhysicsAsset;

public:
	/** Enable if you want to ignore the p.RigidBodyLODThreshold CVAR and force the node to solely use the LOD threshold. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Performance, meta = (PinHiddenByDefault))
	bool bUseLocalLODThresholdOnly = false;

	/** Override gravity*/
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, editcondition = "bOverrideWorldGravity"))
	FVector OverrideWorldGravity;

	/** Applies a uniform external force in world space. This allows for easily faking inertia of movement while still simulating in component space for example */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinShownByDefault))
	FVector ExternalForce;

	/** When using non-world-space sim, this controls how much of the components world-space acceleration is passed on to the local-space simulation. */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	FVector ComponentLinearAccScale;

	/** When using non-world-space sim, this applies a 'drag' to the bodies in the local space simulation, based on the components world-space velocity. */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	FVector ComponentLinearVelScale;

	/** When using non-world-space sim, this is an overall clamp on acceleration derived from ComponentLinearAccScale and ComponentLinearVelScale, to ensure it is not too large. */
	UPROPERTY(EditAnywhere, Category = Settings)
	FVector	ComponentAppliedLinearAccClamp;

	/**
	 * Settings for the system which passes motion of the simulation's space
	 * into the simulation. This allows the simulation to pass a
	 * fraction of the world space motion onto the bodies which allows Bone-Space
	 * and Component-Space simulations to react to world-space movement in a
	 * controllable way.
	 * This system is a superset of the functionality provided by ComponentLinearAccScale,
	 * ComponentLinearVelScale, and ComponentAppliedLinearAccClamp. In general
	 * you should not have both systems enabled.
	 */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	FSimSpaceSettings SimSpaceSettings;


	/**
	 * Scale of cached bounds (vs. actual bounds).
	 * Increasing this may improve performance, but overlaps may not work as well.
	 * (A value of 1.0 effectively disables cached bounds).
	 */
	UPROPERTY(EditAnywhere, Category = CollisionSettings, meta = (ClampMin="1.0", ClampMax="2.0"))
	float CachedBoundsScale;

	/** Matters if SimulationSpace is BaseBone */
	UPROPERTY(EditAnywhere, Category = Settings)
	FBoneReference BaseBoneRef;

	/** The channel we use to find static geometry to collide with */
	UPROPERTY(EditAnywhere, Category = CollisionSettings, meta = (editcondition = "bEnableWorldGeometry"))
	TEnumAsByte<ECollisionChannel> OverlapChannel;

	/** What space to simulate the bodies in. This affects how velocities are generated */
	UPROPERTY(EditAnywhere, Category = Settings)
	ESimulationSpace SimulationSpace;

	/** Whether to allow collisions between two bodies joined by a constraint  */
	UPROPERTY(EditAnywhere, Category = CollisionSettings)
	bool bForceDisableCollisionBetweenConstraintBodies;

	/** If true, kinematic objects will be added to the simulation at runtime to represent any cloth colliders defined for the parent object. */
	UPROPERTY(EditAnywhere, Category = CollisionSettings)
	bool bUseExternalClothCollision;

private:
	ETeleportType ResetSimulatedTeleportType;

public:
	UPROPERTY(EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	uint8 bEnableWorldGeometry : 1;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	uint8 bOverrideWorldGravity : 1;

	/** 
		When simulation starts, transfer previous bone velocities (from animation)
		to make transition into simulation seamless.
	*/
	UPROPERTY(EditAnywhere, Category = Settings, meta=(PinHiddenByDefault))
	uint8 bTransferBoneVelocities : 1;

	/**
		When simulation starts, freeze incoming pose.
		This is useful for ragdolls, when we want the simulation to take over.
		It prevents non simulated bones from animating.
	*/
	UPROPERTY(EditAnywhere, Category = Settings)
	uint8 bFreezeIncomingPoseOnStart : 1;

	/**
		Correct for linear tearing on bodies with all axes Locked.
		This only works if all axes linear translation are locked
	*/
	UPROPERTY(EditAnywhere, Category = Settings)
	uint8 bClampLinearTranslationLimitToRefPose : 1;

	/**
		Allow / Prevent bone transform scales denfined in the input pose from being passed to the output pose. This setting is true by default 
		but will only have an effect if bone scale propagation is enabled project wide with the p.RigidBodyNode.EnablePoseScalePropagation 
		console variable. The details pannel checkbox will be hidden if the feature is not enabled.
	*/
	UPROPERTY(EditAnywhere, Category = Settings)
	uint8 bAllowPoseScalePropagation : 1;

	/**
		For world-space simulations, if the magnitude of the component's 3D scale is less than WorldSpaceMinimumScale, do not update the node.
	*/
	UPROPERTY(EditAnywhere, Category = Settings)
	float WorldSpaceMinimumScale;

	/**
		If the node is not evaluated for this amount of time (seconds), either because a lower LOD was in use for a while or the component was
		not visible, reset the simulation to the default pose on the next evaluation. Set to 0 to disable time-based reset.
	*/
	UPROPERTY(EditAnywhere, Category = Settings)
	float EvaluationResetTime;

	/**
	 * Expands the shape bounds to detect potentially active contacts when moving. Increasing this
	 * (typically up to 1) will increase the number of inactive contacts, which will reduce the
	 * likelihood of penetration, but will also increase solver cost.
	 */
	UPROPERTY(EditAnywhere, Category = CollisionSettings, meta = (ClampMin = 0))
	float BoundsVelocityMultiplier = 0.0f;

	/**
	 * The maximum margin added due to movement. Reducing this can prevent excessive numbers of inactive
	 * contacts being generated.
	 */
	UPROPERTY(EditAnywhere, Category = CollisionSettings, meta = (ClampMin = 0))
	float MaxVelocityBoundsExpansion = 0.0f;

	/**
	 * Allows the solver to use the EnableCCD setting on the bodies
	 */
	UPROPERTY(EditAnywhere, Category = CollisionSettings)
	bool bAllowCCD = false;

private:
	uint8 bEnabled : 1;
	uint8 bSimulationStarted : 1;
	uint8 bCheckForBodyTransformInit : 1;

public:
	ANIMGRAPHRUNTIME_API void PostSerialize(const FArchive& Ar);

private:

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool bComponentSpaceSimulation_DEPRECATED;	//use SimulationSpace
#endif

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	ANIMGRAPHRUNTIME_API void InitPhysics(const UAnimInstance* InAnimInstance);
	ANIMGRAPHRUNTIME_API void UpdateWorldGeometry(const UWorld& World, const USkeletalMeshComponent& SKC);
	ANIMGRAPHRUNTIME_API void UpdateWorldForces(const FTransform& ComponentToWorld, const FTransform& RootBoneTM, const float DeltaSeconds);

	ANIMGRAPHRUNTIME_API void InitializeNewBodyTransformsDuringSimulation(FComponentSpacePoseContext& Output, const FTransform& ComponentTransform, const FTransform& BaseBoneTM);

	ANIMGRAPHRUNTIME_API void InitSimulationSpace(
		const FTransform& ComponentToWorld,
		const FTransform& BoneToComponent);

	// Calculate simulation space transform, velocity etc to pass into the solver
	ANIMGRAPHRUNTIME_API void CalculateSimulationSpaceMotion(
		ESimulationSpace Space,
		const FTransform& SpaceTransform,
		const FTransform& ComponentToWorld,
		const float Dt,
		const FSimSpaceSettings& Settings,
		FVector& SpaceLinearVel,
		FVector& SpaceAngularVel,
		FVector& SpaceLinearAcc,
		FVector& SpaceAngularAcc);

	// Gather cloth collision sources from the supplied Skeletal Mesh and add a kinematic actor
	// representing each one of them to the sim.
	ANIMGRAPHRUNTIME_API void CollectClothColliderObjects(const USkeletalMeshComponent* SkeletalMeshComp);
	
	// Remove all cloth collider objects from the sim.
	ANIMGRAPHRUNTIME_API void RemoveClothColliderObjects();

	// Update the sim-space transforms of all cloth collider objects.
	ANIMGRAPHRUNTIME_API void UpdateClothColliderObjects(const FTransform& SpaceTransform);

	// Gather nearby world objects and add them to the sim
	ANIMGRAPHRUNTIME_API void CollectWorldObjects();

	// Flag invalid world objects to be removed from the sim
	ANIMGRAPHRUNTIME_API void ExpireWorldObjects();

	// Remove simulation objects that are flagged as expired
	ANIMGRAPHRUNTIME_API void PurgeExpiredWorldObjects();

	// Update sim-space transforms of world objects
	ANIMGRAPHRUNTIME_API void UpdateWorldObjects(const FTransform& SpaceTransform);

	// Advances the simulation by a given timestep
	ANIMGRAPHRUNTIME_API void RunPhysicsSimulation(float DeltaSeconds, const FVector& SimSpaceGravity);

	// Waits for the deferred simulation task to complete if it's not already finished
	ANIMGRAPHRUNTIME_API void FlushDeferredSimulationTask();

	// Destroy the simulation and free related structures
	ANIMGRAPHRUNTIME_API void DestroyPhysicsSimulation();

public:

	/* Whether the physics simulation runs synchronously with the node's evaluation or is run in the background until the next frame. */
	UPROPERTY(EditAnywhere, Category=Settings, AdvancedDisplay)
	ESimulationTiming SimulationTiming;

private:

	double WorldTimeSeconds;
	double LastEvalTimeSeconds;

	float AccumulatedDeltaTime;
	float AnimPhysicsMinDeltaTime;
	bool bSimulateAnimPhysicsAfterReset;
	/** This should only be used for removing the delegate during termination. Do NOT use this for any per frame work */
	TWeakObjectPtr<USkeletalMeshComponent> SkelMeshCompWeakPtr;

	ImmediatePhysics::FSimulation* PhysicsSimulation;
	FPhysicsAssetSolverSettings SolverSettings;
	FSolverIterations SolverIterations;	// to be deprecated

	friend class FRigidBodyNodeSimulationTask;
	UE::Tasks::FTask SimulationTask;

	struct FOutputBoneData
	{
		FOutputBoneData()
			: CompactPoseBoneIndex(INDEX_NONE)
		{}

		TArray<FCompactPoseBoneIndex> BoneIndicesToParentBody;
		FCompactPoseBoneIndex CompactPoseBoneIndex;
		int32 BodyIndex;
		int32 ParentBodyIndex;
	};

	struct FBodyAnimData
	{
		FBodyAnimData()
			: TransferedBoneAngularVelocity(ForceInit)
			, TransferedBoneLinearVelocity(ForceInitToZero)
			, LinearXMotion(ELinearConstraintMotion::LCM_Locked)
			, LinearYMotion(ELinearConstraintMotion::LCM_Locked)
			, LinearZMotion(ELinearConstraintMotion::LCM_Locked)
			, LinearLimit(0.0f)
			, RefPoseLength (0.f)
			, bIsSimulated(false)
			, bBodyTransformInitialized(false)
		{}

		FQuat TransferedBoneAngularVelocity;
		FVector TransferedBoneLinearVelocity;

		ELinearConstraintMotion LinearXMotion;
		ELinearConstraintMotion LinearYMotion;
		ELinearConstraintMotion LinearZMotion;
		float LinearLimit;
		// we don't use linear limit but use default length to limit the bodies
		// linear limits are defined per constraint - it can be any two joints that can limit
		// this is just default length of the local space from parent, and we use that info to limit
		// the translation
		float RefPoseLength;

		bool bIsSimulated : 1;
		bool bBodyTransformInitialized : 1;
	};

	struct FWorldObject
	{
		FWorldObject() : ActorHandle(nullptr), LastSeenTick(0), bExpired(false), bNew(true) {}
		FWorldObject(
			ImmediatePhysics::FActorHandle*     InActorHandle, 
			int32                               InLastSeenTick,
			TWeakObjectPtr<UPrimitiveComponent> InComponent) 
			: Component(InComponent), ActorHandle(InActorHandle), LastSeenTick(InLastSeenTick)
			, bExpired(false), bNew(true) {
		}

		TWeakObjectPtr<UPrimitiveComponent> Component;
		ImmediatePhysics::FActorHandle* ActorHandle;
		int32 LastSeenTick;
		uint8 bExpired : 1;
		uint8 bNew : 1;
	};

	TArray<FOutputBoneData> OutputBoneData;
	TArray<ImmediatePhysics::FActorHandle*> Bodies;
	TArray<int32> SkeletonBoneIndexToBodyIndex;
	TArray<FBodyAnimData> BodyAnimData;

	TArray<FPhysicsConstraintHandle*> Constraints;
	TArray<USkeletalMeshComponent::FPendingRadialForces> PendingRadialForces;

	FPerSolverFieldSystem PerSolverField;

	// Information required to identify and update a kinematic object representing a cloth collision source in the sim.
	struct FClothCollider
	{
		FClothCollider(ImmediatePhysics::FActorHandle* const InActorHandle, const USkeletalMeshComponent* const InSkeletalMeshComponent, const uint32 InBoneIndex)
			: ActorHandle(InActorHandle)
			, SkeletalMeshComponent(InSkeletalMeshComponent)
			, BoneIndex(InBoneIndex)
		{}

		ImmediatePhysics::FActorHandle* ActorHandle; // Identifies the physics actor in the sim.
		const USkeletalMeshComponent* SkeletalMeshComponent; // Parent skeleton.
		uint32 BoneIndex; // Bone within parent skeleton that drives physics actors transform.
	};

	// List of actors in the sim that represent objects collected from other parts of this character.
	TArray<FClothCollider> ClothColliders; 
	
	// List of actors in the sim that represent objects collected from the world. Note that the key
	// should never be dereferenced, as it is not a smart pointer, and might have been deleted.
	// Instead it should just be used for comparisons, and the actual pointer to the component is
	// stored in the WorldObject.
	TMap<const UPrimitiveComponent*, FWorldObject> ComponentsInSim;
	int32 ComponentsInSimTick;

	FVector WorldSpaceGravity;

	double TotalMass;

	// Bounds used to gather world objects copied into the simulation
	FSphere CachedBounds;

	FCollisionQueryParams QueryParams;

	FPhysScene* PhysScene;

	// Used by CollectWorldObjects and UpdateWorldGeometry in Task Thread
	// Typically, World should never be accessed off the Game Thread.
	// However, since we're just doing overlaps this should be OK.
	const UWorld* UnsafeWorld;

	// Used by CollectWorldObjects and UpdateWorldGeometry in Task Thread
	// Only used for a pointer comparison.
	const AActor* UnsafeOwner;

	FBoneContainer CapturedBoneVelocityBoneContainer;
	FCSPose<FCompactHeapPose> CapturedBoneVelocityPose;
	FCSPose<FCompactHeapPose> CapturedFrozenPose;
	FBlendedHeapCurve CapturedFrozenCurves;

	FVector PreviousComponentLinearVelocity;

	// Used by the world-space to simulation-space motion transfer system in Component- or Bone-Space sims
	FTransform PreviousSimulationSpaceTransform;
	FTransform PreviousPreviousSimulationSpaceTransform;
	float PreviousDt;

#if ENABLE_LOW_LEVEL_MEM_TRACKER
	FName OwningAssetPackageName;
	FName OwningAssetName;
#endif
};

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<FAnimNode_RigidBody> : public TStructOpsTypeTraitsBase2<FAnimNode_RigidBody>
{
	enum
	{
		WithPostSerialize = true,
	};
};
#endif

inline FTransform SpaceToWorldTransform(
	ESimulationSpace Space, const FTransform& ComponentToWorld, const FTransform& BaseBoneTM)
{
	switch (Space)
	{
	case ESimulationSpace::ComponentSpace: return ComponentToWorld;
	case ESimulationSpace::WorldSpace: return FTransform::Identity;
	case ESimulationSpace::BaseBoneSpace: return BaseBoneTM * ComponentToWorld;
	default: return FTransform::Identity;
	}
}

inline FVector WorldVectorToSpaceNoScale(
	ESimulationSpace Space, const FVector& WorldDir, const FTransform& ComponentToWorld, const FTransform& BaseBoneTM)
{
	switch (Space)
	{
	case ESimulationSpace::ComponentSpace: return ComponentToWorld.InverseTransformVectorNoScale(WorldDir);
	case ESimulationSpace::WorldSpace: return WorldDir;
	case ESimulationSpace::BaseBoneSpace:
		return BaseBoneTM.InverseTransformVectorNoScale(ComponentToWorld.InverseTransformVectorNoScale(WorldDir));
	default: return FVector::ZeroVector;
	}
}

inline FVector WorldPositionToSpace(
	ESimulationSpace Space, const FVector& WorldPoint, const FTransform& ComponentToWorld, const FTransform& BaseBoneTM)
{
	switch (Space)
	{
	case ESimulationSpace::ComponentSpace: return ComponentToWorld.InverseTransformPosition(WorldPoint);
	case ESimulationSpace::WorldSpace: return WorldPoint;
	case ESimulationSpace::BaseBoneSpace:
		return BaseBoneTM.InverseTransformPosition(ComponentToWorld.InverseTransformPosition(WorldPoint));
	default: return FVector::ZeroVector;
	}
}

inline FTransform ConvertCSTransformToSimSpace(
	ESimulationSpace SimSpace, const FTransform& InCSTransform, const FTransform& ComponentToWorld, const FTransform& BaseBoneTM)
{
	switch (SimSpace)
	{
	case ESimulationSpace::ComponentSpace: return InCSTransform;
	case ESimulationSpace::WorldSpace:  return InCSTransform * ComponentToWorld;
	case ESimulationSpace::BaseBoneSpace: return InCSTransform.GetRelativeTransform(BaseBoneTM); break;
	default: ensureMsgf(false, TEXT("Unsupported Simulation Space")); return InCSTransform;
	}
}

inline FVector ConvertCSVectorToSimSpace(
	ESimulationSpace SimSpace, const FVector& InCSVector, const FTransform& ComponentToWorld, const FTransform& BaseBoneTM)
{
	switch (SimSpace)
	{
	case ESimulationSpace::ComponentSpace: return InCSVector;
	case ESimulationSpace::WorldSpace:  return ComponentToWorld.TransformVectorNoScale(InCSVector);
	case ESimulationSpace::BaseBoneSpace: return BaseBoneTM.InverseTransformVectorNoScale(InCSVector); break;
	default: ensureMsgf(false, TEXT("Unsupported Simulation Space")); return InCSVector;
	}
}

inline FVector ConvertCSPositionToSimSpace(
	ESimulationSpace SimSpace, const FVector& InCSPosition, const FTransform& ComponentToWorld, const FTransform& BaseBoneTM)
{
	switch (SimSpace)
	{
	case ESimulationSpace::ComponentSpace: return InCSPosition;
	case ESimulationSpace::WorldSpace:  return ComponentToWorld.TransformPosition(InCSPosition);
	case ESimulationSpace::BaseBoneSpace: return BaseBoneTM.InverseTransformPosition(InCSPosition); break;
	default: ensureMsgf(false, TEXT("Unsupported Simulation Space")); return InCSPosition;
	}
}

ANIMGRAPHRUNTIME_API bool RigidBodyNodeIsPoseScalePropagationEnabledGlobally();
