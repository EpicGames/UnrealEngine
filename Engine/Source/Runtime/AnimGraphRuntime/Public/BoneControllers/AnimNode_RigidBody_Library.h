// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimNodeReference.h"
#include "BoneControllers/AnimNode_RigidBody.h"
#include "AnimNode_RigidBody_Library.generated.h"

USTRUCT(BlueprintType)
struct FRigidBodyAnimNodeReference : public FAnimNodeReference
{
	GENERATED_BODY()

	typedef FAnimNode_RigidBody FInternalNodeType;
};

// Exposes operations to be performed on a rigid body anim node
UCLASS(Experimental, MinimalAPI)
class UAnimNodeRigidBodyLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Get a rigid body anim node context from an anim node context */
	UFUNCTION(BlueprintCallable, Category = "Animation|Dynamics", meta = (BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static ANIMGRAPHRUNTIME_API FRigidBodyAnimNodeReference ConvertToRigidBodyAnimNode(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result);

	/** Get a rigid body anim node context from an anim node context (pure) */
	UFUNCTION(BlueprintPure, Category = "Animation|Dynamics", meta = (BlueprintThreadSafe, DisplayName = "Convert to rigid body"))
	static void ConvertToRigidBodyAnimNodePure(const FAnimNodeReference& Node, FRigidBodyAnimNodeReference& RigidBodyAnimNode, bool& Result);

	/** Set the physics asset on the rigid body anim graph node (RBAN). */
	UFUNCTION(BlueprintCallable, Category = "Animation|Dynamics", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FRigidBodyAnimNodeReference SetOverridePhysicsAsset(const FRigidBodyAnimNodeReference& Node, UPhysicsAsset* PhysicsAsset);
};
