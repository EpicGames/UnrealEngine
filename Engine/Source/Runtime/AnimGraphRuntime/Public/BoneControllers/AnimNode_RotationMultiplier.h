// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneContainer.h"
#include "BonePose.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "AnimNode_RotationMultiplier.generated.h"

class USkeletalMeshComponent;

/**
 *	Simple controller that multiplies scalar value to the translation/rotation/scale of a single bone.
 */

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_RotationMultiplier : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY()

	/** Name of bone to control. This is the main bone chain to modify from. */
	UPROPERTY(EditAnywhere, Category=Multiplier) 
	FBoneReference	TargetBone;

	/** Source to get transform from */
	UPROPERTY(EditAnywhere, Category=Multiplier)
	FBoneReference	SourceBone;

	// To make these to be easily pin-hookable, I'm not making it struct, but each variable
	// 0.f is invalid, and default
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Multiplier, meta=(PinShownByDefault))
	float	Multiplier;

	UPROPERTY(EditAnywhere, Category=Multiplier)
	TEnumAsByte<EBoneAxis> RotationAxisToRefer;
	
	UPROPERTY(EditAnywhere, Category = Multiplier)
	bool bIsAdditive;	

	ANIMGRAPHRUNTIME_API FAnimNode_RotationMultiplier();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	ANIMGRAPHRUNTIME_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

private:
	// Extract Delta Quat of rotation around Axis of animation and reference pose for the SourceBoneIndex
	FQuat ExtractAngle(const FTransform& RefPoseTransform, const FTransform& LocalBoneTransform, const EBoneAxis Axis);
	// Multiply scalar value Multiplier to the delta Quat of SourceBone Index's rotation
	FQuat MultiplyQuatBasedOnSourceIndex(const FTransform& RefPoseTransform, const FTransform& LocalBoneTransform, const EBoneAxis Axis, float InMultiplier, const FQuat& ReferenceQuat);

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface
};
