// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneIndices.h"
#include "BoneContainer.h"
#include "BonePose.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "AnimNode_ScaleChainLength.generated.h"

class USkeletalMeshComponent;

UENUM(BlueprintType)
enum class EScaleChainInitialLength : uint8
{
	/** Use the 'DefaultChainLength' input value. */
	FixedDefaultLengthValue,
	/** Use distance between 'ChainStartBone' and 'ChainEndBone' (in Component Space) */
	Distance,
	/* Use animated chain length (length in local space of every bone from 'ChainStartBone' to 'ChainEndBone' */
	ChainLength,
};

/**
 *	Scale the length of a chain of bones.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_ScaleChainLength : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = Links)
	FPoseLink InputPose;

	/** Default chain length, as animated. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ScaleChainLength, meta = (PinHiddenByDefault))
	float DefaultChainLength;

	UPROPERTY(EditAnywhere, Category = ScaleChainLength)
	FBoneReference ChainStartBone;

	UPROPERTY(EditAnywhere, Category = ScaleChainLength)
	FBoneReference ChainEndBone;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ScaleChainLength, meta = (PinShownByDefault))
	FVector TargetLocation;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinShownByDefault))
	float Alpha;

	float ActualAlpha;

	UPROPERTY(EditAnywhere, Category = Settings)
	FInputScaleBias AlphaScaleBias;

	UPROPERTY(EditAnywhere, Category = ScaleChainLength)
	EScaleChainInitialLength ChainInitialLength;

	bool bBoneIndicesCached;

	TArray<FCompactPoseBoneIndex> ChainBoneIndices;

public:
	ANIMGRAPHRUNTIME_API FAnimNode_ScaleChainLength();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

protected:
	ANIMGRAPHRUNTIME_API double GetInitialChainLength(FCompactPose& InLSPose, FCSPose<FCompactPose>& InCSPose) const;
};
