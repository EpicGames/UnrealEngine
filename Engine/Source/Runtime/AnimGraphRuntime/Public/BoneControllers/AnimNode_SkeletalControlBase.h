// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *	Abstract base class for a skeletal controller.
 *	A SkelControl is a module that can modify the position or orientation of a set of bones in a skeletal mesh in some programmatic way.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BonePose.h"
#include "Animation/BoneSocketReference.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/InputScaleBias.h"
#include "AnimNode_SkeletalControlBase.generated.h"

class USkeletalMeshComponent;

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_SkeletalControlBase : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()

	// Input link
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Links)
	FComponentSpacePoseLink ComponentPose;

	/*
	* Max LOD that this node is allowed to run
	* For example if you have LODThreshold to be 2, it will run until LOD 2 (based on 0 index)
	* when the component LOD becomes 3, it will stop update/evaluate
	* currently transition would be issue and that has to be re-visited
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Performance, meta = (PinHiddenByDefault, DisplayName = "LOD Threshold"))
	int32 LODThreshold;

	UPROPERTY(Transient)
	float ActualAlpha;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha)
	EAnimAlphaInputType AlphaInputType;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (PinShownByDefault, DisplayName = "bEnabled"))
	bool bAlphaBoolEnabled;

	// Current strength of the skeletal control
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (PinShownByDefault))
	float Alpha;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha)
	FInputScaleBias AlphaScaleBias;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (DisplayName = "Blend Settings"))
	FInputAlphaBoolBlend AlphaBoolBlend;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (PinShownByDefault))
	FName AlphaCurveName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha)
	FInputScaleBiasClamp AlphaScaleBiasClamp;

public:

	FAnimNode_SkeletalControlBase()
		: LODThreshold(INDEX_NONE)
		, ActualAlpha(0.f)
		, AlphaInputType(EAnimAlphaInputType::Float)
		, bAlphaBoolEnabled(true)
		, Alpha(1.0f)
	{
	}

	virtual ~FAnimNode_SkeletalControlBase() = default;

public:
#if WITH_EDITORONLY_DATA
	// forwarded pose data from the wired node which current node's skeletal control is not applied yet
	FCSPose<FCompactHeapPose> ForwardedPose;
#endif //#if WITH_EDITORONLY_DATA

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)  override;
	ANIMGRAPHRUNTIME_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) final;
	ANIMGRAPHRUNTIME_API virtual void EvaluateComponentSpace_AnyThread(FComponentSpacePoseContext& Output) final;
	virtual int32 GetLODThreshold() const override { return LODThreshold; }
	// End of FAnimNode_Base interface

	// Set the alpha of this node
	ANIMGRAPHRUNTIME_API void SetAlpha(float InAlpha);

	// Get the alpha of this node
	ANIMGRAPHRUNTIME_API float GetAlpha() const;

	ANIMGRAPHRUNTIME_API void InitializeAndValidateBoneRef(FBoneReference& BoneRef, const FBoneContainer& RequiredBones);

	// Visual warnings are shown on the node but not logged as an error for build system, use with care
	// The warnigns are cleared at CacheBones_AnyThread and should be added during InitializeBoneReferences
#if WITH_EDITOR
	ANIMGRAPHRUNTIME_API void AddBoneRefMissingVisualWarning(const FString& BoneName, const FString& SkeletalMeshName);
	ANIMGRAPHRUNTIME_API void AddValidationVisualWarning(FText ValidationVisualWarning);
	ANIMGRAPHRUNTIME_API bool HasValidationVisualWarnings() const;
	ANIMGRAPHRUNTIME_API void ClearValidationVisualWarnings();
	ANIMGRAPHRUNTIME_API FText GetValidationVisualWarningMessage() const;
#endif

protected:
	// Interface for derived skeletal controls to implement
	// use this function to update for skeletal control base
	ANIMGRAPHRUNTIME_API virtual void UpdateInternal(const FAnimationUpdateContext& Context);

	// Update incoming component pose.
	ANIMGRAPHRUNTIME_API virtual void UpdateComponentPose_AnyThread(const FAnimationUpdateContext& Context);

	// Evaluate incoming component pose.
	ANIMGRAPHRUNTIME_API virtual void EvaluateComponentPose_AnyThread(FComponentSpacePoseContext& Output);

	// use this function to evaluate for skeletal control base
	ANIMGRAPHRUNTIME_API virtual void EvaluateComponentSpaceInternal(FComponentSpacePoseContext& Context);
	// Evaluate the new component-space transforms for the affected bones.
	ANIMGRAPHRUNTIME_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms);
	// return true if it is valid to Evaluate
	virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) { return false; }
	// initialize any bone references you have
	virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones){};

	/** Allow base to add info to the node debug output */
	ANIMGRAPHRUNTIME_API void AddDebugNodeData(FString& OutDebugData);

private:
	// Resused bone transform array to avoid reallocating in skeletal controls
	TArray<FBoneTransform> BoneTransforms;

#if WITH_EDITORONLY_DATA
	UPROPERTY(transient)
	FText ValidationVisualWarningMessage;
#endif

};
