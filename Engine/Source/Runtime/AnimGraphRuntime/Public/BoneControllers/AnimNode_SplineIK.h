// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "Components/SplineComponent.h"
#include "AlphaBlend.h"
#include "AnimNode_SplineIK.generated.h"

/** 
 * The different axes we can align our bones to.
 * Note that the values are set to match up with EAxis (but without 'None')
 */
UENUM()
enum class ESplineBoneAxis : uint8
{
	None = 0 UMETA(Hidden),
	X = 1,
	Y = 2,
	Z = 3,
};

/** Data cached per bone in the chain */
USTRUCT()
struct FSplineIKCachedBoneData
{
	GENERATED_BODY()

	FSplineIKCachedBoneData()
		: Bone(NAME_None)
		, RefSkeletonIndex(INDEX_NONE)
	{}

	FSplineIKCachedBoneData(const FName& InBoneName, int32 InRefSkeletonIndex)
		: Bone(InBoneName)
		, RefSkeletonIndex(InRefSkeletonIndex)
	{}

	/** The bone we refer to */
	UPROPERTY()
	FBoneReference Bone;

	/** Index of the bone in the reference skeleton */
	UPROPERTY()
	int32 RefSkeletonIndex;
};

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_SplineIK : public FAnimNode_SkeletalControlBase
{
	GENERATED_BODY()

	/** Name of root bone from which the spline extends **/
	UPROPERTY(EditAnywhere, Category = "Parameters")
	FBoneReference StartBone;

	/** Name of bone at the end of the spline chain. Bones after this will not be altered by the controller. */
	UPROPERTY(EditAnywhere, Category = "Parameters")
	FBoneReference EndBone;

	/** Axis of the controlled bone (ie the direction of the spline) to use as the direction for the curve. */
	UPROPERTY(EditAnywhere, Category = "Parameters")
	ESplineBoneAxis BoneAxis;

	/** The number of points in the spline if we are specifying it directly */
	UPROPERTY(EditAnywhere, Category = "Parameters")
	bool bAutoCalculateSpline;

	/** The number of points in the spline if we are not auto-calculating */
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (ClampMin = 2, UIMin = 2, EditCondition = "!bAutoCalculateSpline"))
	int32 PointCount;

	/** Transforms applied to spline points **/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, EditFixedSize, Category = "Parameters", meta = (BlueprintCompilerGeneratedDefaults, PinHiddenByDefault))
	TArray<FTransform> ControlPoints;

	/** Overall roll of the spline, applied on top of other rotations along the direction of the spline */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters", meta = (PinHiddenByDefault))
	float Roll;

	/** The twist of the start bone. Twist is interpolated along the spline according to Twist Blend. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters", meta = (PinHiddenByDefault))
	float TwistStart;

	/** The twist of the end bone. Twist is interpolated along the spline according to Twist Blend. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters", meta = (PinHiddenByDefault))
	float TwistEnd;

	/** How to interpolate twist along the length of the spline */
	UPROPERTY(EditAnywhere, Category = "Parameters")
	FAlphaBlend TwistBlend;

	/**
	 * The maximum stretch allowed when fitting bones to the spline. 0.0 means bones do not stretch their length,
	 * 1.0 means bones stretch to the length of the spline
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters", meta = (PinHiddenByDefault))
	float Stretch;

	/** The distance along the spline from the start from which bones are constrained */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters", meta = (PinHiddenByDefault))
	float Offset;

	ANIMGRAPHRUNTIME_API FAnimNode_SplineIK();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	ANIMGRAPHRUNTIME_API virtual void OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance) override;
	virtual bool NeedsOnInitializeAnimInstance() const override { return true; }
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	ANIMGRAPHRUNTIME_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	/** Read-only access to spline curves */
	const FSplineCurves& GetSplineCurves() const { return BoneSpline; }

	/** Read-only access to transformed curves */
	const FSplineCurves& GetTransformedSplineCurves() const { return TransformedSpline; }

	/** Get transformed spline point (in component space) for the spline */
	ANIMGRAPHRUNTIME_API FTransform GetTransformedSplinePoint(int32 TransformIndex) const;

	/** Get specified handle transform (in component space) for the spline */
	ANIMGRAPHRUNTIME_API FTransform GetControlPoint(int32 TransformIndex) const;

	/** Set specified handle transform (in component space) for the spline */
	ANIMGRAPHRUNTIME_API void SetControlPoint(int32 TransformIndex, const FTransform& InTransform);

	/** Set specified handle location (in component space) for the spline */
	ANIMGRAPHRUNTIME_API void SetControlPointLocation(int32 TransformIndex, const FVector& InLocation);

	/** Set specified handle rotation (in component space) for the spline */
	ANIMGRAPHRUNTIME_API void SetControlPointRotation(int32 TransformIndex, const FQuat& InRotation);

	/** Set specified handle scale (in component space) for the spline */
	ANIMGRAPHRUNTIME_API void SetControlPointScale(int32 TransformIndex, const FVector& InScale);

	/** Get the number of spline transforms we are using */
	int32 GetNumControlPoints() const { return ControlPoints.Num(); }

	/** Build bone references & reallocate transforms from the supplied ref skeleton */
	ANIMGRAPHRUNTIME_API void GatherBoneReferences(const FReferenceSkeleton& RefSkeleton);

protected:
	/** Build spline from reference pose */
	ANIMGRAPHRUNTIME_API void BuildBoneSpline(const FReferenceSkeleton& RefSkeleton);

protected:
	/** Transform the spline using our control points */
	ANIMGRAPHRUNTIME_API void TransformSpline();

	/** Use our linear approximation to determine the earliest intersection with a sphere */
	ANIMGRAPHRUNTIME_API float FindParamAtFirstSphereIntersection(const FVector& InOrigin, float InRadius, int32& StartingLinearIndex);

private:
	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	/** Get the current twist value at the specified spline alpha */
	ANIMGRAPHRUNTIME_API float GetTwist(float InAlpha, float TotalSplineAlpha);

	/** Transformed spline */
	FSplineCurves TransformedSpline;

	/** Piecewise linear approximation of the spline, recalculated on creation and deformation */
	TArray<FSplinePositionLinearApproximation> LinearApproximation;

	/** Spline we maintain internally */
	FSplineCurves BoneSpline;

	/** Cached spline length from when the spline was originally applied to the skeleton */
	float OriginalSplineLength;

	/** Cached data for bones in the IK chain, from start to end */
	TArray<FSplineIKCachedBoneData> CachedBoneReferences;

	/** Cached bone lengths. Same size as CachedBoneReferences */
	TArray<float> CachedBoneLengths;

	/** Cached bone offset rotations. Same size as CachedBoneReferences */
	TArray<FQuat> CachedOffsetRotations;
};
