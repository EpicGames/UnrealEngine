// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneContainer.h"
#include "BonePose.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "AnimNode_SpringBone.generated.h"

class UAnimInstance;
class USkeletalMeshComponent;

/**
 *	Simple controller that replaces or adds to the translation/rotation of a single bone.
 */

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_SpringBone : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY()

	/** Name of bone to control. This is the main bone chain to modify from. **/
	UPROPERTY(EditAnywhere, Category=Spring) 
	FBoneReference SpringBone;

	/** If bLimitDisplacement is true, this indicates how long a bone can stretch beyond its length in the ref-pose. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Spring, meta=(EditCondition="bLimitDisplacement"))
	double MaxDisplacement;

	/** Stiffness of spring */
	UPROPERTY(EditAnywhere, Category=Spring, meta = (PinHiddenByDefault))
	double SpringStiffness;

	/** Damping of spring */
	UPROPERTY(EditAnywhere, Category=Spring, meta = (PinHiddenByDefault))
	double SpringDamping;

	/** If spring stretches more than this, reset it. Useful for catching teleports etc */
	UPROPERTY(EditAnywhere, Category=Spring)
	double ErrorResetThresh;

	/** World-space location of the bone. */
	FVector BoneLocation;

	/** World-space velocity of the bone. */
	FVector BoneVelocity;

	/** Velocity of the owning actor */
	FVector OwnerVelocity;

	/** Cache of previous frames local bone transform so that when
	 *  we cannot simulate (timestep == 0) we can still update bone location */
	FVector LocalBoneTransform;

	/** Internal use - Amount of time we need to simulate. */
	float RemainingTime;

	/** Internal use - Current timestep */
	float FixedTimeStep;

	/** Internal use - Current time dilation */
	float TimeDilation;

#if WITH_EDITORONLY_DATA
	/** If true, Z position is always correct, no spring applied */
	UPROPERTY()
	bool bNoZSpring_DEPRECATED;
#endif

	/** Limit the amount that a bone can stretch from its ref-pose length. */
	UPROPERTY(EditAnywhere, Category=Spring)
	uint8 bLimitDisplacement : 1;

	/** If true take the spring calculation for translation in X */
	UPROPERTY(EditAnywhere, Category=FilterChannels)
	uint8 bTranslateX : 1;

	/** If true take the spring calculation for translation in Y */
	UPROPERTY(EditAnywhere, Category=FilterChannels)
	uint8 bTranslateY : 1;

	/** If true take the spring calculation for translation in Z */
	UPROPERTY(EditAnywhere, Category=FilterChannels)
	uint8 bTranslateZ : 1;

	/** If true take the spring calculation for rotation in X */
	UPROPERTY(EditAnywhere, Category=FilterChannels)
	uint8 bRotateX : 1;

	/** If true take the spring calculation for rotation in Y */
	UPROPERTY(EditAnywhere, Category=FilterChannels)
	uint8 bRotateY : 1;

	/** If true take the spring calculation for rotation in Z */
	UPROPERTY(EditAnywhere, Category=FilterChannels)
	uint8 bRotateZ : 1;

	/** Did we have a non-zero ControlStrength last frame. */
	uint8 bHadValidStrength : 1;

	/** Option to override owner velocity used by spring controller. */
	UPROPERTY(EditAnywhere, Category = OwnerVelocity, Meta = (PinHiddenByDefault))
	uint8 bOverrideOwnerVelocity : 1;

	UPROPERTY(EditAnywhere, Category = OwnerVelocity, Meta = (EditCondition = "bOverrideOwnerVelocity", PinHiddenByDefault))
	FVector OwnerVelocityOverride;

public:
	ANIMGRAPHRUNTIME_API FAnimNode_SpringBone();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void UpdateInternal(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	virtual bool HasPreUpdate() const override { return true; }
	ANIMGRAPHRUNTIME_API virtual void PreUpdate(const UAnimInstance* InAnimInstance) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	ANIMGRAPHRUNTIME_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

private:
	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface
};
