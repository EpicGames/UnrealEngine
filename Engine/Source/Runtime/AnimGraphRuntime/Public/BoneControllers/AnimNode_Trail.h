// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneContainer.h"
#include "Curves/CurveFloat.h"
#include "BonePose.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "BoneControllers/AnimNode_AnimDynamics.h"
#include "AnimNode_Trail.generated.h"

class USkeletalMeshComponent;

// in the future, we might use this for stretch set up as well
// for now this is unserializable, and transient only
struct FPerJointTrailSetup
{
	/** How quickly we 'relax' the bones to their animated positions. */
	float	TrailRelaxationSpeedPerSecond;
};

USTRUCT()
struct FRotationLimit
{
	GENERATED_BODY()

	FRotationLimit()
		: LimitMin(-180, -180, -180)
		, LimitMax(+180, +180, +180)
	{}

	UPROPERTY(EditAnywhere, Category = Angular, meta = (UIMin = "-180", UIMax = "180", ClampMin = "-180", ClampMax = "180"))
	FVector LimitMin;

	UPROPERTY(EditAnywhere, Category = Angular, meta = (UIMin = "-180", UIMax = "180", ClampMin = "-180", ClampMax = "180"))
	FVector LimitMax;
};

/**
 * Trail Controller
 */

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_Trail : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY()

	/** LocalToWorld used last frame, used for building transform between frames. */
	FTransform		OldBaseTransform;

	/** Reference to the active bone in the hierarchy to modify. */
	UPROPERTY(EditAnywhere, Category=Trail)
	FBoneReference TrailBone;

	/** Number of bones above the active one in the hierarchy to modify. ChainLength should be at least 2. */
	UPROPERTY(EditAnywhere, Category = Trail, meta = (ClampMin = "2", UIMin = "2"))
	int32	ChainLength;

	/** Axis of the bones to point along trail. */
	UPROPERTY(EditAnywhere, Category=Trail)
	TEnumAsByte<EAxis::Type>	ChainBoneAxis;

	/** Invert the direction specified in ChainBoneAxis. */
	UPROPERTY(EditAnywhere, Category=Trail)
	uint8 bInvertChainBoneAxis:1;

	/** Limit the amount that a bone can stretch from its ref-pose length. */
	UPROPERTY(EditAnywhere, Category=Limit)
	uint8 bLimitStretch:1;

	/** Limit the amount that a bone can stretch from its ref-pose length. */
	UPROPERTY(EditAnywhere, Category = Limit)
	uint8 bLimitRotation:1;

	/** Whether to evaluate planar limits */
	UPROPERTY(EditAnywhere, Category=Limit)
	uint8 bUsePlanarLimit:1;

	/** Whether 'fake' velocity should be applied in actor or world space. */
	UPROPERTY(EditAnywhere,  Category=Velocity)
	uint8 bActorSpaceFakeVel:1;

	/** Fix up rotation to face child for the parent*/
	UPROPERTY(EditAnywhere, Category = Rotation)
	uint8 bReorientParentToChild:1;

	/** Did we have a non-zero ControlStrength last frame. */
	uint8 bHadValidStrength:1;

#if WITH_EDITORONLY_DATA
	/** Enable Debug in the PIE. This doesn't work in game*/
	UPROPERTY(EditAnywhere, Category = Debug)
	uint8 bEnableDebug:1;

	/** Show Base Motion */
	UPROPERTY(EditAnywhere, Category = Debug)
	uint8 bShowBaseMotion:1;

	/** Show Trail Location **/
	UPROPERTY(EditAnywhere, Category = Debug)
	uint8 bShowTrailLocation:1;

	/** Show Planar Limits **/
	UPROPERTY(EditAnywhere, Category = Debug)
	uint8 bShowLimit:1;

	/** This is used by selection node. Use this transient flag. */
	uint8 bEditorDebugEnabled:1;

	/** Debug Life Time */
	UPROPERTY(EditAnywhere, Category = Debug)
	float DebugLifeTime;

	/** How quickly we 'relax' the bones to their animated positions. Deprecated. Replaced to TrailRelaxationCurve */
	UPROPERTY()
	float TrailRelaxation_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

	/** To avoid hitches causing stretch of trail, you can use MaxDeltaTime to clamp the long delta time. If you want 30 fps to set it to 0.03333f ( = 1/30 ).  */
	UPROPERTY(EditAnywhere, Category = Limit)
	float MaxDeltaTime;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Trail, meta = (PinHiddenByDefault))
	float RelaxationSpeedScale;

	/** How quickly we 'relax' the bones to their animated positions. Time 0 will map to top root joint, time 1 will map to the bottom joint. */
	UPROPERTY(EditAnywhere, Category=Trail, meta=(CustomizeProperty))
	FRuntimeFloatCurve TrailRelaxationSpeed;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Trail)
	FInputScaleBiasClamp RelaxationSpeedScaleInputProcessor;

	UPROPERTY(EditAnywhere, EditFixedSize, Category = Limit, meta = (EditCondition =bLimitRotation))
	TArray<FRotationLimit> RotationLimits;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, EditFixedSize, Category = Limit, meta = (PinHiddenByDefault, EditCondition = bLimitRotation))
	TArray<FVector> RotationOffsets;

	/** List of available planar limits for this node */
	UPROPERTY(EditAnywhere, Category=Limit, meta = (EditCondition = bUsePlanarLimit))
	TArray<FAnimPhysPlanarLimit> PlanarLimits;

	/** If bLimitStretch is true, this indicates how long a bone can stretch beyond its length in the ref-pose. */
	UPROPERTY(EditAnywhere, Category=Limit)
	float	StretchLimit;

	/** 'Fake' velocity applied to bones. */
	UPROPERTY(EditAnywhere, Category=Velocity, meta = (PinHiddenByDefault))
	FVector	FakeVelocity;

	/** Base Joint to calculate velocity from. If none, it will use Component's World Transform. . */
	UPROPERTY(EditAnywhere, Category=Velocity)
	FBoneReference BaseJoint;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	float TrailBoneRotationBlendAlpha_DEPRECATED;
#endif // WITH_EDITORONLY_DATA
	/* How to set last bone rotation. It copies from previous joint if alpha is 0.f, or 1.f will use animated pose 
	 * This alpha dictates the blend between parent joint and animated pose, and how much you'd like to use animated pose for
	 */
	UPROPERTY(EditAnywhere, Category = Rotation, meta = (EditCondition = bReorientParentToChild))
	float LastBoneRotationAnimAlphaBlend;

	/** Internal use - we need the timestep to do the relaxation in CalculateNewBoneTransforms. */
	float	ThisTimstep;

	/** Component-space locations of the bones from last frame. Each frame these are moved towards their 'animated' locations. */
	TArray<FVector>	TrailBoneLocations;

	/** Per Joint Trail Set up*/
	TArray<FPerJointTrailSetup> PerJointTrailData;

#if WITH_EDITORONLY_DATA
	/** debug transient data to draw debug better */
	TArray<FColor>	TrailDebugColors;
	TArray<FColor>	PlaneDebugColors;
#endif // WITH_EDITORONLY_DATA

	ANIMGRAPHRUNTIME_API FAnimNode_Trail();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void UpdateInternal(const FAnimationUpdateContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	ANIMGRAPHRUNTIME_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	ANIMGRAPHRUNTIME_API void PostLoad();

#if WITH_EDITOR
	ANIMGRAPHRUNTIME_API void EnsureChainSize();
#endif // WITH_EDITOR
private:
	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	FVector GetAlignVector(EAxis::Type AxisOption, bool bInvert);

	// skeleton index
	TArray<int32> ChainBoneIndices;
};
