// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneContainer.h"
#include "Animation/AnimCurveTypes.h"
#include "CommonAnimTypes.h"
#include "BonePose.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "AnimNode_TwistCorrectiveNode.generated.h"

/** Reference Bone Frame */
USTRUCT()
struct FReferenceBoneFrame
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "FReferenceBoneFrame")
	FBoneReference Bone;

	UPROPERTY(EditAnywhere, Category = "FReferenceBoneFrame")
	FAxis Axis;

	FReferenceBoneFrame() {};
};

/**
 * This is the node that apply corrective morphtarget for twist 
 * Good example is that if you twist your neck too far right or left, you're going to see odd stretch shape of neck, 
 * This node can detect the angle and apply morphtarget curve 
 * This isn't the twist control node for bone twist
 */
USTRUCT()
struct FAnimNode_TwistCorrectiveNode : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY()

	/** Base Frame of the reference for the twist node */
	UPROPERTY(EditAnywhere, Category="Reference Frame")
	FReferenceBoneFrame BaseFrame;

	// Transform component to use as input
	UPROPERTY(EditAnywhere, Category="Reference Frame")
	FReferenceBoneFrame TwistFrame;

	/** Normal of the Plane that we'd like to calculate angle calculation from in BaseFrame. Please note we're looking for Normal Axis */
	UPROPERTY(EditAnywhere, Category = "Reference Frame")
	FAxis TwistPlaneNormalAxis;

	// @todo since this isn't used yet, I'm commenting it out. 
	// The plan is to support mapping curve between input to output
	//UPROPERTY(EditAnywhere, Category = "Mapping")
	//FAlphaBlend MappingCurve;

 	// Maximum limit of the input value (mapped to RemappedMax, only used when limiting the source range)
 	// We can't go more than 180 right now because this is dot product driver
	UPROPERTY(EditAnywhere, Category = "Mapping", meta = (UIMin = 0.f, ClampMin = 0.f, UIMax = 90.f, ClampMax = 90.f, DisplayName = "Max Angle In Degree"))
 	float RangeMax;

 	// Minimum value to apply to the destination (remapped from the input range)
	UPROPERTY(EditAnywhere, Category = "Mapping", meta = (EditCondition = bUseRange, DisplayName = "Mapped Range Min"))
 	float RemappedMin;
 
 	// Maximum value to apply to the destination (remapped from the input range)
	UPROPERTY(EditAnywhere, Category = "Mapping", meta = (EditCondition = bUseRange, DisplayName = "Mapped Range Max"))
 	float RemappedMax;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FAnimCurveParam Curve_DEPRECATED;
#endif

	UPROPERTY(EditAnywhere, Category = "Output Curve")
	FName CurveName;

public:
	ANIMGRAPHRUNTIME_API FAnimNode_TwistCorrectiveNode();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	ANIMGRAPHRUNTIME_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)  override;
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void EvaluateComponentSpaceInternal(FComponentSpacePoseContext& Context) override;
	ANIMGRAPHRUNTIME_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	// Type traits support
	ANIMGRAPHRUNTIME_API bool Serialize(FArchive& Ar);
	ANIMGRAPHRUNTIME_API void PostSerialize(const FArchive& Ar);

protected:
	
	// FAnimNode_SkeletalControlBase protected interface
	ANIMGRAPHRUNTIME_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;

private:
	// Reference Pose Angle
	float ReferenceAngle;
	// Ranged Max in Radian, so that we don't keep having to convert
	float RangeMaxInRadian;

	// Get Reference Axis from the MeshBases
	FVector GetReferenceAxis(FCSPose<FCompactPose>& MeshBases, const FReferenceBoneFrame& Reference) const;
	// Get Angle of Base, and Twist from Reference Bone Transform
	float	GetAngle(const FVector& Base, const FVector& Twist, const FTransform& ReferencetBoneTransform) const;
};

template<>
struct TStructOpsTypeTraits<FAnimNode_TwistCorrectiveNode> : public TStructOpsTypeTraitsBase2<FAnimNode_TwistCorrectiveNode>
{
	enum
	{
		WithSerializer = true,
		WithPostSerialize = true
	};
};
