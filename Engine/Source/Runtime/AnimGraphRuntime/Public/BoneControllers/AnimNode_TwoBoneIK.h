// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneContainer.h"
#include "BonePose.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "CommonAnimTypes.h"
#include "AnimNode_TwoBoneIK.generated.h"

class USkeletalMeshComponent;

/**
 * Simple 2 Bone IK Controller.
 */

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_TwoBoneIK : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY()
	
	/** Name of bone to control. This is the main bone chain to modify from. **/
	UPROPERTY(EditAnywhere, Category=IK)
	FBoneReference IKBone;

	/** Limits to use if stretching is allowed. This value determines when to start stretch. For example, 0.9 means once it reaches 90% of the whole length of the limb, it will start apply. */
	UPROPERTY(EditAnywhere, Category=IK, meta = (editcondition = "bAllowStretching", ClampMin = "0.0", UIMin = "0.0"))
	double StartStretchRatio;

	/** Limits to use if stretching is allowed. This value determins what is the max stretch scale. For example, 1.5 means it will stretch until 150 % of the whole length of the limb.*/
	UPROPERTY(EditAnywhere, Category= IK, meta = (editcondition = "bAllowStretching", ClampMin = "0.0", UIMin = "0.0"))
	double MaxStretchScale;

#if WITH_EDITORONLY_DATA
	/** Limits to use if stretching is allowed - old property DEPRECATED */
	UPROPERTY()
	FVector2D StretchLimits_DEPRECATED;

	/** Whether or not to apply twist on the chain of joints. This clears the twist value along the TwistAxis */
	UPROPERTY()
	bool bNoTwist_DEPRECATED;

	/** If JointTargetSpaceBoneName is a bone, this is the bone to use. **/
	UPROPERTY()
	FName JointTargetSpaceBoneName_DEPRECATED;

	/** If EffectorLocationSpace is a bone, this is the bone to use. **/
	UPROPERTY()
	FName EffectorSpaceBoneName_DEPRECATED;
#endif

	/** Effector Location. Target Location to reach. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Effector, meta = (PinShownByDefault))
	FVector EffectorLocation;

	// cached limb index for upper
	FCompactPoseBoneIndex CachedUpperLimbIndex;

	UPROPERTY(EditAnywhere, Category=Effector)
	FBoneSocketTarget EffectorTarget;

	/** Joint Target Location. Location used to orient Joint bone. **/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=JointTarget, meta=(PinShownByDefault))
	FVector JointTargetLocation;

	// cached limb index for lower
	FCompactPoseBoneIndex CachedLowerLimbIndex;

	UPROPERTY(EditAnywhere, Category = JointTarget)
	FBoneSocketTarget JointTarget;

	/** Specify which axis it's aligned. Used when removing twist */
	UPROPERTY(EditAnywhere, Category = IK, meta = (editcondition = "!bAllowTwist"))
	FAxis TwistAxis;

	/** Reference frame of Effector Location. */
	UPROPERTY(EditAnywhere, Category=Effector)
	TEnumAsByte<enum EBoneControlSpace> EffectorLocationSpace;

	/** Reference frame of Joint Target Location. */
	UPROPERTY(EditAnywhere, Category=JointTarget)
	TEnumAsByte<enum EBoneControlSpace> JointTargetLocationSpace;

	/** Should stretching be allowed, to be prevent over extension */
	UPROPERTY(EditAnywhere, Category=IK)
	uint8 bAllowStretching:1;

	/** Set end bone to use End Effector rotation */
	UPROPERTY(EditAnywhere, Category=IK)
	uint8 bTakeRotationFromEffectorSpace : 1;

	/** Keep local rotation of end bone */
	UPROPERTY(EditAnywhere, Category = IK)
	uint8 bMaintainEffectorRelRot : 1;

	/** Whether or not to apply twist on the chain of joints. This clears the twist value along the TwistAxis */
	UPROPERTY(EditAnywhere, Category = IK)
	uint8 bAllowTwist : 1;

	ANIMGRAPHRUNTIME_API FAnimNode_TwoBoneIK();

	// FAnimNode_Base interface
	ANIMGRAPHRUNTIME_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	ANIMGRAPHRUNTIME_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	ANIMGRAPHRUNTIME_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
#if WITH_EDITOR
	ANIMGRAPHRUNTIME_API void ConditionalDebugDraw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* MeshComp) const;
#endif // WITH_EDITOR
	// End of FAnimNode_SkeletalControlBase interface
	static ANIMGRAPHRUNTIME_API FTransform GetTargetTransform(const FTransform& InComponentTransform, FCSPose<FCompactPose>& MeshBases, FBoneSocketTarget& InTarget, EBoneControlSpace Space, const FVector& InOffset);
private:
	// FAnimNode_SkeletalControlBase interface
	ANIMGRAPHRUNTIME_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

#if WITH_EDITOR
	FVector CachedJoints[3];
	FVector CachedJointTargetPos;
#endif // WITH_EDITOR
};
