// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Engine/SpringInterpolator.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "UObject/ObjectMacros.h"

#include "BoneControllerSolvers.generated.h"

USTRUCT(BlueprintInternalUseOnly)
struct FIKFootPelvisPullDownSolver
{
	GENERATED_BODY()

	// Specifies the spring interpolation parameters applied during pelvis adjustment
	UPROPERTY(EditAnywhere, Category=Settings)
	FVectorRK4SpringInterpolator PelvisAdjustmentInterp;

	// Specifies an alpha between the original and final adjusted pelvis locations
	// This is used to retain some degree of the original pelvis motion
	UPROPERTY(EditAnywhere, Category=Settings, meta=(ClampMin="0.0", ClampMax="1.0"))
	double PelvisAdjustmentInterpAlpha = 0.5;

	// Specifies the maximum displacement the pelvis can be adjusted relative to its original location
	UPROPERTY(EditAnywhere, Category=Settings, meta=(ClampMin="0.0"))
	double PelvisAdjustmentMaxDistance = 10.0;

	// Specifies the pelvis adjustment distance error that is tolerated for each iteration of the solver
	// 
	// When it is detected that the pelvis adjustment distance is incrementing at a value lower or equal
	// to this value for each iteration, the solve will halt. Lower values will marginally increase visual
	// quality at the cost of performance, but may require a higher PelvisAdjustmentMaxIter to be specified
	//
	// The default value of 0.01 specifies 1 centimeter of error
	UPROPERTY(EditAnywhere, Category=Advanced, meta=(ClampMin="0.001"))
	double PelvisAdjustmentErrorTolerance = 0.01;

	// Specifies the maximum number of iterations to run for the pelvis adjustment solver
	// Higher iterations will guarantee closer PelvisAdjustmentErrorTolerance convergence at the cost of performance
	UPROPERTY(EditAnywhere, Category=Advanced, meta=(ClampMin="0"))
	int32 PelvisAdjustmentMaxIter = 3;

	// Iteratively pulls the character pelvis towards the ground based on the relationship of driven IK foot targets versus FK foot limits
	ANIMGRAPHRUNTIME_API FTransform Solve(FTransform PelvisTransform, TArrayView<const float> FKFootDistancesToPelvis, TArrayView<const FVector> IKFootLocations, float DeltaTime);
};
