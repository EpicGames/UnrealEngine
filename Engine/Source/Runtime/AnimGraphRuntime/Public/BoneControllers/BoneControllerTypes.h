// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "BoneContainer.h"
#include "BoneControllerTypes.generated.h"

struct FAnimInstanceProxy;

// Specifies the evaluation mode of an animation warping node
UENUM(BlueprintInternalUseOnly)
enum class EWarpingEvaluationMode : uint8
{
	// Animation warping evaluation parameters are driven by user settings.
	Manual,
	// Animation warping evaluation parameters are graph-driven. This means some
	// properties of the node are automatically computed using the accumulated 
	// root motion delta contribution of the animation graph leading into it.
	Graph
};

// The supported spaces of a corresponding input vector value
UENUM(BlueprintInternalUseOnly)
enum class EWarpingVectorMode : uint8
{
	// Component-space input vector
	ComponentSpaceVector,
	// Actor-space input vector
	ActorSpaceVector,
	// World-space input vector
	WorldSpaceVector,
	// IK Foot Root relative local-space input vector
	IKFootRootLocalSpaceVector,
};

// Vector values which may be specified in a configured space
USTRUCT(BlueprintType)
struct FWarpingVectorValue
{
	GENERATED_BODY()

	// Space of the corresponding Vector value
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings, meta=(PinShownByDefault))
	EWarpingVectorMode Mode = EWarpingVectorMode::ComponentSpaceVector;

	// Specifies a vector relative to the space defined by Mode
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings, meta=(PinShownByDefault))
	FVector Value = FVector::ZeroVector;

	// Retrieves a normalized Component-space direction from the specified DirectionMode and Direction value
	ANIMGRAPHRUNTIME_API FVector AsComponentSpaceDirection(const FAnimInstanceProxy* AnimInstanceProxy, const FTransform& IKFootRootTransform) const;
};
