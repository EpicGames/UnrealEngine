// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CommonAnimationTypes.h"
#include "CoreMinimal.h"
#include "Curves/CurveFloat.h"
#include "Math/MathFwd.h"
#include "Math/Quat.h"
#include "Math/Transform.h"
#include "Math/Vector.h"

struct FRuntimeFloatCurve;

/**
 *	A library of the most common animation functions.
 */
namespace CommonAnimationLibrary
{
	/**
	 *	This function perform easing on a float value using a variety of easing types.
	 *	@param Value The float value to ease
	 *	@param EasingType The easing function to use
	 *	@param CustomCurve The curve to use if the easing type is "Custom"
	 *	@param bFlip If set to true the easing is flipping around
	 *	@param Weight The amount of easing to use against linear (0.0 to 1.0)
	 */
	ANIMGRAPHRUNTIME_API float ScalarEasing(
		float Value, 
		const FRuntimeFloatCurve& CustomCurve,
		EEasingFuncType EasingType = EEasingFuncType::Linear,
		bool bFlip = false,
		float Weight = 1.f
	);

	/**
	 *	This function performs retargeting of translation using an easing function.
	 *	For this a range of motion needs to be defined given a direction and the constraints.
	 *	@param Location The input location to be retargeted
	 *	@param Source The transform to be used as the frame of reference
	 *	@param Target The transform to be used as the target space
	 *	@param Axis The direction to use for the range measurement. Defaults to (1.0, 0.0, 0.0)
	 *	@param SourceMinimum The minimum of the source range
	 *	@param SourceMaximum The maximum of the source range
	 *	@param TargetMinimum The minimum of the target range
	 *	@param TargetMaximum The maximum of the target range
	 *	@param EasingType The type of easing to apply
	 *	@param CustomCurve The curve to use if the easing type is "Custom"
	 *	@param bFlipEasing If set to true the easing is flipping around
	 *	@param EasingWeight The amount of easing to use against linear (0.0 to 1.0)
	 */
	ANIMGRAPHRUNTIME_API FVector RetargetSingleLocation(
		FVector Location,
		const FTransform& Source,
		const FTransform& Target,
		const FRuntimeFloatCurve& CustomCurve,
		EEasingFuncType EasingType = EEasingFuncType::Linear,
		bool bFlipEasing = false,
		float EasingWeight = 1.f,
		FVector Axis = FVector(1.f, 0.f, 0.f),
		float SourceMinimum = -1.f,
		float SourceMaximum = 1.f,
		float TargetMinimum = -1.f,
		float TargetMaximum = 1.f
	);

	/**
	 *	This function performs retargeting of rotation using an easing function. 
	 *	For this a range of motion needs to be defined as a euler angle, swing angle or twist
	 *	@param Rotation The input rotation to be retargeted
	 *	@param Source The transform to be used as the frame of reference
	 *	@param Target The transform to be used as the target space
	 *	@param CustomCurve The curve to use if the easing type is "Custom"
	 *	@param bFlipEasing If set to true the easing is flipping around
	 *	@param EasingWeight The amount of easing to use against linear (0.0 to 1.0)
	 *	@param RotationComponent The component of the rotation to retarget
	 *	@param TwistAxis The axis to use when extracting swing / twist rotations
	 *	@param bUseAbsoluteAngle If set to true negative angles will be flipped to positive. This can be used to mirror the rotation.
	 *	@param SourceMinimum The minimum of the source range in degrees
	 *	@param SourceMaximum The maximum of the source range in degrees
	 *	@param TargetMinimum The minimum of the target range in degrees
	 *	@param TargetMaximum The maximum of the target range in degrees
	 *	@param EasingType The type of easing to apply
	 */
	ANIMGRAPHRUNTIME_API FQuat RetargetSingleRotation(
		const FQuat& Rotation,
		const FTransform& Source,
		const FTransform& Target,
		const FRuntimeFloatCurve& CustomCurve,
		EEasingFuncType EasingType = EEasingFuncType::Linear,
		bool bFlipEasing = false,
		float EasingWeight = 1.f,
		ERotationComponent RotationComponent = ERotationComponent::SwingAngle,
		FVector TwistAxis = FVector(1.f, 0.f, 0.f),
		bool bUseAbsoluteAngle = false,
		float SourceMinimum = 0.0f,
		float SourceMaximum = 45.f,
		float TargetMinimum = 0.f,
		float TargetMaximum = 45.f
	);
};

