// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Curves/CurveFloat.h"
#include "CommonAnimationTypes.generated.h"

/**
 *	An easing type defining how to ease float values.
 */
UENUM()
enum class EEasingFuncType : uint8
{
	// Linear easing (no change to the value)
	Linear,
	// Easing using a sinus function
	Sinusoidal,
	// Cubic version of the value (only in)
	Cubic,
	// Quadratic version of the value (in and out)
	QuadraticInOut,
	// Cubic version of the value (in and out)
	CubicInOut,
	// Easing using a cubic hermite function
	HermiteCubic,
	// Quartic version of the value (in and out)
	QuarticInOut,
	// Quintic version of the value (in and out)
	QuinticInOut,
	// Circular easing (only in)
	CircularIn,
	// Circular easing (only out)
	CircularOut,
	// Circular easing (in and out)
	CircularInOut,
	// Exponential easing (only in)
	ExpIn,
	// Exponential easing (only out)
	ExpOut,
	// Exponential easing (in and out)
	ExpInOut,
	// Custom - based on an optional Curve
	CustomCurve
};

// A rotational component. This is used for retargeting, for example.
UENUM()
enum class ERotationComponent : uint8
{
	// Using the X component of the Euler rotation
	EulerX,
	// Using the Y component of the Euler rotation
	EulerY,
	// Using the Z component of the Euler rotation
	EulerZ,
	// Using the angle of the quaternion
	QuaternionAngle,
	// Using the angle of the swing quaternion
	SwingAngle,
	// Using the angle of the twist quaternion
	TwistAngle
};

/**
 *	The FRotationRetargetingInfo is used to provide all of the 
 *	settings required to perform rotational retargeting on a single
 *	transform.
 */
USTRUCT(BlueprintType)
struct FRotationRetargetingInfo
{
	GENERATED_BODY()

public:

	/** Default constructor */
	FRotationRetargetingInfo(bool bInEnabled = true)
		: bEnabled(bInEnabled)
		, Source(FTransform::Identity)
		, Target(FTransform::Identity)
		, RotationComponent(ERotationComponent::SwingAngle)
		, TwistAxis(FVector(1.f, 0.f, 0.f))
		, bUseAbsoluteAngle(false)
		, SourceMinimum(0.f)
		, SourceMaximum(45.f)
		, TargetMinimum(0.f)
		, TargetMaximum(45.f)
		, EasingType(EEasingFuncType::Linear)
		, CustomCurve(FRuntimeFloatCurve())
		, bFlipEasing(false)
		, EasingWeight(1.f)
		, bClamp(false)
	{
	}

	// Set to true this enables retargeting
	UPROPERTY(EditAnywhere, Category = "FRotationRetargetingInfo")
	bool bEnabled;

	// The source transform of the frame of reference. The rotation is made relative to this space
	UPROPERTY(EditAnywhere, Category = "FRotationRetargetingInfo")
	FTransform Source;

	// The target transform to project the rotation. In most cases this is the same as Source
	UPROPERTY(EditAnywhere, Category = "FRotationRetargetingInfo")
	FTransform Target;

	// The rotation component to perform retargeting with
	UPROPERTY(EditAnywhere, Category = "FRotationRetargetingInfo")
	ERotationComponent RotationComponent;

	// In case the rotation component is SwingAngle or TwistAngle this vector is used as the twist axis
	UPROPERTY(EditAnywhere, Category = "FRotationRetargetingInfo")
	FVector TwistAxis;

	// If set to true the angle will be always positive, thus resulting in mirrored rotation both ways
	UPROPERTY(EditAnywhere, Category = "FRotationRetargetingInfo")
	bool bUseAbsoluteAngle;

	// The minimum value of the source angle in degrees
	UPROPERTY(EditAnywhere, Category = "FRotationRetargetingInfo", meta = (UIMin = "-90.0", UIMax = "90.0"))
	float SourceMinimum;

	// The maximum value of the source angle in degrees
	UPROPERTY(EditAnywhere, Category = "FRotationRetargetingInfo", meta = (UIMin = "-90.0", UIMax = "90.0"))
	float SourceMaximum;

	// The minimum value of the target angle in degrees (can be the same as SourceMinimum)
	UPROPERTY(EditAnywhere, Category = "FRotationRetargetingInfo", meta = (UIMin = "-90.0", UIMax = "90.0"))
	float TargetMinimum;

	// The target value of the target angle in degrees (can be the same as SourceMaximum)
	UPROPERTY(EditAnywhere, Category = "FRotationRetargetingInfo", meta = (UIMin = "-90.0", UIMax = "90.0"))
	float TargetMaximum;

	// The easing to use - pick linear if you don't want to apply any easing
	UPROPERTY(EditAnywhere, Category = "FRotationRetargetingInfo")
	EEasingFuncType EasingType;

	/** Custom curve mapping to apply if bApplyCustomCurve is true */
	UPROPERTY(EditAnywhere, Category = "FRotationRetargetingInfo")
	FRuntimeFloatCurve CustomCurve;

	// If set to true the interpolation value for the easing will be flipped (1.0 - Value)
	UPROPERTY(EditAnywhere, Category = "FRotationRetargetingInfo")
	bool bFlipEasing;

	// The amount of easing to apply (value should be 0.0 to 1.0)
	UPROPERTY(EditAnywhere, Category = "FRotationRetargetingInfo", meta = (UIMin = "0.0", UIMax = "1.0"))
	float EasingWeight;
	
	// If set to true the value for the easing will be clamped between 0.0 and 1.0
	UPROPERTY(EditAnywhere, Category = "FRotationRetargetingInfo")
	bool bClamp;
};
