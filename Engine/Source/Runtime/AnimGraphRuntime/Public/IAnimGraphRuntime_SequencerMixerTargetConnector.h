// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeatures.h"

struct FPoseContext;

class IAnimGraphRuntime_SequencerMixerTargetConnector : public IModularFeature
{
public:

	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("AnimGraphRuntime_SequencerMixerTargetConnector"));
		return FeatureName;
	}

	static const IAnimGraphRuntime_SequencerMixerTargetConnector* Get()
	{
		if (IModularFeatures::Get().IsModularFeatureAvailable(IAnimGraphRuntime_SequencerMixerTargetConnector::GetModularFeatureName()))
		{
			return &IModularFeatures::Get().GetModularFeature<IAnimGraphRuntime_SequencerMixerTargetConnector>(IAnimGraphRuntime_SequencerMixerTargetConnector::GetModularFeatureName());
		}
		return nullptr;
	}

	virtual ~IAnimGraphRuntime_SequencerMixerTargetConnector() = default;
	
	virtual void ApplySequencerMixedPose(FPoseContext& Output, FName InTargetName) const = 0;
};
