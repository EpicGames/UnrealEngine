// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Components/SceneComponent.h"
#include "Curves/CurveFloat.h"
#include "KismetAnimationTypes.h"
#include "KismetAnimationLibrary.generated.h"

class USkeletalMeshComponent;

/**
 *	A library of the most common animation blueprint functions.
 */
UCLASS(meta=(ScriptName="AnimGraphLibrary", BlueprintThreadSafe, DocumentationPolicy="Strict"), MinimalAPI)
class UKismetAnimationLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/**
	 *	Computes the transform for two bones using inverse kinematics.
	 *
	 *	@param RootPos The input root position of the two bone chain
	 *	@param JointPos The input center (elbow) position of the two bone chain
	 *	@param EndPos The input end (wrist) position of the two bone chain
	 *	@param JointTarget The IK target for the write to reach
	 *	@param Effector The position of the target effector for the IK Chain.
	 *	@param OutJointPos The resulting position for the center (elbow)
	 *	@param OutEndPos The resulting position for the end (wrist)
	 *	@param bAllowStretching If set to true the bones are allowed to stretch
	 *	@param StartStretchRatio The ratio at which the bones should start to stretch. The higher the value, the later the stretching wil start.
	 *	@param MaxStretchScale The maximum multiplier for the stretch to reach.
	 */
	UFUNCTION(BlueprintPure, Category = "Animation|Utilities", meta = (DisplayName = "Two Bone IK Function", ScriptName = "TwoBoneIK", bAllowStretching = "false", StartStretchRatio = "1.0", MaxStretchScale = "1.2"))
	static ANIMGRAPHRUNTIME_API void K2_TwoBoneIK(
		const FVector& RootPos, 
		const FVector& JointPos, 
		const FVector& EndPos, 
		const FVector& JointTarget, 
		const FVector& Effector, 
		FVector& OutJointPos, 
		FVector& OutEndPos, 
		bool bAllowStretching, 
		float StartStretchRatio,
		float MaxStretchScale
	);

	/**
	 *	Computes the transform which is "looking" at target position with a local axis.
	 *
	 *	@param CurrentTransform The input transform to modify
	 *	@param TargetPosition The position this transform should look at
	 *	@param LookAtVector The local vector to align with the target
	 *	@param bUseUpVector If set to true the lookat will also perform a twist rotation
	 *	@param UpVector The position to use for the upvector target (only used is bUseUpVector is turned on)
	 *	@param ClampConeInDegree A limit for only allowing the lookat to rotate as much as defined by the float value
	 */
	UFUNCTION(BlueprintPure, Category = "Animation|Utilities", meta = (DisplayName = "Look At Function", ScriptName = "LookAt", bUseUpVector = "false"))
	static ANIMGRAPHRUNTIME_API FTransform K2_LookAt(
		const FTransform& CurrentTransform, 
		const FVector& TargetPosition, 
		FVector LookAtVector, 
		bool bUseUpVector, 
		FVector UpVector, 
		float ClampConeInDegree
	);

	/**
	 *	Computes the distance between two bones / sockets and can remap the range.
	 *
	 *	@param Component The skeletal component to look for the sockets / bones within
	 *	@param SocketOrBoneNameA The name of the first socket / bone
	 *	@param SocketSpaceA The space for the first socket / bone
	 *	@param SocketOrBoneNameB The name of the second socket / bone
	 *	@param SocketSpaceB The space for the second socket / bone
	 *	@param bRemapRange If set to true, the distance will be remapped using the range parameters
	 *	@param InRangeMin The minimum for the input range (commonly == 0.0)
	 *	@param InRangeMax The maximum for the input range (the max expected distance)
	 *	@param OutRangeMin The minimum for the output range (commonly == 0.0)
	 *	@param OutRangeMax The maximum for the output range (commonly == 1.0)
	 */
	UFUNCTION(BlueprintPure, Category = "Animation|Utilities", meta = (NotBlueprintThreadSafe, DisplayName = "Get Distance Between Two Sockets", ScriptName = "DistanceBetweenSockets", bRemapRange= "false"))
	static ANIMGRAPHRUNTIME_API float K2_DistanceBetweenTwoSocketsAndMapRange(
		const USkeletalMeshComponent* Component, 
		const FName SocketOrBoneNameA, 
		ERelativeTransformSpace SocketSpaceA, 
		const FName SocketOrBoneNameB, 
		ERelativeTransformSpace SocketSpaceB, 
		bool bRemapRange, 
		float InRangeMin,
		float InRangeMax,
		float OutRangeMin,
		float OutRangeMax);

	/**
	 *	Computes the direction between two bones / sockets.
	 *
	 *	@param Component The skeletal component to look for the sockets / bones within
	 *	@param SocketOrBoneNameFrom The name of the first socket / bone
	 *	@param SocketOrBoneNameTo The name of the second socket / bone
	 */
	UFUNCTION(BlueprintPure, Category = "Animation|Utilities", meta = (NotBlueprintThreadSafe, DisplayName = "Get Direction Between Sockets", ScriptName = "DirectionBetweenSockets"))
	static ANIMGRAPHRUNTIME_API FVector K2_DirectionBetweenSockets(const USkeletalMeshComponent* Component, const FName SocketOrBoneNameFrom, const FName SocketOrBoneNameTo);

	/** 
	 *	This function creates perlin noise from input X, Y, Z, and then range map to RangeOut, and out put to OutX, OutY, OutZ
	 *
	 *	@param X The x component for the input position of the noise
	 *	@param Y The y component for the input position of the noise
	 *	@param Z The z component for the input position of the noise
	 *	@param RangeOutMinX The minimum for the output range for the x component
	 *	@param RangeOutMaxX The maximum for the output range for the x component
	 *	@param RangeOutMinY The minimum for the output range for the y component
	 *	@param RangeOutMaxY The maximum for the output range for the y component
	 *	@param RangeOutMinZ The minimum for the output range for the z component
	 *	@param RangeOutMaxZ The maximum for the output range for the z component
	 */
	UFUNCTION(BlueprintPure, Category = "Animation|Utilities", meta = (DisplayName = "Make Perlin Noise Vector and Remap", ScriptName = "MakeVectorFromPerlinNoise", RangeOutMinX = "-1.f", RangeOutMaxX = "1.f", RangeOutMinY = "-1.f", RangeOutMaxY = "1.f", RangeOutMinZ = "-1.f", RangeOutMaxZ = "1.f"))
	static ANIMGRAPHRUNTIME_API FVector K2_MakePerlinNoiseVectorAndRemap(
		float X,
		float Y,
		float Z,
		float RangeOutMinX,
		float RangeOutMaxX,
		float RangeOutMinY,
		float RangeOutMaxY,
		float RangeOutMinZ,
		float RangeOutMaxZ
	);

	/**
	 *	This function creates perlin noise for a single float and then range map to RangeOut
	 *
	 *	@param Value The input value for the noise function
	 *	@param RangeOutMin The minimum for the output range
	 *	@param RangeOutMax The maximum for the output range
	 */
	UFUNCTION(BlueprintPure, Category = "Animation|Utilities", meta = (DisplayName = "Make Perlin Noise and Remap", ScriptName = "MakeFloatFromPerlinNoise", RangeOutMinX = "-1.f", RangeOutMaxX = "1.f", RangeOutMinY = "-1.f", RangeOutMaxY = "1.f", RangeOutMinZ = "-1.f", RangeOutMaxZ = "1.f"))
	static ANIMGRAPHRUNTIME_API float K2_MakePerlinNoiseAndRemap(
		float Value,
		float RangeOutMin,
		float RangeOutMax
	);

	/** 
	 *	This function calculates the velocity of a position changing over time.
	 *	You need to hook up a valid PositionHistory variable to this for storage.
	 *
	 *	@param DeltaSeconds The time passed in seconds
	 *	@param Position The position to track over time.
	 *	@param History The history to use for storage.
	 *	@param NumberOfSamples The number of samples to use for the history. The higher the number of samples - the smoother the velocity changes.
	 *	@param VelocityMin The minimum velocity to use for normalization (if both min and max are set to 0, normalization is turned off)
	 *	@param VelocityMax The maximum velocity to use for normalization (if both min and max are set to 0, normalization is turned off)
	 */
	UFUNCTION(BlueprintCallable, Category = "Animation|Utilities", meta = (DisplayName = "Calculate Velocity From Position History", ScriptName = "CalculateVelocityFromPositionHistory", NumberOfSamples = "16", VelocityMin = "0.f", VelocityMax = "128.f"))
	static ANIMGRAPHRUNTIME_API float K2_CalculateVelocityFromPositionHistory(
		float DeltaSeconds,
		FVector Position,
		UPARAM(ref) FPositionHistory& History,
		int32 NumberOfSamples,
		float VelocityMin,
		float VelocityMax
	);

	/** 
	 *	This function calculates the velocity of an offset position on a bone / socket over time.
	 *	The bone's / socket's motion can be expressed within a reference frame (another bone / socket). 
	 *	You need to hook up a valid PositionHistory variable to this for storage.
	 *
	 *	@param DeltaSeconds The time passed in seconds
	 *	@param Component The skeletal component to look for the bones / sockets
	 *	@param SocketOrBoneName The name of the bone / socket to track.
	 *	@param ReferenceSocketOrBone The name of the bone / socket to use as a frame of reference (or None if no frame of reference == world space).
	 *	@param SocketSpace The space to use for the two sockets / bones
	 *	@param OffsetInBoneSpace The relative position in the space of the bone / socket to track over time.
	 *	@param History The history to use for storage.
	 *	@param NumberOfSamples The number of samples to use for the history. The higher the number of samples - the smoother the velocity changes.
	 *	@param VelocityMin The minimum velocity to use for normalization (if both min and max are set to 0, normalization is turned off)
	 *	@param VelocityMax The maximum velocity to use for normalization (if both min and max are set to 0, normalization is turned off)
	 *	@param EasingType The easing function to use
	 *	@param CustomCurve The curve to use if the easing type is "Custom"
	 */
	UFUNCTION(BlueprintCallable, Category = "Animation|Utilities", meta = (NotBlueprintThreadSafe, DisplayName = "Calculate Velocity From Sockets", ScriptName = "CalculateVelocityFromSockets", NumberOfSamples = "16", VelocityMin = "0.f", VelocityMax = "128.f"))
	static ANIMGRAPHRUNTIME_API float K2_CalculateVelocityFromSockets(
		float DeltaSeconds,
		USkeletalMeshComponent * Component,
		const FName SocketOrBoneName,
		const FName ReferenceSocketOrBone,
		ERelativeTransformSpace SocketSpace,
		FVector OffsetInBoneSpace, 
		UPARAM(ref) FPositionHistory& History, 
		int32 NumberOfSamples,
		float VelocityMin,
		float VelocityMax,
		EEasingFuncType EasingType,
		const FRuntimeFloatCurve& CustomCurve
		);

	/**
	 *	This function starts measuring the time for a profiling bracket
	 */
	UFUNCTION(BlueprintCallable, Category = "Animation|Utilities", meta = (DisplayName = "Start Profiling Timer"))
	static ANIMGRAPHRUNTIME_API void K2_StartProfilingTimer();

	/**
	 *	This function ends measuring a profiling bracket and optionally logs the result
	 *
	 *  @param bLog If set to true the result is logged to the OutputLog
	 *  @param LogPrefix A prefix to use for the log
	 *  @result The time spent in milliseconds
	 */
	UFUNCTION(BlueprintCallable, Category = "Animation|Utilities", meta = (DisplayName = "End Profiling Timer"))
	static ANIMGRAPHRUNTIME_API float K2_EndProfilingTimer(bool bLog = true, const FString& LogPrefix = "");

	/** 
	 * Returns degree of the angle between Velocity and Rotation forward vector
	 * The range of return will be from [-180, 180]. Useful for feeding directional blendspaces.
	 * @param	Velocity		The velocity to use as direction relative to BaseRotation
	 * @param	BaseRotation	The base rotation, e.g. of a pawn
	 */
	UFUNCTION(BlueprintPure, Category="Animation|Utilities")
	static ANIMGRAPHRUNTIME_API float CalculateDirection(const FVector& Velocity, const FRotator& BaseRotation);
};

