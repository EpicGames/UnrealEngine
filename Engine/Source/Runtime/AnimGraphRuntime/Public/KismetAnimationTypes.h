// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CommonAnimationTypes.h"
#include "KismetAnimationTypes.generated.h"

/**
 *	An easing type defining how to ease float values.
 *	The FPositionHistory is a container to record position changes
 *	over time. This is used to calculate velocity of a bone, for example.
 *	The FPositionArray also tracks the last index used to allow for 
 *	reuse of entries (instead of appending to the array all the time).
 */
USTRUCT(BlueprintType, meta = (DocumentationPolicy = "Strict"))
struct FPositionHistory
{
	GENERATED_BODY()

public:

	/** Default constructor */
	FPositionHistory()
		: Positions(TArray<FVector>())
		, Range(0.f)
		, Velocities(TArray<float>())
		, LastIndex(0)
	{}

	/** The recorded positions */
	UPROPERTY(EditAnywhere, Category = "FPositionHistory")
	TArray<FVector> Positions;

	/** The range for this particular history */
	UPROPERTY(EditAnywhere, Category = "FPositionHistory", meta = (UIMin = "0.0", UIMax = "1.0"))
	float Range;

	TArray<float> Velocities;
	uint32 LastIndex; /// The last index used to store a position
};
