// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimExecutionContext.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimNodeReference.h"
#include "LayeredBoneBlendLibrary.generated.h"

struct FAnimNode_LayeredBoneBlend;

USTRUCT(BlueprintType)
struct FLayeredBoneBlendReference : public FAnimNodeReference
{
	GENERATED_BODY()

	typedef FAnimNode_LayeredBoneBlend FInternalNodeType;
};

/**
 * Exposes operations to be performed on a layered bone blend anim node.
 */
UCLASS(MinimalAPI)
class ULayeredBoneBlendLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Get a layered bone blend context from an anim node context. */
	UFUNCTION(BlueprintCallable, Category = "Layered Bone Blend", meta = (BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static ANIMGRAPHRUNTIME_API FLayeredBoneBlendReference ConvertToLayeredBoneBlend(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result);

	/** Get a layered bone blend context from an anim node context (pure). */
	UFUNCTION(BlueprintPure, Category = "Layered Bone Blend", meta = (BlueprintThreadSafe, DisplayName = "Convert to Layered Bone Blend"))
	static void ConvertToLayeredBlendPerBonePure(const FAnimNodeReference& Node, FLayeredBoneBlendReference& LayeredBoneBlend, bool& Result)
	{
		EAnimNodeReferenceConversionResult ConversionResult;
		LayeredBoneBlend = ConvertToLayeredBoneBlend(Node, ConversionResult);
		Result = (ConversionResult == EAnimNodeReferenceConversionResult::Succeeded);
	}

	/** Get the number of poses that a layered bone blend node has (this does not include the base pose) */
	UFUNCTION(BlueprintPure, Category = "Layered Bone Blend", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API int32 GetNumPoses(const FLayeredBoneBlendReference& LayeredBoneBlend);
	
	/**
	 * Sets the currently-used blend mask for a blended input pose by name.
	 * @param	UpdateContext		The update context to use. This is used to extract the current skeleton to derive the blend mask from
	 * @param	LayeredBoneBlend	A reference to the node
	 * @param	PoseIndex			The pose index to set the blend mask for
	 * @param	BlendMaskName	The name of the blend mask to use 
	 */
	UFUNCTION(BlueprintCallable, Category = "Layered Bone Blend", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FLayeredBoneBlendReference SetBlendMask(const FAnimUpdateContext& UpdateContext, const FLayeredBoneBlendReference& LayeredBoneBlend, int32 PoseIndex, FName BlendMaskName);
};
