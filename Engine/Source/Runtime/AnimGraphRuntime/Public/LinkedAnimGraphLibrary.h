// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimNodeReference.h"
#include "LinkedAnimGraphLibrary.generated.h"

struct FAnimNode_LinkedAnimGraph;

USTRUCT(BlueprintType)
struct FLinkedAnimGraphReference : public FAnimNodeReference
{
	GENERATED_BODY()

	typedef FAnimNode_LinkedAnimGraph FInternalNodeType;
};

// Exposes operations to be performed on anim node contexts
UCLASS(MinimalAPI)
class ULinkedAnimGraphLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Get a linked anim graph reference from an anim node reference */
	UFUNCTION(BlueprintCallable, Category = "Animation|Linked Anim Graphs", meta=(BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static ANIMGRAPHRUNTIME_API FLinkedAnimGraphReference ConvertToLinkedAnimGraph(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result);

	/** Get a linked anim graph reference from an anim node reference (pure) */
	UFUNCTION(BlueprintPure, Category = "Animation|Linked Anim Graphs", meta=(BlueprintThreadSafe, DisplayName = "Convert to Linked Anim Graph"))
	static void ConvertToLinkedAnimGraphPure(const FAnimNodeReference& Node, FLinkedAnimGraphReference& LinkedAnimGraph, bool& Result)
	{
		EAnimNodeReferenceConversionResult ConversionResult;
		LinkedAnimGraph = ConvertToLinkedAnimGraph(Node, ConversionResult);
		Result = (ConversionResult == EAnimNodeReferenceConversionResult::Succeeded);
	}
	
	/** Returns whether the node hosts an instance (e.g. linked anim graph or layer) */
	UFUNCTION(BlueprintPure, Category = "Animation|Linked Anim Graphs", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API bool HasLinkedAnimInstance(const FLinkedAnimGraphReference& Node);

	/** Get the linked instance is hosted by this node. If the node does not host an instance then HasLinkedAnimInstance will return false */
	UFUNCTION(BlueprintPure, Category = "Animation|Linked Anim Graphs", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API UAnimInstance* GetLinkedAnimInstance(const FLinkedAnimGraphReference& Node);
};
