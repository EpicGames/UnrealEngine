// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Animation/AnimExecutionContext.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimNodeReference.h"
#include "MirrorAnimLibrary.generated.h"

struct FAnimNode_MirrorBase;

USTRUCT(BlueprintType)
struct FMirrorAnimNodeReference : public FAnimNodeReference
{
	GENERATED_BODY()
	typedef FAnimNode_MirrorBase FInternalNodeType;
};

/** Exposes operations that can be run on a Mirror node via Anim Node Functions such as "On Become Relevant" and "On Update". */
UCLASS()
class UMirrorAnimLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	
	/** Get a mirror node context from an anim node context */
	UFUNCTION(BlueprintCallable, Category = "Animation|Mirroring", meta=(BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static ANIMGRAPHRUNTIME_API FMirrorAnimNodeReference ConvertToMirrorNode(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result);
	
	/** Get a mirror context from an anim node context (pure) */
	UFUNCTION(BlueprintPure, Category = "Animation|Mirroring", meta=(BlueprintThreadSafe, DisplayName = "Convert to Mirror Node"))
	static void ConvertToMirrorNodePure(const FAnimNodeReference& Node, FMirrorAnimNodeReference& MirrorNode, bool& Result)
	{
		EAnimNodeReferenceConversionResult ConversionResult;
		MirrorNode = ConvertToMirrorNode(Node, ConversionResult);
		Result = (ConversionResult == EAnimNodeReferenceConversionResult::Succeeded);
	}
	
	/** Set the mirror state */
	UFUNCTION(BlueprintCallable, Category = "Animation|Mirroring", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FMirrorAnimNodeReference SetMirror(const FMirrorAnimNodeReference& MirrorNode, bool bInMirror);
	
	/** Set how long to blend using inertialization when switching mirrored state */
	UFUNCTION(BlueprintCallable, Category = "Animation|Mirroring", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FMirrorAnimNodeReference SetMirrorTransitionBlendTime(const FMirrorAnimNodeReference& MirrorNode, float InBlendTime);
	
	/** Get the mirror state */
	UFUNCTION(BlueprintPure, Category = "Animation|Mirroring", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API bool GetMirror(const FMirrorAnimNodeReference& MirrorNode);
	
	/** Get MirrorDataTable used to perform mirroring*/
	UFUNCTION(BlueprintPure, Category = "Animation|Mirroring", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API UMirrorDataTable* GetMirrorDataTable(const FMirrorAnimNodeReference& MirrorNode);
	
	/** Get how long to blend using inertialization when switching mirrored state */
	UFUNCTION(BlueprintPure, Category = "Animation|Mirroring", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API float GetMirrorTransitionBlendTime(const FMirrorAnimNodeReference& MirrorNode);
};