// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Animation/AnimExecutionContext.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimNodeReference.h"
#include "ModifyCurveLibrary.generated.h"

struct FAnimNode_ModifyCurve;

USTRUCT(BlueprintType)
struct FModifyCurveAnimNodeReference : public FAnimNodeReference
{
	GENERATED_BODY()
	typedef FAnimNode_ModifyCurve FInternalNodeType;
};

/** Exposes operations that can be run on a Modify Curve Node via Anim Node Functions such as "On Become Relevant" and "On Update". */
UCLASS()
class UModifyCurveAnimLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	
	/** Get a modify curve node context from an anim node context */
	UFUNCTION(BlueprintCallable, Category = "Animation", meta=(BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static ANIMGRAPHRUNTIME_API FModifyCurveAnimNodeReference ConvertToModifyCurveNode(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result);
	
	/** Get a modify curve context from an anim node context (pure) */
	UFUNCTION(BlueprintPure, Category = "Animation", meta=(BlueprintThreadSafe, DisplayName = "Convert to Modify Curve node"))
	static void ConvertToModifyCurveNodePure(const FAnimNodeReference& Node, FModifyCurveAnimNodeReference& ModifyCurveNode, bool& Result)
	{
		EAnimNodeReferenceConversionResult ConversionResult;
		ModifyCurveNode = ConvertToModifyCurveNode(Node, ConversionResult);
		Result = (ConversionResult == EAnimNodeReferenceConversionResult::Succeeded);
	}

	UFUNCTION(BlueprintCallable, Category = "Animation", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FModifyCurveAnimNodeReference SetCurveMap(const FModifyCurveAnimNodeReference& ModifyCurveNode, const TMap<FName, float> & InCurveMap);

	UFUNCTION(BlueprintCallable, Category = "Animation", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API EModifyCurveApplyMode GetApplyMode(const FModifyCurveAnimNodeReference& ModifyCurveNode);
	
	UFUNCTION(BlueprintCallable, Category = "Animation", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FModifyCurveAnimNodeReference SetApplyMode(const FModifyCurveAnimNodeReference& ModifyCurveNode, EModifyCurveApplyMode InMode);
	
	UFUNCTION(BlueprintCallable, Category = "Animation", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API float GetAlpha(const FModifyCurveAnimNodeReference& ModifyCurveNode);
	
	UFUNCTION(BlueprintCallable, Category = "Animation", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FModifyCurveAnimNodeReference SetAlpha(const FModifyCurveAnimNodeReference& ModifyCurveNode, float InAlpha);
};