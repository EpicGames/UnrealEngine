// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "Animation/AnimInstance.h"
#include "PlayMontageCallbackProxy.generated.h"

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnMontagePlayDelegate, FName, NotifyName);

UCLASS(MinimalAPI)
class UPlayMontageCallbackProxy : public UObject
{
	GENERATED_UCLASS_BODY()

	// Called when Montage finished playing and wasn't interrupted
	UPROPERTY(BlueprintAssignable)
	FOnMontagePlayDelegate OnCompleted;

	// Called when Montage starts blending out and is not interrupted
	UPROPERTY(BlueprintAssignable)
	FOnMontagePlayDelegate OnBlendOut;

	// Called when Montage has been interrupted (or failed to play)
	UPROPERTY(BlueprintAssignable)
	FOnMontagePlayDelegate OnInterrupted;

	UPROPERTY(BlueprintAssignable)
	FOnMontagePlayDelegate OnNotifyBegin;

	UPROPERTY(BlueprintAssignable)
	FOnMontagePlayDelegate OnNotifyEnd;

	// Called to perform the query internally
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true"))
	static ANIMGRAPHRUNTIME_API UPlayMontageCallbackProxy* CreateProxyObjectForPlayMontage(
		class USkeletalMeshComponent* InSkeletalMeshComponent, 
		class UAnimMontage* MontageToPlay, 
		float PlayRate = 1.f, 
		float StartingPosition = 0.f, 
		FName StartingSection = NAME_None,
		bool bShouldStopAllMontages = true);

public:
	//~ Begin UObject Interface
	ANIMGRAPHRUNTIME_API virtual void BeginDestroy() override;
	//~ End UObject Interface

protected:
	UFUNCTION()
	ANIMGRAPHRUNTIME_API void OnMontageBlendingOut(UAnimMontage* Montage, bool bInterrupted);

	UFUNCTION()
	ANIMGRAPHRUNTIME_API void OnMontageEnded(UAnimMontage* Montage, bool bInterrupted);

	UFUNCTION()
	ANIMGRAPHRUNTIME_API void OnNotifyBeginReceived(FName NotifyName, const FBranchingPointNotifyPayload& BranchingPointNotifyPayload);

	UFUNCTION()
	ANIMGRAPHRUNTIME_API void OnNotifyEndReceived(FName NotifyName, const FBranchingPointNotifyPayload& BranchingPointNotifyPayload);

private:
	TWeakObjectPtr<UAnimInstance> AnimInstancePtr;
	int32 MontageInstanceID;
	uint32 bInterruptedCalledBeforeBlendingOut : 1;

	bool IsNotifyValid(FName NotifyName, const FBranchingPointNotifyPayload& BranchingPointNotifyPayload) const;
	void UnbindDelegates();

	FOnMontageBlendingOutStarted BlendingOutDelegate;
	FOnMontageEnded MontageEndedDelegate;

protected:
	// Attempts to play a montage with the specified settings. Returns whether it started or not.
	ANIMGRAPHRUNTIME_API bool PlayMontage(
		class USkeletalMeshComponent* InSkeletalMeshComponent,
		class UAnimMontage* MontageToPlay,
		float PlayRate = 1.f,
		float StartingPosition = 0.f,
		FName StartingSection = NAME_None,
		bool bShouldStopAllMontages = true);
};
