// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Math/Quat.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/MemStack.h"
#include "Templates/Function.h"
#include "Templates/Tuple.h"

/* A collection of distance metrics between two values of the same type */
namespace RBFDistanceMetric
{
	/* Returns the Euclidean (L2) distance between two coordinate vectors. */
	static inline double Euclidean(const FVector& A, const FVector& B)
	{
		return FVector::Distance(A, B);
	}

	/* Returns the Manhattan (L1), or Taxi-cab distance between two coordinate vectors. */
	static inline double Manhattan(const FVector& A, const FVector& B)
	{
		FVector AbsDiff = (A - B).GetAbs();
		return AbsDiff.X + AbsDiff.Y + AbsDiff.Z;
	}

	/* Returns the arc length between two unit vectors (i.e. the distance between two
	   points on a unit sphere, traveling along the surface of the sphere) */
	static inline double ArcLength(const FVector& A, const FVector B)
	{
		return FMath::Acos(A.GetSafeNormal() | B.GetSafeNormal());
	}


	/* Returns a straight-up Euclidean distance between two rotation values expressed
	   in radians.
	*/
	static inline double Euclidean(const FRotator& A, const FRotator& B)
	{
		return Euclidean(FVector(FMath::DegreesToRadians(A.Roll),
								 FMath::DegreesToRadians(A.Pitch),
								 FMath::DegreesToRadians(A.Yaw)),
						 FVector(FMath::DegreesToRadians(B.Roll),
								 FMath::DegreesToRadians(B.Pitch),
								 FMath::DegreesToRadians(B.Yaw)));
	}

	/* Returns the arc-length distance, on a unit sphere, between two rotation vectors.
	*/
	static inline double ArcLength(const FRotator& A, const FRotator& B)
	{
		return FMath::Acos(A.Vector() | B.Vector());
	}

	/* Returns the Euclidean (L2) distance between two quaternion values expressed.
	*/
	static inline double Euclidean(const FQuat& A, const FQuat& B)
	{
		return (A - B).Size();
	}

	/* Returns the arc-length distance, on a unit sphere, between two quaternions.
	*/
	static inline double ArcLength(const FQuat& A, const FQuat& B)
	{
		return A.GetNormalized().AngularDistance(B.GetNormalized());
	}

	/* Returns the swing arc length distance between two quaternions, using a specific 
	   twist basis vector as reference.
	*/
	static inline double SwingAngle(const FQuat& A, const FQuat& B, const FVector& TwistAxis)
	{
		FQuat ASwing, BSwing, DummyTwist;
		A.ToSwingTwist(TwistAxis, ASwing, DummyTwist);
		B.ToSwingTwist(TwistAxis, BSwing, DummyTwist);
		return ASwing.AngularDistance(BSwing);
	}

	/* Returns the twist arc length distance between two quaternions, using a specific 
	   twist basis vector as reference.
	*/
	static inline double TwistAngle(const FQuat& A, const FQuat& B, const FVector& TwistAxis)
	{
		return FMath::Abs(A.GetTwistAngle(TwistAxis) - B.GetTwistAngle(TwistAxis));
	}
}


/* A collection of smoothing kernels, all of which map the input of zero to 1.0 and 
   all values on either side as monotonically decreasing as they move away from zero.
   The width of the falloff can be specified using the Sigma parameter.
   */
namespace RBFKernel
{
	/* A simple linear falloff, clamping at zero out when the norm of Value exceeds Sigma */
	static inline float Linear(float Value, float Sigma)
	{
		return (Sigma - FMath::Clamp(Value, 0.0f, Sigma)) / Sigma;
	}

	/* A gaussian falloff */
	static inline float Gaussian(float Value, float Sigma)
	{
		return FMath::Exp(-Value * FMath::Square(1.0f / Sigma));
	}

	/* An exponential falloff with a sharp peak */
	static inline float Exponential(float Value, float Sigma)
	{
		return FMath::Exp(-2.0f * Value / Sigma);
	}

	/* A cubic falloff, with identical clamping behavior to the linear falloff, 
	   but with a smooth peak */
	static inline float Cubic(float Value, float Sigma)
	{
		Value /= Sigma;
		return FMath::Max(1.f - (Value * Value * Value), 0.f);
	}

	/* A quintic falloff, with identical clamping behavior to the linear falloff, 
	   but with a flatter peak than cubic */
	static inline float Quintic(float Value, float Sigma)
	{
		Value /= Sigma;
		return FMath::Max(1.f - FMath::Pow(Value, 5.0f), 0.f);
	}
}


// An implementation detail for the RBF interpolator to hide the use of Eigen from components
// outside AnimGraphRuntime.
class FRBFInterpolatorBase
{
protected:
	ANIMGRAPHRUNTIME_API bool SetUpperKernel(const TArrayView<float>& UpperKernel, int32 Size);

	// A square matrix of the solved coefficients.
public:
	TArray<float> Coeffs;
	bool bIsValid = false;
};


template<typename T>
class TRBFInterpolator
	: public FRBFInterpolatorBase
{
public:
	using WeightFuncT = TFunction<float(const T& A, const T& B)>;

	TRBFInterpolator() = default;

	/* Construct an RBF interpolator, taking in a set of sparse nodes and a symmetric weighing
	   function that computes the distance between two nodes, and, optionally, smooths 
	   the distance with a smoothing kernel.
	*/
	TRBFInterpolator(
		const TArrayView<T>& InNodes,
		WeightFuncT InWeightFunc)
		: Nodes(InNodes)
		, WeightFunc(InWeightFunc)
	{
		MakeUpperKernel();
	}

	TRBFInterpolator(const TRBFInterpolator<T>&) = default;
	TRBFInterpolator(TRBFInterpolator<T>&&) = default;
	TRBFInterpolator<T>& operator=(const TRBFInterpolator<T>&) = default;
	TRBFInterpolator<T>& operator=(TRBFInterpolator<T>&&) = default;

	/* Given a value, compute the weight values to use to calculate each node's contribution
	   to that value's location.
	*/
	template<typename U, typename InAllocator>
	void Interpolate(
		TArray<float, InAllocator>& OutWeights,
		const U& Value,
		bool bClip = true,
		bool bNormalize = false) const
	{
		int NumNodes = Nodes.Num();

		if (!bIsValid)
		{
			OutWeights.Init(0.0f, NumNodes);
			return;
		}	

		if (NumNodes > 1)
		{
			TArray<float, TMemStackAllocator<> > ValueWeights;
			ValueWeights.SetNum(Nodes.Num());

			for (int32 i = 0; i < NumNodes; i++)
			{
				ValueWeights[i] = WeightFunc(Value, Nodes[i]);
			}

			OutWeights.Reset(NumNodes);
			for (int32 i = 0; i < NumNodes; i++)
			{
				const float* C = &Coeffs[i * NumNodes];
				float W = 0.0f;

				for (int32 j = 0; j < NumNodes; j++)
				{
					W += C[j] * ValueWeights[j];
				}

				OutWeights.Add(W);
			}


			if (bNormalize)
			{
				// Clip here behaves differently than it does when no normalization
				// is taking place. Instead of clipping blindly, we rescale the values based
				// on the minimum value and then use the normalization to bring the values
				// within the 0-1 range.
				if (bClip)
				{
					float MaxNegative = 0.0f;
					for (int32 i = 0; i < NumNodes; i++)
					{
						if (OutWeights[i] < MaxNegative)
							MaxNegative = OutWeights[i];
					}
					for (int32 i = 0; i < NumNodes; i++)
					{
						OutWeights[i] -= MaxNegative;
					}
				}

				float TotalWeight = 0.0f;
				for (int32 i = 0; i < NumNodes; i++)
				{
					TotalWeight += OutWeights[i];
				}
				for (int32 i = 0; i < NumNodes; i++)
				{
					// Clamp to clear up any precision issues. This may make the weights not
					// quite add up to 1.0, but that should be sufficient for our needs.
					OutWeights[i] = FMath::Clamp(OutWeights[i] / TotalWeight, 0.0f, 1.0f);
				}
			}
			else if (bClip)
			{
				// This can easily happen when the value being interpolated is outside of the
				// convex hull bounded by the nodes, resulting in an extrapolation.
				for (int32 i = 0; i < NumNodes; i++)
				{
					OutWeights[i] = FMath::Clamp(OutWeights[i], 0.0f, 1.0f);
				}
			}
		}
		else if (NumNodes == 1)
		{
			OutWeights.Reset(1);
			OutWeights.Add(1);
		}
		else
		{
			OutWeights.Reset(0);
		}
	}

	// Returns a list of integer pairs indicating which distinct pair of nodes have the same
	// weight as a pair of the same node. These result in an ill-formed coefficient matrix
	// which kills the interpolation. The user can then either simply remove one of the pairs
	// and retry, or warn the user that they have an invalid setup.
	static bool GetIdenticalNodePairs(
		const TArrayView<T>& InNodes,
		WeightFuncT InWeightFunc,
		TArray<TTuple<int, int>>& OutInvalidPairs
		) 
	{
		int NumNodes = InNodes.Num();
		if (NumNodes < 2)
		{
			return false;		
		}

		// One of the assumptions we make, is that the smoothing function is symmetric, 
		// hence we can use the weight between the same node as the functional equivalent
		// of the identity weight between any two nodes.
		float IdentityWeight = InWeightFunc(InNodes[0], InNodes[0]);

		OutInvalidPairs.Empty();
		for (int32 i = 0; i < (NumNodes - 1); i++)
		{
			for (int32 j = i + 1; j < NumNodes; j++)
			{
				float Weight = InWeightFunc(InNodes[i], InNodes[j]);

				// Don't use the default ULP, but be a little more cautious, since a matrix
				// inversion can lose a chunk of float precision.
				if (FMath::IsNearlyEqualByULP(Weight, IdentityWeight, 32))
				{
					OutInvalidPairs.Add(MakeTuple(i, j));
				}
			}
		}
		return OutInvalidPairs.Num() != 0;
	}

private:
	void MakeUpperKernel()
	{
		// If there are less than two nodes, nothing to do, since the interpolated value
		// will be the same across the entire space. This is handled in Interpolate().
		int32 NumNodes = Nodes.Num();
		if (NumNodes < 2)
		{
			bIsValid = true;
			return;
		}

		// Compute the upper diagonal of the target kernel for solving the weight coefficients.
		TArray<float, TMemStackAllocator<> > UpperKernel;
		UpperKernel.Reserve(NumNodes * (NumNodes - 1) / 2);

		// We need to include the diagonal itself, since we can't guarantee that the weight
		// function returns 1.0 for nodes of the same coordinates.
		for (int32 i = 0; i < NumNodes; i++)
		{
			// PVS thinks the use of 'j = i' might be an bug. It is not.
			for (int32 j = i; j < NumNodes; j++) //-V791 
			{
				UpperKernel.Add(WeightFunc(Nodes[i], Nodes[j]));
			}
		}

		bIsValid = SetUpperKernel(UpperKernel, NumNodes);
	}


	TArrayView<T> Nodes;
	WeightFuncT WeightFunc;
};
