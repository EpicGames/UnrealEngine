// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimTypes.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "CoreMinimal.h"
#include "Curves/RichCurve.h"
#include "Math/Quat.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"

#include "RBFSolver.generated.h"


/** The solver type to use. The two solvers have different requirements. */
UENUM()
enum class ERBFSolverType : uint8
{
	/** The additive solver sums up contributions from each target. It's faster
	    but may require more targets for a good coverage, and requires the 
		normalization step to be performed for smooth results.
	*/
	Additive,

	/** The interpolative solver interpolates the values from each target based
		on distance. As long as the input values are within the area bounded by
		the targets, the interpolation is well-behaved and return weight values 
		within the 0% - 100% limit with no normalization required. 
		Interpolation also gives smoother results, with fewer targets, than additive
		but at a higher computational cost.
	*/
	Interpolative
};

/** Function to use for each target falloff */
UENUM()
enum class ERBFFunctionType : uint8
{
	Gaussian,

	Exponential,

	Linear,

	Cubic,

	Quintic,

	/** Uses the setting of the parent container */
	DefaultFunction
};

/** Method for determining distance from input to targets */
UENUM()
enum class ERBFDistanceMethod : uint8
{
	/** Standard n-dimensional distance measure */
	Euclidean,

	/** Treat inputs as quaternion */
	Quaternion,

	/** Treat inputs as quaternion, and find distance between rotated TwistAxis direction */
	SwingAngle,

	/** Treat inputs as quaternion, and find distance between rotations around the TwistAxis direction */
	TwistAngle,

	/** Uses the setting of the parent container */
	DefaultMethod
};

/** Method to normalize weights */
UENUM()
enum class ERBFNormalizeMethod : uint8
{
	/** Only normalize above one */
	OnlyNormalizeAboveOne,

	/** 
		Always normalize. 
		Zero distribution weights stay zero.
	*/
	AlwaysNormalize,

	/** 
		Normalize only within reference median. The median
		is a cone with a minimum and maximum angle within
		which the value will be interpolated between 
		non-normalized and normalized. This helps to define
		the volume in which normalization is always required.
	*/
	NormalizeWithinMedian,

	/** 
		Don't normalize at all. This should only be used with
		the interpolative method, if it is known that all input
		values will be within the area bounded by the targets.
	*/
	NoNormalization,

};

/** Struct storing a particular entry within the RBF */
USTRUCT()
struct FRBFEntry
{
	GENERATED_BODY()

	/** Set of values for this target, size must be TargetDimensions  */
	UPROPERTY(EditAnywhere, Category = RBFData)
	TArray<float> Values;

	/** Return a target as an rotator, assuming Values is a sequence of Euler entries. Index is which Euler to convert.*/
	ANIMGRAPHRUNTIME_API FRotator AsRotator(int32 Index) const;

	/** Return a target as a quaternion, assuming Values is a sequence of Euler entries. Index is which Euler to convert. */
	ANIMGRAPHRUNTIME_API FQuat AsQuat(int32 Index) const;

	ANIMGRAPHRUNTIME_API FVector AsVector(int32 Index) const;


	/** Set this entry to 3 floats from supplied rotator */
	ANIMGRAPHRUNTIME_API void AddFromRotator(const FRotator& InRot);
	/** Set this entry to 3 floats from supplied vector */
	ANIMGRAPHRUNTIME_API void AddFromVector(const FVector& InVector);

	/** Return dimensionality of this target */
	int32 GetDimensions() const
	{
		return Values.Num();
	}
};

/** Data about a particular target in the RBF, including scaling factor */
USTRUCT()
struct FRBFTarget : public FRBFEntry
{
	GENERATED_BODY()

	/** How large the influence of this target. */
	UPROPERTY(EditAnywhere, Category = RBFData)
	float ScaleFactor;

	/** Whether we want to apply an additional custom curve when activating this target. 
	    Ignored if the solver type is Interpolative. 
	*/
	UPROPERTY(EditAnywhere, Category = RBFData)
	bool bApplyCustomCurve;

	/** Custom curve to apply to activation of this target, if bApplyCustomCurve is true.
		Ignored if the solver type is Interpolative. */
	UPROPERTY(EditAnywhere, Category = RBFData)
	FRichCurve CustomCurve;

	/** Override the distance method used to calculate the distance from this target to
		the input. Ignored if the solver type is Interpolative. */
	UPROPERTY(EditAnywhere, Category = RBFData)
	ERBFDistanceMethod DistanceMethod;

	/** Override the falloff function used to smooth the distance from this target to
		the input. Ignored if the solver type is Interpolative. */
	UPROPERTY(EditAnywhere, Category = RBFData)
	ERBFFunctionType FunctionType;

	FRBFTarget()
		: ScaleFactor(1.f)
		, bApplyCustomCurve(false)
		, DistanceMethod(ERBFDistanceMethod::DefaultMethod)
		, FunctionType(ERBFFunctionType::DefaultFunction)
	{}
};

/** Struct for storing RBF results - target index and corresponding weight */
struct FRBFOutputWeight
{
	/** Index of target */
	int32 TargetIndex;
	/** Weight of target */
	float TargetWeight;

	FRBFOutputWeight(int32 InTargetIndex, float InTargetWeight)
		: TargetIndex(InTargetIndex)
		, TargetWeight(InTargetWeight)
	{}

	FRBFOutputWeight()
		: TargetIndex(0)
		, TargetWeight(0.f)
	{}
};

/** Parameters used by RBF solver */
USTRUCT(BlueprintType)
struct FRBFParams
{
	GENERATED_BODY()

	/** How many dimensions input data has */
	UPROPERTY()
	int32 TargetDimensions;

	/** Specifies the type of solver to use. The additive solver requires normalization, for the
		most part, whereas the Interpolative solver is not as reliant on it. The interpolative
		solver also has smoother blending, whereas the additive solver requires more targets but
		has a more precise control over the influence of each target.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RBFData)
	ERBFSolverType SolverType;

	/** Default radius for each target. 
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RBFData, meta = (EditCondition = "!bAutomaticRadius"))
	float Radius;

	/* Automatically pick the radius based on the average distance between targets */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RBFData)
	bool bAutomaticRadius;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RBFData)
	ERBFFunctionType Function;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RBFData)
	ERBFDistanceMethod DistanceMethod;

	/** Axis to use when DistanceMethod is SwingAngle */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RBFData)
	TEnumAsByte<EBoneAxis> TwistAxis;

	/** Weight below which we shouldn't bother returning a contribution from a target */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RBFData)
	float WeightThreshold;

	/** Method to use for normalizing the weight */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RBFData)
	ERBFNormalizeMethod NormalizeMethod;

	/** Rotation or position of median (used for normalization) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RBFData, meta = (EditCondition = "NormalizeMethod == ERBFNormalizeMethod::NormalizeWithinMedian"))
	FVector MedianReference;

	/** Minimum distance used for median */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RBFData, meta = (UIMin = "0", UIMax = "90", EditCondition = "NormalizeMethod == ERBFNormalizeMethod::NormalizeWithinMedian"))
	float MedianMin;

	/** Maximum distance used for median */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RBFData, meta = (UIMin = "0", UIMax = "90", EditCondition = "NormalizeMethod == ERBFNormalizeMethod::NormalizeWithinMedian"))
	float MedianMax;

	ANIMGRAPHRUNTIME_API FRBFParams();

	/** Util for returning unit direction vector for swing axis */
	ANIMGRAPHRUNTIME_API FVector GetTwistAxisVector() const;
};

struct ANIMGRAPHRUNTIME_API FRBFSolverData;

/** Library of Radial Basis Function solver functions */
struct FRBFSolver
{
	/** Given a list of targets, verify which ones are valid for solving the RBF setup. This is mostly about removing identical targets
		which invalidates the interpolative solver. Returns true if all targets are valid. */
	static ANIMGRAPHRUNTIME_API bool ValidateTargets(const FRBFParams& Params, const TArray<FRBFTarget>& Targets, TArray<int>& InvalidTargets);

	/** Given a set of targets and new input entry, give list of activated targets with weights */
	static ANIMGRAPHRUNTIME_API TSharedPtr<const FRBFSolverData> InitSolver(const FRBFParams& Params, const TArray<FRBFTarget>& Targets);

	static ANIMGRAPHRUNTIME_API bool IsSolverDataValid(const FRBFSolverData& SolverData, const FRBFParams& Params, const TArray<FRBFTarget>& Targets);

	/** Given a set of targets and new input entry, give list of activated targets with weights */
	static ANIMGRAPHRUNTIME_API void Solve(const FRBFSolverData& SolverData, const FRBFParams& Params, const TArray<FRBFTarget>& Targets, const FRBFEntry& Input, TArray<FRBFOutputWeight>& OutputWeights);

	/** Util to find distance to nearest neighbour target for each target */
	static ANIMGRAPHRUNTIME_API bool FindTargetNeighbourDistances(const FRBFParams& Params, const TArray<FRBFTarget>& Targets, TArray<float>& NeighbourDists);

	/** Util to find distance between two entries, using provided params */
	static ANIMGRAPHRUNTIME_API float FindDistanceBetweenEntries(const FRBFEntry& A, const FRBFEntry& B, const FRBFParams& Params, ERBFDistanceMethod OverrideMethod = ERBFDistanceMethod::DefaultMethod);

	/** Returns the radius for a given target */
	static ANIMGRAPHRUNTIME_API float GetRadiusForTarget(const FRBFTarget& Target, const FRBFParams& Params);

	/** Compute the optimal radius for the given targets. Returns the radius */
	static ANIMGRAPHRUNTIME_API float GetOptimalRadiusForTargets(const FRBFParams& Params, const TArray<FRBFTarget>& Targets);
};
