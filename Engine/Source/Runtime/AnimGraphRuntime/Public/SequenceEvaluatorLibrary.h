// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Animation/AnimExecutionContext.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimNodeReference.h"
#include "SequenceEvaluatorLibrary.generated.h"

struct FAnimNode_SequenceEvaluator;

USTRUCT(BlueprintType)
struct FSequenceEvaluatorReference : public FAnimNodeReference
{
	GENERATED_BODY()

	typedef FAnimNode_SequenceEvaluator FInternalNodeType;
};

// Exposes operations to be performed on a sequence evaluator anim node
// Note: Experimental and subject to change!
UCLASS(Experimental, MinimalAPI)
class USequenceEvaluatorLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Get a sequence evaluator context from an anim node context */
	UFUNCTION(BlueprintCallable, Category = "Animation|Sequences", meta=(BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static ANIMGRAPHRUNTIME_API FSequenceEvaluatorReference ConvertToSequenceEvaluator(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result);

	/** Get a sequence evaluator context from an anim node context (pure) */
	UFUNCTION(BlueprintPure, Category = "Animation|Sequences", meta=(BlueprintThreadSafe, DisplayName = "Convert to Sequence Evaluator"))
	static void ConvertToSequenceEvaluatorPure(const FAnimNodeReference& Node, FSequenceEvaluatorReference& SequenceEvaluator, bool& Result)
	{
		EAnimNodeReferenceConversionResult ConversionResult;
		SequenceEvaluator = ConvertToSequenceEvaluator(Node, ConversionResult);
		Result = (ConversionResult == EAnimNodeReferenceConversionResult::Succeeded);	
	}
	
	/** Set the current accumulated time of the sequence evaluator */
	UFUNCTION(BlueprintCallable, Category = "Animation|Sequences", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FSequenceEvaluatorReference SetExplicitTime(const FSequenceEvaluatorReference& SequenceEvaluator, float Time);

	/** Set the current accumulated time, using a frame number, of the sequence evaluator */
	UFUNCTION(BlueprintCallable, Category = "Animation|Sequences", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FSequenceEvaluatorReference SetExplicitFrame(const FSequenceEvaluatorReference& SequenceEvaluator, int32 Frame);

	/** Advance the current accumulated time of the sequence evaluator */
	UFUNCTION(BlueprintCallable, Category = "Animation|Sequences", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FSequenceEvaluatorReference AdvanceTime(const FAnimUpdateContext& UpdateContext, const FSequenceEvaluatorReference& SequenceEvaluator, float PlayRate = 1.0f);

	/** Set the current sequence of the sequence evaluator */
	UFUNCTION(BlueprintCallable, Category = "Animation|Sequences", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FSequenceEvaluatorReference SetSequence(const FSequenceEvaluatorReference& SequenceEvaluator, UAnimSequenceBase* Sequence);

	/** Set the current sequence of the sequence evaluator with an inertial blend time */
	UFUNCTION(BlueprintCallable, Category = "Animation|Sequences", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FSequenceEvaluatorReference SetSequenceWithInertialBlending(const FAnimUpdateContext& UpdateContext, const FSequenceEvaluatorReference& SequenceEvaluator, UAnimSequenceBase* Sequence, float BlendTime = 0.2f);
	
	/** Gets the current accumulated time of the sequence evaluator */
	UFUNCTION(BlueprintPure, Category = "Sequence Evaluator", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API float GetAccumulatedTime(const FSequenceEvaluatorReference& SequenceEvaluator);

	/** Gets the current sequence of the sequence evaluator */
	UFUNCTION(BlueprintPure, Category = "Sequence Evaluator", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API UAnimSequenceBase* GetSequence(const FSequenceEvaluatorReference& SequenceEvaluator);
};
