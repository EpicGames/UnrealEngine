// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Animation/AnimExecutionContext.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimNodeReference.h"
#include "SequencePlayerLibrary.generated.h"

struct FAnimNode_SequencePlayer;

USTRUCT(BlueprintType)
struct FSequencePlayerReference : public FAnimNodeReference
{
	GENERATED_BODY()

	typedef FAnimNode_SequencePlayer FInternalNodeType;
};

// Exposes operations to be performed on a sequence player anim node
// Note: Experimental and subject to change!
UCLASS(Experimental, MinimalAPI)
class USequencePlayerLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Get a sequence player context from an anim node context */
	UFUNCTION(BlueprintCallable, Category = "Animation|Sequences", meta = (BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static ANIMGRAPHRUNTIME_API FSequencePlayerReference ConvertToSequencePlayer(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result);

	/** Get a sequence player context from an anim node context (pure) */
	UFUNCTION(BlueprintPure, Category = "Animation|Sequences", meta = (BlueprintThreadSafe, DisplayName = "Convert to Sequence Player"))
	static void ConvertToSequencePlayerPure(const FAnimNodeReference& Node, FSequencePlayerReference& SequencePlayer, bool& Result)
	{
		EAnimNodeReferenceConversionResult ConversionResult;
		SequencePlayer = ConvertToSequencePlayer(Node, ConversionResult);
		Result = (ConversionResult == EAnimNodeReferenceConversionResult::Succeeded);
	}

	/** Set the current accumulated time of the sequence player */
	UFUNCTION(BlueprintCallable, Category = "Animation|Sequences", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FSequencePlayerReference SetAccumulatedTime(const FSequencePlayerReference& SequencePlayer, float Time);

	/** 
	 * Set the start position of the sequence player. 
	 * If this is called from On Become Relevant or On Initial Update then it should be accompanied by a call to
	 * SetAccumulatedTime to achieve the desired effect of resetting the play time of a sequence player.
	 */
	UFUNCTION(BlueprintCallable, Category = "Animation|Sequences", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FSequencePlayerReference SetStartPosition(const FSequencePlayerReference& SequencePlayer, float StartPosition);

	/** Set the play rate of the sequence player */
	UFUNCTION(BlueprintCallable, Category = "Animation|Sequences", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FSequencePlayerReference SetPlayRate(const FSequencePlayerReference& SequencePlayer, float PlayRate);

	/** Set the current sequence of the sequence player */
	UFUNCTION(BlueprintCallable, Category = "Animation|Sequences", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FSequencePlayerReference SetSequence(const FSequencePlayerReference& SequencePlayer, UAnimSequenceBase* Sequence);

	/** Set the current sequence of the sequence player with an inertial blend time */
	UFUNCTION(BlueprintCallable, Category = "Animation|Sequences", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FSequencePlayerReference SetSequenceWithInertialBlending(const FAnimUpdateContext& UpdateContext, const FSequencePlayerReference& SequencePlayer, UAnimSequenceBase* Sequence, float BlendTime = 0.2f);

	/** Get the current sequence of the sequence player - DEPRECATED, please use pure version */
	UFUNCTION(BlueprintCallable, Category = "Animation|Sequences", meta = (BlueprintThreadSafe, DeprecatedFunction))
	static ANIMGRAPHRUNTIME_API FSequencePlayerReference GetSequence(const FSequencePlayerReference& SequencePlayer, UPARAM(Ref) UAnimSequenceBase*& SequenceBase);
	
	/** Get the current sequence of the sequence player */
	UFUNCTION(BlueprintPure, Category = "Animation|Sequences", meta = (BlueprintThreadSafe, DisplayName = "Get Sequence"))
	static ANIMGRAPHRUNTIME_API UAnimSequenceBase* GetSequencePure(const FSequencePlayerReference& SequencePlayer);

	/** Gets the current accumulated time of the sequence player */
	UFUNCTION(BlueprintPure, Category = "Animation|Sequences", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API float GetAccumulatedTime(const FSequencePlayerReference& SequencePlayer);

	/** Get the start position of the sequence player */
	UFUNCTION(BlueprintPure, Category = "Animation|Sequences", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API float GetStartPosition(const FSequencePlayerReference& SequencePlayer);

	/** Get the play rate of the sequence player */
	UFUNCTION(BlueprintPure, Category = "Animation|Sequences", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API float GetPlayRate(const FSequencePlayerReference& SequencePlayer);

	/** Get the looping state of the sequence player */
	UFUNCTION(BlueprintPure, Category = "Animation|Sequences", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API bool GetLoopAnimation(const FSequencePlayerReference& SequencePlayer);

	/** Returns the Play Rate to provide when playing this animation if a specific animation duration is desired */
	UFUNCTION(BlueprintPure, Category = "Animation|Sequences", meta = (BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API float ComputePlayRateFromDuration(const FSequencePlayerReference& SequencePlayer, float Duration = 1.f);
};
