// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Sequencer Animation Track Support interface - this is required for animation track to work
 */

#pragma once
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UObject/Interface.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SequencerAnimationSupport.generated.h"

class UAnimInstance;
class UAnimSequenceBase;
class UObject;

UINTERFACE(meta = (CannotImplementInterfaceInBlueprint), MinimalAPI)
class USequencerAnimationSupport : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class ISequencerAnimationSupport
{
	GENERATED_IINTERFACE_BODY()

	/** Source Animation Getter for the support of the Sequencer Animation Track interface */
	virtual UAnimInstance* GetSourceAnimInstance() = 0;
	virtual void SetSourceAnimInstance(UAnimInstance* SourceAnimInstance) = 0;
	virtual bool DoesSupportDifferentSourceAnimInstance() const = 0;
	/** Update an animation sequence player in this instance */
	virtual void UpdateAnimTrack(UAnimSequenceBase* InAnimSequence, int32 SequenceId, float InPosition, float Weight, bool bFireNotifies) = 0;
	virtual void UpdateAnimTrack(UAnimSequenceBase* InAnimSequence, int32 SequenceId, float InFromPosition, float InToPosition, float Weight, bool bFireNotifies) = 0;

	/** Construct all nodes in this instance */
	virtual void ConstructNodes() = 0;

	/** Reset all nodes in this instance */
	virtual void ResetNodes() = 0;

	/** Reset the pose for this instance*/
	virtual void ResetPose() = 0;

	/** Saved the named pose to restore after */
	virtual void SavePose() = 0;
};

