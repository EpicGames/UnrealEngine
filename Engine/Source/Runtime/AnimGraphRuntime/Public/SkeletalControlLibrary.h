// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimNodeReference.h"
#include "SkeletalControlLibrary.generated.h"

struct FAnimNode_SkeletalControlBase;

USTRUCT(BlueprintType)
struct FSkeletalControlReference : public FAnimNodeReference
{
	GENERATED_BODY()

	typedef FAnimNode_SkeletalControlBase FInternalNodeType;
};

// Exposes operations to be performed on a skeletal control anim node
// Note: Experimental and subject to change!
UCLASS(Experimental, MinimalAPI)
class USkeletalControlLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Get a skeletal control from an anim node */
	UFUNCTION(BlueprintCallable, Category = "Animation|Skeletal Controls", meta=(BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static ANIMGRAPHRUNTIME_API FSkeletalControlReference ConvertToSkeletalControl(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result);

	/** Get a skeletal control from an anim node (pure) */
	UFUNCTION(BlueprintPure, Category = "Animation|Skeletal Controls", meta=(BlueprintThreadSafe, DisplayName = "Convert to Skeletal Control"))
	static void ConvertToSkeletalControlPure(const FAnimNodeReference& Node, FSkeletalControlReference& SkeletalControl, bool& Result)
	{
		EAnimNodeReferenceConversionResult ConversionResult;
		SkeletalControl = ConvertToSkeletalControl(Node, ConversionResult);
		Result = (ConversionResult == EAnimNodeReferenceConversionResult::Succeeded);
	}
	
	/** Set the alpha value of this skeletal control */
	UFUNCTION(BlueprintCallable, Category = "Animation|Skeletal Controls", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API FSkeletalControlReference SetAlpha(const FSkeletalControlReference& SkeletalControl, float Alpha);

	/** Get the alpha value of this skeletal control */
	UFUNCTION(BlueprintPure, Category = "Animation|Skeletal Controls", meta=(BlueprintThreadSafe))
	static ANIMGRAPHRUNTIME_API float GetAlpha(const FSkeletalControlReference& SkeletalControl);
};
